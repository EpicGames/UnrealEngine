// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Diagnostics;
using CSVStats;

namespace CSVTools
{
    class Version
    {
        private static string VersionString = "1.35";

        public static string Get() { return VersionString; }
    };

    class Program : CommandLineTool
    {
        static string formatString =
            "Format: \n" +
            "       -csvs <filename or ; separated list>  -csvDir <path>\n" +
			"       [-searchPattern <pattern, e.g *.csv>] - for use with -csvDir\n" +
			"       [-avg] -stats will be per frame averaged\n" +
			"       [-recurse] - for use with -csvdir\n" +
			"       [-filterOutlierStat <stat>] - discard CSVs if this stat has very high values\n" +
			"       [-filterOutlierThreshold <value>] - threshold for outliers (default:1000)\n" +
			"       [-metadataFilter <key=value,key=value...>] : filters based on CSV metadata\n" +
			"       [-startEvent <event name>] : starts from a particular event\n" +
			"       -o <csvFilename> \n";

		void Run(string[] args)
        {
            // Read the command line
            if (args.Length < 1)
            {
                WriteLine("Invalid args");
                WriteLine(formatString);
                return;
            }

            ReadCommandLine(args);

            string csvOutFilename = GetArg("o", false);
            if (csvOutFilename.Length == 0)
            {
                WriteLine("Missing -o arg");
                WriteLine(formatString);
                return;
            }
            
            // Set the title
            string title = GetArg("title", false);
            if (title.Length == 0)
            {
                title = MakeShortFilename(csvOutFilename).ToLower().Replace(".csv", "");
            }
            char c = title[0];
            c = char.ToUpper(c);
            title = c + title.Substring(1);

			string filterOutlierStat = GetArg("filterOutlierStat", false);
			float filterOutlierThreshold = GetFloatArg("filterOutlierThreshold", 1000.0f);

			// Whether or not we want stats to be averaged rather than appended
			bool bAverage = GetBoolArg("avg");

			string csvDir = GetArg("csvDir");
			string csvFilenamesStr = GetArg("csvs", false);

			if ( csvDir == null && csvFilenamesStr == null )
			{
				WriteLine("-csvs or -csvdir is required");
				WriteLine(formatString);
				return;
			}

			string searchPattern = GetArg("searchPattern", null);
			if (csvFilenamesStr.Contains("*"))
			{
				// If passed a wildcard to -csvs, this is equivalent to -csvdir . -searchpattern <csvs>
				if (csvDir != null && csvDir != "")
				{
					throw new Exception("Can't use -csvs with -csvdir");
				}
				csvDir = ".";
				searchPattern = csvFilenamesStr;
			}

			// Read CSV filenames from a directory or list
			string[] csvFilenames;
            if (csvDir.Length > 0)
            {
				if (searchPattern == null)
				{
					searchPattern = "*.csv";
				}
				DirectoryInfo di = new DirectoryInfo(csvDir);
				bool bRecurse = GetBoolArg("recurse");
				var files = di.GetFiles(searchPattern, bRecurse ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
                csvFilenames = new string[files.Length];
                int i = 0;
                foreach (FileInfo csvFile in files)
                {
                    csvFilenames[i] = csvFile.FullName;
                    i++;
                }
            }
            else
            {
                if (csvFilenamesStr.Length == 0)
                {
                    System.Console.Write(formatString);
                    return;
                }
                csvFilenames = csvFilenamesStr.Split(';');
            }

			Console.WriteLine("Collating " + csvFilenames.Length + " csvs:");
			foreach (string csvFilename in csvFilenames)
			{
				Console.WriteLine("  "+csvFilename);
			}
			Console.WriteLine("");

			CsvStats combinedCsvStats = new CsvStats();

			string startEventName = GetArg("startEvent", null);
			string metadataFilterString = GetArg("metadataFilter", null);
			List<int> frameCsvCounts=new List<int>();
			List<string> allCsvFilenames = new List<string>();
			int csvIndex = 0;
            foreach (string csvFilename in csvFilenames)
            {
                CsvStats srcCsvStats = CsvStats.ReadCSVFile(csvFilename, null);

				if (startEventName != null)
				{
					int startFrame = 0;
					foreach (CsvEvent ev in srcCsvStats.Events)
					{
						if (CsvStats.DoesSearchStringMatch(ev.Name, startEventName))
						{
							startFrame = ev.Frame;
							break;
						}
					}
					if (startFrame != 0 )
					{
						srcCsvStats.CropStats(startFrame);
					}
				}

				// Check for outliers
				bool skip = false;
				if ( filterOutlierStat != null )
				{
					StatSamples outlierStat = srcCsvStats.GetStat(filterOutlierStat);
					if ( outlierStat != null )
					{
						foreach (float sample in outlierStat.samples)
						{
							if (sample>filterOutlierThreshold)
							{
								WriteLine("CSV " + csvFilename + " ignored due to bad " + filterOutlierStat + " value: " + sample);
								skip = true;
								break;
							}
						}
					}
				}
				if (metadataFilterString != null)
				{
					if (srcCsvStats.metaData == null || !CsvStats.DoesMetadataMatchFilter(srcCsvStats.metaData, metadataFilterString))
					{
						WriteLine("Skipping CSV " + csvFilename + " due to metadata filter");
						skip = true;
					}
				}
				if ( skip )
				{
					continue;
				}

				// Add the CSV filename as the first event if we're not averaging
				if (!bAverage)
				{
					CsvEvent firstEvent = new CsvEvent();
					firstEvent.Frame = 0;
					firstEvent.Name = "CSV:" + MakeShortFilename(csvFilename).Replace(' ', '_').Replace(',', '_').Replace('\n', '_');
					srcCsvStats.Events.Insert(0, firstEvent);
				}

				// Combine the stats
				if (csvIndex == 0)
				{
					combinedCsvStats = srcCsvStats;
				}
				else
				{
					CsvMetadata metadataA = combinedCsvStats.metaData;
					CsvMetadata metadataB = srcCsvStats.metaData;

					// If there is metadata, it should match
					if (metadataA != null || metadataB != null)
					{
						metadataA.CombineAndValidate(metadataB);
					}
					combinedCsvStats.Combine(srcCsvStats, bAverage, false);
				}

				// If we're computing the average, update the counts for each frame
				if (bAverage)
				{
					// Resize frameCsvCounts if necessary
					for (int i = frameCsvCounts.Count; i < combinedCsvStats.SampleCount; i++)
					{
						frameCsvCounts.Add(0);
					}
					for (int i = 0; i < srcCsvStats.SampleCount; i++)
					{
						frameCsvCounts[i] += 1;
					}
				}

				allCsvFilenames.Add(Path.GetFileName(csvFilename));
				csvIndex++;
				WriteLine("Csvs Processed: " + csvIndex + " / " + csvFilenames.Length);
			}


			if (bAverage)
			{
				// Divide all samples by the total number of CSVs 
				foreach (StatSamples stat in combinedCsvStats.Stats.Values)
				{
					for (int i=0; i<stat.samples.Count;i++)
					{
						stat.samples[i] /= (float)(frameCsvCounts[i]);
					}
				}

				// Add a stat for the csv count
				string csvCountStatName = "csvCount";
				if (!combinedCsvStats.Stats.ContainsKey(csvCountStatName))
				{
					StatSamples csvCountStat = new StatSamples(csvCountStatName);
					foreach (int count in frameCsvCounts)
					{
						csvCountStat.samples.Add((int)count);
					}
					combinedCsvStats.Stats.Add(csvCountStatName, csvCountStat);
				}
				if (combinedCsvStats.metaData != null)
				{
					// Add some metadata
					combinedCsvStats.metaData.Values.Add("Averaged", allCsvFilenames.Count.ToString());
					combinedCsvStats.metaData.Values.Add("SourceFiles", string.Join(";", allCsvFilenames));
				}
			}

			combinedCsvStats.ComputeAveragesAndTotal();

			// Write the csv stats to a CSV
			combinedCsvStats.WriteToCSV(csvOutFilename);
        }


        static void Main(string[] args)
        {
            Program program = new Program();
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.Error.WriteLine("[ERROR] " + e.Message);
                }
            }
        }

    }
}
