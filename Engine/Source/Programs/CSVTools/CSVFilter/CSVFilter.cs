// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Diagnostics;
using CSVStats;

namespace CSVTools
{
    class Version
    {
        private static string VersionString = "1.02";

        public static string Get() { return VersionString; }
    };

    class Program : CommandLineTool
    {
        static string formatString =
            "Format: \n" +
            "       -csv <filename> \n"+
            "       -stats <stat names> AND/OR -defaults (specifies a default stat list)\n" +
            "       -o <csvFilename> \n";


        private bool IsStatIncluded(string statName, string[] statList)
        {
            string lowerName = statName.ToLower();
            foreach ( string stat in statList )
            {
                string lowerStat = stat.ToLower();
                if (lowerStat.ToLower() == lowerName )
                {
                    return true;
                }
                if (lowerStat.EndsWith("*"))
                {
                    int index = stat.LastIndexOf('*');
                    string prefix = stat.Substring(0, index);
                    if ( lowerName.StartsWith(prefix) )
                    {
                        return true;
                    }
                }
            }
            return false;
        }
            //foreach (string showEventName in showEventNames)
            //{
            //    string showEventNameLower = showEventName.ToLower();
            //    if (showEventNameLower.EndsWith("*"))
            //    {
            //        int index = showEventNameLower.LastIndexOf('*');
            //        string prefix = showEventNameLower.Substring(0, index);
            //        if (eventString.StartsWith(prefix))
            //        {
            //            return true;
            //        }
            //    }
            //    else if (eventString == showEventNameLower)
            //    {
            //        return true;
            //    }
            //}
            //return false;
        


        void Run(string[] args)
        {
            // Read the command line
            if (args.Length < 1)
            {
                WriteLine("Invalid args");
                WriteLine(formatString);
                return;
            }

            ReadCommandLine(args);

            string csvOutFilename = GetArg("o", false);
            if (csvOutFilename.Length == 0)
            {
                WriteLine("Missing -o arg");
                WriteLine(formatString);
                return;
            }


            string csvInFilename = GetArg("csv", false);
            if (csvInFilename.Length == 0)
            {
                WriteLine("Missing -csv arg");
                WriteLine(formatString);
                return;
            }

            bool bDefaults = GetIntArg("defaults",0) != 0;
            string statList = GetArg("stats", false);
            if (statList.Length == 0 && !bDefaults)
            {
                WriteLine("Missing -stats arg");
                WriteLine(formatString);
                return;
            }


            if ( bDefaults )
            {
                if ( statList.Length > 0 )
                {
                    statList += ",";
                }
                statList += "frame,gpu,renderthread,gamethread,framenumber,gpu/basepass,events";
            }

            string [] filteredStatList = statList.Split(',');

            // Read the file line by line and filter the stats
            // We don't read as a CSV as this will run out of memory with very large files
            System.IO.StreamReader file = new System.IO.StreamReader(csvInFilename);
            string headerLine = file.ReadLine();
            string [] csvStats = headerLine.Split(',');

            bool [] columnIncludedMap = new bool[csvStats.Length];

            List<string> filteredStatNames = new List<string>();

            System.IO.StreamWriter fileOut = new System.IO.StreamWriter(csvOutFilename);
            for ( int i=0; i<csvStats.Length; i++ )
            {
                if (IsStatIncluded(csvStats[i], filteredStatList))
                {
                    columnIncludedMap[i] = true;
                    filteredStatNames.Add(csvStats[i]);
                }
                else
                {
                    columnIncludedMap[i] = false;
                }
            }

            // Write the header out
            StringBuilder headerLineOut = new StringBuilder();
            for ( int i=0; i<filteredStatNames.Count; i++ )
            {
                headerLineOut.Append(filteredStatNames[i]);
                if ( i<filteredStatNames.Count-1 )
                {
                    headerLineOut.Append(",");
                }
            }
            fileOut.WriteLine(headerLineOut.ToString());

            // Read the file and write out the stats
            string line;
            while ((line = file.ReadLine()) != null)
            {
                string [] values = line.Split(',');
                List<string> filteredValuesOut = new List<string>();
                if (values.Length != columnIncludedMap.Length)
                {
                    // If the count doesn't match, just write out the line (e.g. metadata)
                    fileOut.WriteLine(line);
                }
                else
                {
                    for (int i = 0; i < values.Length; i++)
                    {
                        if (columnIncludedMap[i])
                        {
                            filteredValuesOut.Add(values[i]);
                        }
                    }
                    StringBuilder lineOut = new StringBuilder();
                    for (int i = 0; i < filteredValuesOut.Count; i++)
                    {
                        lineOut.Append(filteredValuesOut[i]);
                        if (i < filteredValuesOut.Count - 1)
                        {
                            lineOut.Append(",");
                        }
                    }
                    fileOut.WriteLine(lineOut.ToString());
                }
            }


        }


        static void Main(string[] args)
        {
            Program program = new Program();
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.Error.WriteLine("[ERROR] " + e.Message);
                }
            }
        }

    }
}
