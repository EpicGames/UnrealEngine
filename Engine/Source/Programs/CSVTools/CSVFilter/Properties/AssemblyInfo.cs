// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("352b7c21-e47c-44d0-969a-bc0e68530df6")]
