// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Diagnostics;
using CSVStats;

namespace CSVTools
{
    class Version
    {
        private static string VersionString = "1.33";

        public static string Get() { return VersionString; }
    };

    class Program : CommandLineTool
    {
        static string formatString =
            "Format: \n" +
            "       -csv <filename>\n" +
            "       -splitStat <statname>" +
            "       [ -o <csvFilename> ] \n" +
            "       [ -delay <frame count> ] \n" +
            "       [ -virtualEvents ] \n";

        void Run(string[] args)
        {
            // Read the command line
            if (args.Length < 1)
            {
                WriteLine("Invalid args");
                WriteLine(formatString);
                return;
            }

            ReadCommandLine(args);

            string csvFilename = GetArg("csv", true);
            if (csvFilename.Length == 0)
            {
                System.Console.Write(formatString);
                return;
            }

            string splitStat = GetArg("splitStat").ToLower(); // We write a CSV out for every value of this that we encounter
            if (splitStat.Length == 0)
            {
                System.Console.Write(formatString);
                return;
            }

            string csvOutFilename = GetArg("o", false);
            if (csvOutFilename.Length == 0)
            {
                int dotIndex = csvFilename.LastIndexOf('.');
                csvOutFilename = csvFilename.Substring(0, dotIndex);
            }

            int delay = GetIntArg("delay", 0);  // We delay by this many frames
            bool bVirtualEvents = GetBoolArg("virtualEvents");

            // Read the CSV
            CsvStats csvStats = CsvStats.ReadCSVFile(csvFilename, null);

            List<CsvStats> csvStatsOutList = new List<CsvStats>();
            List<float> uniqueValues = new List<float>();
            Dictionary<float, int> switchValueToCsvIndex = new Dictionary<float,int>();
            StatSamples [] statSamples = csvStats.Stats.Values.ToArray();
            StatSamples switchStatSamples = csvStats.GetStat(splitStat);
            if ( switchStatSamples == null )
            {
                System.Console.WriteLine("Split stat '" + switchStatSamples + "' not found");
                return;
            }

            StatSamples[] sourceStats = csvStats.Stats.Values.ToArray();

            List<int> SourceFrameToDestFrameIndex = new List<int>();
            List<int> SourceFrameToDestCsvIndex = new List<int>();

			for (int i = 0; i < delay; i++)
			{
				SourceFrameToDestFrameIndex.Add(0);
				SourceFrameToDestCsvIndex.Add(0);
			}

			for (int i = delay; i < switchStatSamples.samples.Count; i++)
            {
                float switchValue = switchStatSamples.samples[i - delay];

                // Find the CSV stats for this value, or create it if it doesn't exist
                CsvStats destStats;
                if (!switchValueToCsvIndex.ContainsKey(switchValue))
                {
                    destStats = new CsvStats();

                    // Create a new CsvStats as a copy of csvStats but with no samples
                    destStats.metaData = csvStats.metaData;
                    foreach (StatSamples stat in sourceStats)
                    {
                        StatSamples newStat = new StatSamples(stat,false);
                        destStats.AddStat(newStat);
                    }
                    switchValueToCsvIndex.Add(switchValue, csvStatsOutList.Count);
                    csvStatsOutList.Add(destStats);
                    uniqueValues.Add(switchValue);
                }
                else
                {
                    destStats = csvStatsOutList[switchValueToCsvIndex[switchValue]];
                }

                // Add the samples to the current destination
                int index = 0;
                foreach (StatSamples destStat in destStats.Stats.Values.ToArray())
                {
                    if (i < sourceStats[index].samples.Count)
                    {
                        destStat.samples.Add(sourceStats[index].samples[i]);
                    }
                    index++;
                }

                int DestCsvIndex = switchValueToCsvIndex[switchValue];
                int DestFrameIndex = destStats.SampleCount - 1;

                SourceFrameToDestFrameIndex.Add(DestFrameIndex);
                SourceFrameToDestCsvIndex.Add(DestCsvIndex);
            }


            for (int i = 0; i < csvStats.Events.Count; i++)
            {
                CsvEvent ev = csvStats.Events[i];
                int eventFrame = ev.Frame - delay;
                if (eventFrame>0)
                {
                    int destCsvIndex = SourceFrameToDestCsvIndex[eventFrame];
                    CsvStats destStats = csvStatsOutList[destCsvIndex];
                    CsvEvent newEvent = new CsvEvent(ev.Name, SourceFrameToDestFrameIndex[ev.Frame], ev.Timestamp);
                    destStats.Events.Add(newEvent);

                    if (bVirtualEvents)
                    {
                        // Write the event to the other CSVs with [VIRTUAL]
                        for (int j = 0; j < csvStatsOutList.Count; j++)
                        {
                            if (j != destCsvIndex)
                            {
                                CsvEvent newEvent2 = new CsvEvent(ev.Name + " [VIRTUAL]", newEvent.Frame, ev.Timestamp);
                                destStats.Events.Add(newEvent2);
                            }
                        }
                    }

                }
            }

            for (int i = 0; i < csvStatsOutList.Count;i++ )
            {
                // Write the csv stats to a CSV
                csvStatsOutList[i].WriteToCSV(csvOutFilename + "_" + splitStat + "_" + uniqueValues[i] + ".csv");
            }
        }


        static void Main(string[] args)
        {
            Program program = new Program();
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.Error.WriteLine("[ERROR] " + e.Message);
                }
            }
        }

    }
}
