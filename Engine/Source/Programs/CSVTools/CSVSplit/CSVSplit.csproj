<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AssemblyTitle>CSVSplit</AssemblyTitle>
    <Product>CSVSplit</Product>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
</Project>