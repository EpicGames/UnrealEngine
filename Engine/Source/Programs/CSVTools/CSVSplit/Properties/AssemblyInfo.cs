// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("a7c8e49c-6a8a-40f4-9dfd-8135acf7bf98")]
