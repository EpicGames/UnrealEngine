<?xml version="1.0" encoding="utf-8"?>
<configuration>
    <startup> 
        <supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.8"/>
    </startup>
</configuration>
