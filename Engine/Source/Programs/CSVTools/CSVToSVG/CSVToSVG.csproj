<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <RuntimeIdentifiers>win-x64;osx-x64;win-arm64;osx-arm64;linux-x64;linux-arm64</RuntimeIdentifiers>
  </PropertyGroup>
  <PropertyGroup>
    <StartupObject>
    </StartupObject>
    <AssemblyTitle>CSVToSVG</AssemblyTitle>
    <Product>CSVToSVG</Product>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
    <ProjectReference Include="..\CsvToSvgLib\CsvToSvgLib.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
</Project>