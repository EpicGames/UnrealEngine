// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Diagnostics;
using System.Security.Cryptography;
using CSVStats;

namespace CSVTools
{
	class CsvCache
	{
		public CsvStats ReadCSVFile(string csvFilename, string[] statNames, int numRowsToSkip)
		{
			lock (readLock)
			{
				if (csvFilename != filename || skipRows != numRowsToSkip)
				{
					csvStats = CsvStats.ReadCSVFile(csvFilename, null, numRowsToSkip);
					filename = csvFilename;
					skipRows = numRowsToSkip;
				}
				// Copy just the stats we're interested in from the cached file
				CsvStats outStats = new CsvStats(csvStats, statNames);
				return outStats;
			}
		}
		public CsvStats csvStats;
		public string filename;
		public int skipRows;
		private readonly object readLock = new object();
	};


	class Program : CSVStats.CommandLineTool
    {
		static CsvCache csvCache = null;

		static string formatString =
			"Format: \n" +
			"       -csvs <list> OR -csv <list> OR -csvDir <path>\n" +
			"       [ -o <svgFilename> ]\n" +
			"       -stats <stat names> (can include wildcards)\n" +
			"     OR \n" +
			"       -batchCommands <response file with commandlines>\n" +
			"       -mt <number of threads> \n" +
			"     OR \n" +
			"       -updatesvg <svgFilename>\n" +
			"       NOTE: this updates an svg by regenerating it with the original commandline parameters - requires original data\n\n" +
			"\nOptional Args: \n" +
			"       -averageThreshold <value>\n" +
			"       -budget <ms>\n" +
			"       -colourOffset <value>\n" +
			"       -compression <pixel error value>\n" +
			"       -discardLastFrame <1|0> (default 1)\n" +
			"       -filterOutZeros\n" +
			"       -fixedPointGraphs 1|0 (default 1 - smaller graphs but no subpixel accuracy)\n" +
			"       -fixedPointPrecisionScale <1..N> - scale for fixed point graph rendering (>1 gives subpixel accuracy)" +
			"       -graphOnly\n" +
			"       -hideEventNames <1|0>\n" +
			"       -showAllEventNames <1|0> - don't filter out event names if they're too dense\n" +
			"       -hideStatPrefix <list>\n" +
			"       -hierarchySeparator <character>\n" +
			"       -highlightEventRegions <startEventName0;endEventName0;...>\n" +
			"       -ignoreStats <list> (can include wildcards. Separate states with stat1;stat2;etc)\n" +
			"       -interactive\n" +
			"       -legend <list> \n" +
			"       -maxHierarchyDepth <depth>\n" +
			"       -minX <value> -maxX <value> -minY <value> -maxY <value>\n" +
			"       -startEvent <name>\n" +
			"       -startEventOffset <value>\n" +
			"       -endEvent <name>\n" +
			"       -endEventOffset <value>\n" +
			"       -maxAutoMaxY <value> - clamp automatic maxY to this\n" +
			"       -noMetadata\n" +
			"       -noSnap\n" +
			"       -recurse\n" +
			"       -showAverages \n" +
			"       -showEvents <names> (can include wildcards)\n" +
			"       -showTotals \n" +
			"       -skipRows <n>\n" +
			"       -smooth\n" +
			"       -smoothKernelPercent <percentage>\n" +
			"       -smoothKernelSize <numFrames>\n" +
			"       -smoothMultithreaded <1|0>\n" +
			"       -stacked\n" +
			"       -stackTotalStat <stat name>\n" +
			"		-minFilterStatValue <value>\n" +
			"		-minFilterStatName <stat name>\n" +
			"       -stackedUnsorted\n" +
			"       -statMultiplier <multiplier>\n" +
			"       -theme <dark|light>\n" +
			"       -thickness <multipler>\n" +
			"       -threshold <value>\n" +
			"       -title <name>\n" +
			"       -forceLegendSort\n" +
			"       -width <value> -height <value>\n" +
			"       -writeErrorsToSVG\n" +
			"       -percentile \n" +
			"       -percentile90 \n" +
			"       -percentile99 \n" +
			"       -uniqueId <string> : unique ID for JS (needed if this is getting embedded in HTML alongside other graphs)\n" +
			"       -nocommandlineEmbed : don't embed the commandline in the SVG\n" +
			"       -lineDecimalPlaces <N> (only effective with with -fixedPointGraphs 0. Default=3)\n" +
			"       -frameOffset <N> : offset used for frame display name (default 0)\n" +
			"       -csvColors <list> : list of hex colors. Applied in order to corresponding csvs when used with a list of csv files or csvDir\n" +
			"		-view : open the graph in the default viewer\n" + 
			"";

		void Run(string[] args)
		{
			System.Globalization.CultureInfo.DefaultThreadCurrentCulture = System.Globalization.CultureInfo.InvariantCulture;

			if (args.Length < 2)
			{
				WriteLine("CsvToSVG " + CsvToSvgLibVersion.Get());
				WriteLine(formatString);
				return;
			}

			// Read the command line
			ReadCommandLine(args);

			string svgToUpdate = GetArg("updatesvg");
			if (svgToUpdate.Length > 0)
			{
				if (args.Length > 2)
				{
					WriteLine("-UpdateSVG <svgFilename> must be the only argument!");
					return;
				}
				string newCommandLine = "";
				string[] svgLines = ReadLinesFromFile(svgToUpdate);
				for (int i = 0; i < svgLines.Length - 2; i++)
				{
					if (svgLines[i].StartsWith("<![CDATA[") && svgLines[i + 1].StartsWith("Created with CSVtoSVG ") && svgLines[i + 1].EndsWith(" with commandline:"))
					{
						newCommandLine = svgLines[i + 2];
						break;
					}
				}

				ReadCommandLine(MakeArgsArray(newCommandLine));
			}

			MakeGraph();
		}

		string GenerateIDFromString(string str)
		{
			string id = "";
			HashAlgorithm algorithm = SHA256.Create();
			byte[] hash = algorithm.ComputeHash(Encoding.UTF8.GetBytes(commandLine.GetCommandLine()));
			for ( int i=24; i<32; i++ )
			{
				id += hash[i].ToString("x2");
			}
			return id;
		}


		void MakeGraph()
		{
			// Read CSV filenames from a directory or list
			string[] csvFilenames;
			string csvDir = GetArg("csvDir");
			if (csvDir.Length > 0)
			{
				DirectoryInfo di = new DirectoryInfo(csvDir);
				bool recurse = GetBoolArg("recurse");
				FileInfo[] csvFiles = di.GetFiles("*.csv", recurse ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
				FileInfo[] binFiles = di.GetFiles("*.csv.bin", recurse ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
				List<FileInfo> allFiles = new List<FileInfo>(csvFiles);
				allFiles.AddRange(binFiles);
				csvFilenames = new string[allFiles.Count];
				int i = 0;
				foreach (FileInfo csvFile in allFiles)
				{
					csvFilenames[i] = csvFile.FullName;
					i++;
				}
			}
			else
			{
				string csvFilenamesStr = GetArg("csv");
				if (csvFilenamesStr.Length == 0)
				{
					csvFilenamesStr = GetArg("csvs", true);
					if (csvFilenamesStr.Length == 0)
					{
						System.Console.Write(formatString);
						return;
					}
				}
				csvFilenames = csvFilenamesStr.Split(';');
			}

			GraphParams graphParams = getGraphParamsFromArgs();

			if (graphParams.statNames.Count == 0)
			{
				System.Console.Write(formatString);
				return;
			}

			if (graphParams.statNames.Count > 1 && csvFilenames.Length > 1)
			{
				WriteLine("Can't display multiple stats and multiple CSVs");
				return;
			}

			// Figure out the filename, if it wasn't provided
			string svgFilename = GetArg("o", false);
			if (svgFilename.Length == 0)
			{
				if (csvFilenames.Length == 1)
				{
					int index = csvFilenames[0].LastIndexOf('.');
					if (index >= 0)
					{
						svgFilename = csvFilenames[0].Substring(0, index);
					}
					else
					{
						svgFilename = csvFilenames[0];
					}
				}
				if (graphParams.statNames.Count == 1)
				{
					if (svgFilename.Length > 0)
					{
						svgFilename += "_";
					}
					string statName = graphParams.statNames[0];
					// If this is a stat expression and we've defined the name via the = operator then use that
					int idx = statName.IndexOf('=');
					if (idx > 0)
					{
						statName = statName[..idx];
					}
					svgFilename += statName.Replace('*', 'X').Replace('/', '_').Replace("(", "").Replace(")", "");
				}
				svgFilename += ".svg";
			}
			if (svgFilename.Length == 0)
			{
				WriteLine("Couldn't figure out an appropriate filename, and no filename was provided with -o");
			}

			// Load the CSVs
			List<CsvInfo> csvs = new List<CsvInfo>();
			foreach (string csvFilename in csvFilenames)
			{
				CsvStats csvStats = LoadCsv(csvFilename, graphParams.statNames.ToArray());
				csvs.Add(new CsvInfo(csvStats, csvFilename));
			}

			// Read graph params from the commandline
			GraphGenerator generator = new GraphGenerator(csvs);

			bool writeErrorsToSVG = GetIntArg("writeErrorsToSVG", 1) == 1;
			if (Debugger.IsAttached && !GetBoolArg("catchExceptions"))
			{
				generator.MakeGraph(graphParams, svgFilename, writeErrorsToSVG);
			}
			else
			{
				try
				{
					generator.MakeGraph(graphParams, svgFilename, writeErrorsToSVG);
				}
				catch (System.Exception e)
				{
					// Write the error to the SVG
					string errorString = e.ToString();
					Console.Out.WriteLine("Error: " + e.ToString());
				}
			}
			if (GetBoolArg("view"))
			{
				ViewGraph(System.IO.Path.GetFullPath(svgFilename));
			}
		}

		static string[] MakeArgsArray(string argsStr)
        {
            List<string> argsOut = new List<string>();
            string currentArg = "";
            bool bInQuotes = false;
            for ( int i=0; i< argsStr.Length;i++ )
            {
                bool flush = false;
                char c = argsStr[i];
                if ( c == '"')
                {
                    if ( bInQuotes )
                    {
                        flush = true;
                    }
                    bInQuotes = !bInQuotes;
                }
                else if ( c==' ' )
                {
                    if ( bInQuotes )
                    {
                        currentArg += c;
                    }
                    else
                    {
                        flush = true;
                    }
                }
                else
                {
                    currentArg += c;
                }

                if (flush && currentArg.Length > 0)
                {
                    argsOut.Add(currentArg);
                    currentArg = "";
                }
            }
			if (currentArg.Length > 0)
			{
				argsOut.Add(currentArg);
			}
			return argsOut.ToArray();
        }

		void ViewGraph(string svgPath)
		{
			if (!OperatingSystem.IsWindows())
			{
				Console.WriteLine("View graph is not currently supported on this platform");
				return;
			}
			System.Diagnostics.Process process = new System.Diagnostics.Process();
			System.Diagnostics.ProcessStartInfo startInfo = new System.Diagnostics.ProcessStartInfo();
			startInfo.WindowStyle = System.Diagnostics.ProcessWindowStyle.Hidden;
			startInfo.FileName = "explorer.exe";
			startInfo.Arguments = svgPath;
			process.StartInfo = startInfo;
			process.Start();
		}

		GraphParams getGraphParamsFromArgs()
		{
			GraphParams graphParams = new GraphParams();

			graphParams.width = GetIntArg("width", graphParams.width);
			graphParams.height = GetIntArg("height", graphParams.height);
			graphParams.title = GetArg("title", graphParams.title);
			graphParams.statNames = GetListArg("stats", ';', false, true);
			graphParams.ignoreStats = GetListArg("ignoreStats");
			graphParams.themeName = GetArg("theme").ToLower();
			graphParams.colorOffset = GetIntArg("colorOffset", GetIntArg("colourOffset", 0) );
			graphParams.csvColors = GetListArg("csvColors");
			if(graphParams.csvColors.Count == 0)
			{
				graphParams.csvColors = GetListArg("csvColours");
			}

			// Events
			graphParams.showEventNames = GetListArg("showEvents");
			if ( GetIntArg("hideEventNames", 0) == 1 )
			{
				graphParams.showEventNameTextMode = ShowEventTextMode.Hide;
			}
			else if (GetIntArg("showAllEventNames", 0) == 1)
			{
				graphParams.showEventNameTextMode = ShowEventTextMode.ShowAll;
			}
			graphParams.highlightEventRegions = GetListArg("highlightEventRegions");

			// Start/end event
			graphParams.startEvent = GetArg("startEvent", null);
			graphParams.startEventOffset = GetIntArg("startEventOffset", 0);
			graphParams.endEvent = GetArg("endEvent", null);
			graphParams.endEventOffset = GetIntArg("endEventOffset", 0);

			// Smoothing
			graphParams.smooth = GetArg("smooth") == "1";
			graphParams.smoothKernelPercent = GetFloatArg("smoothKernelPercent", graphParams.smoothKernelPercent);
			graphParams.smoothKernelSize = GetIntArg("smoothKernelSize", graphParams.smoothKernelSize);

			// Other flags
			graphParams.showMetadata = GetArg("nometadata") != "1";
			graphParams.graphOnly = GetArg("graphOnly") == "1";
			graphParams.interactive = GetArg("interactive") == "1";
			graphParams.snapToPeaks = !GetBoolArg("noSnap");

			// Compression
			graphParams.compression = GetFloatArg("compression", graphParams.compression);
			graphParams.bFixedPointGraphs = GetIntArg("fixedPointGraphs", graphParams.bFixedPointGraphs?1:0) == 1;
			graphParams.fixedPointPrecisionScale = GetFloatArg("fixedPointPrecisionScale", graphParams.fixedPointPrecisionScale);
			graphParams.lineDecimalPlaces=GetIntArg("lineDecimalPlaces", graphParams.lineDecimalPlaces);

			// Sometimes the last frame is garbage. We might want to remove it
			graphParams.discardLastFrame = (GetIntArg("discardLastFrame", graphParams.discardLastFrame?1:0) == 1);

			graphParams.maxHierarchyDepth = GetIntArg("maxHierarchyDepth", graphParams.maxHierarchyDepth);

			string hierarchySeparatorStr = GetArg("hierarchySeparator","");
			if (hierarchySeparatorStr.Length > 0)
			{
				graphParams.hierarchySeparator = hierarchySeparatorStr[0];
			}
			graphParams.hideStatPrefixes = GetListArg("hideStatPrefix", ';', true);

			// Stacking
			graphParams.stacked = GetArg("stacked") == "1";
			if (graphParams.stacked)
			{
				graphParams.stackTotalStat = GetArg("stackTotalStat").ToLower();
			}
			graphParams.stackedUnsorted = GetArg("stackedUnsorted") == "1";

			// Percentiles
			graphParams.percentileTop90 = GetArg("percentile90") == "1";
			graphParams.percentileTop99 = GetArg("percentile99") == "1";
			graphParams.percentile = GetArg("percentile") == "1";

			// X/Y range
			graphParams.minX = GetFloatArg("minx", graphParams.minX);
			graphParams.maxX = GetFloatArg("maxx", graphParams.maxX);
			graphParams.minY = GetFloatArg("miny", graphParams.minY);
			graphParams.maxY = GetFloatArg("maxy", graphParams.maxY);
			graphParams.maxAutoMaxY = GetFloatArg("maxAutoMaxY", graphParams.maxAutoMaxY);

			graphParams.budget = GetFloatArg("budget", graphParams.budget);
			graphParams.lineThickness = GetFloatArg("thickness", graphParams.lineThickness);

			// Thresholds
			graphParams.threshold = GetFloatArg("threshold", graphParams.threshold);
			graphParams.averageThreshold = GetFloatArg("averageThreshold", graphParams.averageThreshold);

			graphParams.embedText = "Created with CSVtoSVG " + CsvToSvgLibVersion.Get();
			if (!GetBoolArg("nocommandlineEmbed"))
			{
				graphParams.embedText += " with commandline:\n" + commandLine.GetCommandLine();
			}
			graphParams.uniqueId = GetArg("uniqueID", graphParams.uniqueId);

			// Legend
			graphParams.showAverages = GetBoolArg("showAverages");
			graphParams.showTotals = GetBoolArg("showTotals");
			graphParams.forceLegendSort = GetBoolArg("forceLegendSort");
			graphParams.legendAverageThreshold = GetFloatArg("legendAverageThreshold", graphParams.legendAverageThreshold);

			graphParams.customLegendNames = GetListArg("legend");

			// Process options (don't affect output)
			string smoothMultiThreadedStr = GetArg("smoothMultithreaded",null);
			if (smoothMultiThreadedStr != null)
			{
				graphParams.bSmoothMultithreaded = smoothMultiThreadedStr == "1";
			}
			graphParams.bPerfLog = GetBoolArg("perfLog");

			// Misc params (rarely used)
			graphParams.frameOffset = GetIntArg("frameOffset", graphParams.frameOffset);
			graphParams.statMultiplier = GetFloatArg("statMultiplier", graphParams.statMultiplier);

			graphParams.minFilterStatName = GetArg("minFilterStatName", null);
			if (graphParams.minFilterStatName != null)
			{
				graphParams.minFilterStatValue = GetFloatArg("minFilterStatValue", -float.MaxValue);
			}
			graphParams.filterOutZeros = GetBoolArg("filterOutZeros");

			graphParams.FinalizeSettings();
			return graphParams;
		}

		CsvStats LoadCsv(string csvFilename, string[] statNames)
        {
            int numRowsToSkip = GetIntArg("skipRows",0);

			// If we have derived stat expressions then load all stats
			bool bHasDerivedStatExpression = false;
			foreach (string stat in statNames)
			{
				if ( CsvStats.IsDerivedStatExpression(stat) )
				{
					bHasDerivedStatExpression = true; 
					break;
				}
			}

			// Read the file from the cache if we have one
			CsvStats csvStats = null;
			if (csvCache != null)
			{
				csvStats = csvCache.ReadCSVFile(csvFilename, bHasDerivedStatExpression ? null : statNames, numRowsToSkip);
			}
			else
			{
				csvStats = CsvStats.ReadCSVFile(csvFilename, bHasDerivedStatExpression ? null : statNames, numRowsToSkip);
			}
            return csvStats;
        }

		static void Main(string[] args)
        {
			CommandLine commandline = new CommandLine(args);
			string batchCommandsFilename = commandline.GetArg("batchCommands");

			if (batchCommandsFilename.Length > 0)
			{
				// In batch mode, just output a list of commandlines
				string[] commandlines = System.IO.File.ReadAllLines(batchCommandsFilename);
				if (commandlines.Length > 1)
				{
					// Enable caching
					csvCache = new CsvCache();
				}
				int numThreads = commandline.GetIntArg("mt", 4);
				if (numThreads>1)
				{
					var result = Parallel.For(0, commandlines.Length, new ParallelOptions { MaxDegreeOfParallelism = numThreads }, i =>
					{
						Program program = new Program();
						program.Run(MakeArgsArray(commandlines[i]));
					});
				}
				else
				{
					foreach (string cmdLine in commandlines)
					{
						Program program = new Program();
						program.Run(MakeArgsArray(cmdLine));
					}
				}
			}
			else
			{
				Program program = new Program();
				try
				{
					program.Run(args);
				}
				catch (System.Exception e)
				{
					Console.Error.WriteLine("[ERROR] " + e.Message);
					if (Debugger.IsAttached)
					{
						throw;
					}
				}
			}
        }
     }
}
