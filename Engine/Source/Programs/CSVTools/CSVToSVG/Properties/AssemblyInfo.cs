// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("dc6aed41-e787-4807-a139-52b9557567d4")]
