<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121. 
-->
<Project ToolsVersion="4.0" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <PublishProtocol>FileSystem</PublishProtocol>
    <Configuration>Release</Configuration>
    <TargetFramework>netcoreapp2.0</TargetFramework>
    <PublishDir>bin\Release\PublishOutput\win-x64</PublishDir>
    <RuntimeIdentifier>win-x64</RuntimeIdentifier>
  </PropertyGroup>
</Project>