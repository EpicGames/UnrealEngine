// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CSVToTelemetry : ModuleRules
{
	public CSVToTelemetry(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(new string[] {
			"AnalyticsET",
			"ApplicationCore",
			"AssetRegistry",
			"BuildSettings",
			"Core",
			"CoreUObject",
			"CSVUtils",
			"Horde",
			"HTTP",
			"Projects",
			"RSA",
			"StudioTelemetry",
			"TargetPlatform"
		});
	}
}
