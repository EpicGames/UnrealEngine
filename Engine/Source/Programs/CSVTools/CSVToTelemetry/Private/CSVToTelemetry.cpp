// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSVToTelemetry.h"
#include "CSVProfilerUtils.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Analytics.h"
#include "AnalyticsET.h"
#include "StudioTelemetry.h"
#include "Logging/LogMacros.h"
#include "Serialization/Csv/CsvParser.h"
#include "ProjectUtilities.h"


DEFINE_LOG_CATEGORY_STATIC(LogCSVToTelemetry, Log, All);
IMPLEMENT_APPLICATION(CSVToTelemetry, "CSVToTelemetry");

FORCEINLINE static bool TryNormalizeBooleanString(FString& InString)
{
	if (InString.IsEmpty())
	{
		return false;
	}

	if (InString.Equals(TEXT("true"), ESearchCase::IgnoreCase) || InString.Equals(TEXT("false"), ESearchCase::IgnoreCase))
	{
		InString = InString.ToLower();
		return true;
	}

	return false;
}

static int GenerateTelemetryFromCSVFile(const FString& FilePath)
{
	// Assume the result will be an error state, non-zero, and only returns zero when fully successful.
	int32 Result = 1;

	if (FilePath.Len() > 0)
	{
		FString EventName;

		if (FParse::Value(FCommandLine::Get(), TEXT("event="), EventName) == false)
		{
			UE_LOGF(LogCSVToTelemetry, Error, "Must provide a Event name with -event=[eventname]");
			return Result;
		}

		bool Validate = false;

		if (FParse::Param(FCommandLine::Get(), TEXT("validate")))
		{
			Validate = true;
			UE_LOGF(LogCSVToTelemetry, Error, "Validating CSV file %ls, events will not be sent", *FilePath);
		}

		STUDIO_TELEMETRY_SESSION_SCOPE

		// Start the telemetry session and record each row as a single telemetry event 
		if (FStudioTelemetry::Get().IsSessionRunning())
		{
			FString CSVString;

			// Load all the file into memory
			if (FFileHelper::LoadFileToString(CSVString, *FilePath))
			{	
				// Parse the file using the CSV Parser
				FCsvParser Parser(CSVString);
				const FCsvParser::FRows& Rows = Parser.GetRows();

				if ( Rows.Num()==0 )
				{
					// The CSV file failed to parse
					UE_LOGF(LogCSVToTelemetry, Warning, "Failed to import rows from file %ls", *FilePath);
					return -1;
				}
				else
				{
					UE_LOGF(LogCSVToTelemetry, Display, "Imported %d rows from file %ls", Rows.Num(), *FilePath);
				}
				
				uint32 TotalUploadSize = 0;
				uint32 TotalEventsSent = 0;
				uint32 KeySize = 0;
				uint32 ValueSize = 0;
				uint32 MaxRows = Rows.Num();
				TArray<FString> KeyArray;
				TArray<FAnalyticsEventAttribute> AdditionalAttributes;

				bool NormalizeBooleanStrings = false;
				int32 SchemaVersion(-1);

				if (FParse::Value(FCommandLine::Get(), TEXT("schema="), SchemaVersion) == true)
				{
					// Add our schema version
					AdditionalAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
				}

				if (FParse::Param(FCommandLine::Get(), TEXT("normalize-bool-strings")))
				{
					NormalizeBooleanStrings = true;
				}

				FParse::Value(FCommandLine::Get(), TEXT("maxrows="), MaxRows);
				
				FString ColumnsString;
				if (FParse::Value(FCommandLine::Get(), TEXT("columns="), ColumnsString) == true)
				{
					// We have specified the CSV columns on the command line; persist empty values to guarantee strict alignment in rows.
					ColumnsString.ParseIntoArray(KeyArray, TEXT("|"), false);
					KeySize = ColumnsString.NumBytesWithoutNull();
				}
				
				FString AttributesString;
				if (FParse::Value(FCommandLine::Get(), TEXT("attributes="), AttributesString) == true)
				{
					// Parse a list of additional attributes to pass to the each event
					TArray<FString> AttributesArray;
					AttributesString.ParseIntoArray(AttributesArray, TEXT("|"), false);

					if (AttributesArray.Num() > 0)
					{
						for (int32 AttributeIndex = 0; AttributeIndex < AttributesArray.Num(); ++AttributeIndex)
						{
							TArray<FString> KeyValuePairArray;
							AttributesArray[AttributeIndex].ParseIntoArray(KeyValuePairArray, TEXT("="), false);

							if (KeyValuePairArray.Num() == 2)
							{
								AdditionalAttributes.Emplace(KeyValuePairArray[0], KeyValuePairArray[1]);
							}
							else
							{
								UE_LOGF(LogCSVToTelemetry, Error, "Invalid additional attributes \"%ls\". They must be a valid Key Value Pair Key=Value", *AttributesArray[AttributeIndex]);
								return Result;
							}
						}

						UE_LOGF(LogCSVToTelemetry, Display, "Appending %d additional attributes", AdditionalAttributes.Num());
					}
					else
					{
						UE_LOGF(LogCSVToTelemetry, Warning, "No additional attributes found in %ls", *AttributesString);
					}
				}

				uint32 RowIndex = 0;

				if (KeyArray.IsEmpty())
				{
					// The user has not specified any keys so assume these are on the first row
					const TArray<const TCHAR*>& Row = Rows[RowIndex];

					for (const TCHAR* Column : Row)
					{
						FString ColumnString(Column);
						KeyArray.Push(ColumnString.TrimStartAndEnd());
					}

					KeySize = Row.NumBytes();

					// Increment the RowIndex and the MaxRows as we have consumed the fist row for the keys  
					RowIndex++;
					MaxRows++;
				}

				// Validate that we have a valid set of keys. These are either specified on the first row of the csv file or explicitly via the -columns= commandline
				if (KeyArray.IsEmpty())
				{
					// No keys we found at all
					UE_LOGF(LogCSVToTelemetry, Error, "No keys are specified for csv file %ls", *FilePath);
					return Result;
				}
				else
				{
					// Keys were found but validate that any are not empty
					for (const FString& Key : KeyArray)
					{
						if (Key.IsEmpty())
						{
							// Keys can't ever be empty
							UE_LOGF(LogCSVToTelemetry, Error, "Empty key specifed in csv file %ls", *FilePath);
							return Result;
						}
					}
				}

				// Limit the maximum number of rows or choose the full set
				MaxRows = FMath::Min<uint32>(Rows.Num(), MaxRows);
				
				for ( ; RowIndex < MaxRows; ++RowIndex )
				{
					const TArray<const TCHAR*>& Row = Rows[RowIndex];

					TArray<FString> ValueArray;

					// Parse the values from subsequent rows; persist empty values to match columns.
					for (const TCHAR* Column : Row)
					{
						ValueArray.Push(Column);
					}

					// Keys and values count should always be the same
					if (KeyArray.Num() == ValueArray.Num())
					{
						ValueSize = Row.NumBytes();

						TArray<FAnalyticsEventAttribute> Attributes = AdditionalAttributes;

						// Add the Key/Value pair to the attribute list
						for (int32 Column = 0; Column < ValueArray.Num(); ++Column)
						{
							FString Value = ValueArray[Column].TrimStartAndEnd();
						
							// Normalize boolean strings
							if (NormalizeBooleanStrings)
							{
								TryNormalizeBooleanString(Value);
							}

							Attributes.Emplace(KeyArray[Column], Value);
						}

						// Keep track of the total data we're uploading
						TotalUploadSize += KeySize + ValueSize;

						// Now we have a complete event so send it
						if (Validate == false)
						{
							FStudioTelemetry::Get().RecordEvent(EventName, Attributes);
						}
						
						// Keep track of the event count
						TotalEventsSent++;
					}
					else
					{
						// The key/value counts don't match	
						UE_LOGF(LogCSVToTelemetry, Warning, "Row %d contains an incorrect number of values %d ( expected %d ) and will be skipped.", RowIndex, ValueArray.Num(), KeyArray.Num());
					}
				}

				UE_LOGF(LogCSVToTelemetry, Display, "Generated %d bytes of event data", TotalUploadSize);
				UE_LOGF(LogCSVToTelemetry, Display, "Uploading %d %ls event(s)...", TotalEventsSent, *EventName);

				// Everything worked as expected return code should be zero
				Result = 0;
			}
			else
			{
				UE_LOGF(LogCSVToTelemetry, Error, "Unable to read rows from CSV file %ls", *FilePath);
			}
		}
		else
		{
			UE_LOGF(LogCSVToTelemetry, Error, "Unable to start a telemetry session");
		}
	}

	return Result;
}

static int GenerateTelemetryFromCSVProfileFile(const FString& FilePath)
{
	// Assume the result will be an error state, non-zero, and only returns zero when fully successful.
	int32 Result = 1;

	FString EventName;

	if (FParse::Value(FCommandLine::Get(), TEXT("event="), EventName) == false)
	{
		UE_LOGF(LogCSVToTelemetry, Error, "Must provide a Event name with -event=[eventname]");
		return Result;
	}

	CsvUtils::FCsvProfilerCapture Capture;

	if (FilePath.Len() > 0)
	{
		if (FilePath.Contains(".bin"))
		{
			// Binary CSV file
			CsvUtils::ReadFromCsvBin(Capture, *FilePath);
		}
		else
		{
			// Text CSV file
			CsvUtils::ReadFromCsv(Capture, *FilePath);
		}
	}

	if (Capture.Events.Num())
	{
		FStudioTelemetry::Get().StartSession();

		if (FStudioTelemetry::Get().IsSessionRunning())
		{
			TSharedPtr<IAnalyticsProvider> Provider = FStudioTelemetry::Get().GetProvider().Pin();
			TArray<FAnalyticsEventAttribute> DefaultAttributes = Provider->GetDefaultEventAttributesSafe();

			for (TMap<FString, FString>::TConstIterator It(Capture.Metadata); It; ++It)
			{
				DefaultAttributes.Emplace(It->Key, It->Value);
			}

			Provider->SetDefaultEventAttributes(MoveTemp(DefaultAttributes));

			for (const CsvUtils::FCsvProfilerEvent& Event : Capture.Events)
			{
				int32 Frame = Event.Frame;

				TArray<FAnalyticsEventAttribute> Attributes;

				Attributes.Emplace(TEXT("Name"), Event.Name);

				for (TMap<FString, CsvUtils::FCsvProfilerSample>::TConstIterator It(Capture.Samples); It; ++It)
				{
					const FString& Name = It->Key;
					const CsvUtils::FCsvProfilerSample& Sample = It->Value;

					if (Frame < Sample.Values.Num())
					{
						Attributes.Emplace(Name, Sample.Values[Frame]);
					}
				}

				FStudioTelemetry::Get().RecordEvent(EventName, Attributes);
			}

			FStudioTelemetry::Get().FlushEvents();
			FStudioTelemetry::Get().EndSession();

			Result = 0;
		}
	}

	return Result;
}

static int ShowHelp()
{
	UE_LOGF(LogCSVToTelemetry, Display, "\n\nCSVToTelemetry Help\n\nUsage:\n\tCSVToTelemetry.exe -csv=[filename] -event=[eventname] ( -schema=[value] -columns=[name1|name2|....] -attributes=[key1=value1|key2=value2|....] -normalize-bool-strings -maxrows=[value] )\n\tCSVToTelemetry.exe -csvprofile=[filename] -event=[eventname]\n\tCSVToTelemetry.exe -help\n\nRequired:\n\t-csv=[filename]\t\t\tGeneric text based csv input file.\n\t-csvprofile=[filename]\t\tCSVProfiler csv input file. Denote binary with .csv.bin otherwise assumed text.\n\t-event=[name]\t\t\tName of telemetry event to send each row to.\nOptional:\n\t-schema=[value]\t\t\tEvent schema value.\n\t-columns=[name1|name2|....]\tSpecify column names.\n\t-normalize-bool-strings\t\tNormalizes any (\"True\"|\"False\"|\"TRUE\"|\"FALSE\") strings to lowercase.\n\t-maxrows=[value]\t\tLimits the number of rows sent.");
	return 0;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	FDateTime StartTime = FDateTime::UtcNow();

	// Allows this program to accept a project argument on the command-line and use project-specific config
	UE::ProjectUtilities::ParseProjectDirFromCommandline(ArgC, ArgV);

	// start up the main loop,
	GEngineLoop.PreInit(ArgC, ArgV);

	int32 Result = 1;
	const float Version = 1.0;

	UE_LOGF(LogCSVToTelemetry, Display, "CSVToTelemetry v%.1f", Version);

	FString FilePath;

	if (FParse::Param(FCommandLine::Get(), TEXT("help")) == true)
	{
		// Show help message
		Result = ShowHelp();
	}
	else
	{
		if (FParse::Value(FCommandLine::Get(), TEXT("csvprofile="), FilePath) == true)
		{
			Result = GenerateTelemetryFromCSVProfileFile(FilePath);
		}
		else if (FParse::Value(FCommandLine::Get(), TEXT("csv="), FilePath) == true)
		{
			Result = GenerateTelemetryFromCSVFile(FilePath);
		}

		if (Result == 0)
		{
			// Upload completed successfully
			UE_LOGF(LogCSVToTelemetry, Display, "CSVToTelemetry upload succeeded in %0.2f seconds", (FDateTime::UtcNow() - StartTime).GetTotalSeconds());
		}
	}

	if (Result != 0)
	{
		// Always show the usage example if we have not been successful
		UE_LOGF(LogCSVToTelemetry, Error, "\nUsage:\n\tCSVToTelemetry.exe -csv=[filename] -event=[eventname] ( -schema=[value] -columns=[name1|name2|....] -attributes=[key1=value1|key2=value2|....] )\n\tCSVToTelemetry.exe -csvprofile=[filename] -event=[eventname]\n\tCSVToTelemetry.exe -help");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("fastexit")))
	{
		FPlatformMisc::RequestExitWithStatus(true, Result);
	}

	GLog->Flush();

	RequestEngineExit(TEXT("CSVToTelemetry Exiting"));

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return Result;
}