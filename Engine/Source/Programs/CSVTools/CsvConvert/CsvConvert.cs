// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Diagnostics;
using System.IO.Compression;
using CSVStats;

namespace CSVInfo
{
    class Version
    {
        private static string VersionString = "1.06";

        public static string Get() { return VersionString; }
    };

    class Program : CommandLineTool
    {
        static int Main(string[] args)
        {
            Program program = new Program();
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.Error.WriteLine("[ERROR] " + e.Message);
                    return 1;
                }
            }

            return 0;
        }

		static bool isCsvFile(string filename)
		{
			return filename.ToLowerInvariant().EndsWith(".csv");
		}
		static bool isCsvBinFile(string filename)
		{
			return filename.ToLowerInvariant().EndsWith(".csv.bin");
		}

		void Run(string[] args)
        {
			string formatString =
				"Format: \n" +
				"  <filename>\n" +
				"OR \n" +
				"  -in <filename>\n" +
				"  -outFormat <csv|bin|csvNoMetadata>\n" +
				"  [-binCompress <0|1|2>] (0=none)\n" +
				"  [-o outFilename]\n" +
				"  [-force] - overwrite even if output file exists already\n" +
				"  [-inPlace] - overwrite the input file (implies -force)\n" +
				"  [-verify]\n" +
				"  [-dontFixMissingID] - doesn't generate a CsvID if it's not found in the metadata\n" +
				"  [-skipIfMetadataMissing] - don't set metadata if the csv doesn't have any.\n" +
				"  [-setMetadata key0=value0;key1=value1;...]\n";


			// Read the command line
			if (args.Length < 1)
            {
                WriteLine(formatString);
                return;
            }

			bool bInPlace = false;
            ReadCommandLine(args);

			string inFilename;
			string outFormat;
			string outFilename = null;
			if (args.Length==1)
			{
				inFilename = args[0];
				// Determine the output format automatically
				if (isCsvBinFile(inFilename))
				{
					outFormat = "csv";
				}
				else if (isCsvFile(inFilename))
				{
					outFormat = "bin";
				}
				else
				{
					throw new Exception("Unknown input format!");
				}
			}
			else
			{
				// Advanced mode
				inFilename = GetArg("in",true);
				outFormat = GetArg("outformat", true);
				if (inFilename == "")
				{
					throw new Exception("Need to specify an input filename");
				}
				outFilename = GetArg("o", GetArg("out", null));
				bInPlace = GetBoolArg("inPlace");
				if (bInPlace)
				{
					if (outFilename != null)
					{
						throw new Exception("Output filename specified with -inPlace");
					}
					outFilename = inFilename;
				}
			}

			string inFilenameWithoutExtension;
			if (isCsvBinFile(inFilename))
			{
				inFilenameWithoutExtension = inFilename.Substring(0, inFilename.Length - 8);
			}
			else if (isCsvFile(inFilename))
			{
				inFilenameWithoutExtension = inFilename.Substring(0, inFilename.Length - 4);
			}
			else
			{
				throw new Exception("Unexpected input filename extension!");
			}

			Console.WriteLine("Converting "+inFilename+" to "+outFormat+" format.");
			Console.WriteLine("Reading input file...");

			bool bFixMissingCsvId = !GetBoolArg("dontFixMissingID");
			CsvFileInfo inFileInfo = new CsvFileInfo();
			CsvStats csvStats = CsvStats.ReadCSVFile(inFilename, null, 0, bFixMissingCsvId, inFileInfo);

			// If output format is not set, determine it automatically
			if (outFormat.Length == 0)
			{
				outFormat = inFileInfo.bIsCsvBin ? "bin" : "csv";
			}

			// Figure out the output format
			outFormat = outFormat.ToLower();
			bool bBinOut = false;
			bool bWriteMetadata = true;
			if (outFormat == "bin")
			{
				bBinOut = true;
			}
			else if (outFormat == "csv")
			{
				bBinOut = false;
			}
			else if (outFormat == "csvnometadata")
			{
				bBinOut = false;
				bWriteMetadata = false;
			}
			else
			{
				throw new Exception("Unknown output format!");
			}

			if (outFilename == null)
			{
				outFilename = inFilenameWithoutExtension + (bBinOut ? ".csv.bin" : ".csv");
			}
			if (outFilename.ToLowerInvariant() == inFilename.ToLowerInvariant() && bInPlace == false)
			{
				throw new Exception("Input and output filenames can't match! Specify -inPlace to override");
			}
			if (System.IO.File.Exists(outFilename) && !GetBoolArg("force") && !bInPlace)
			{
				throw new Exception("Output file already exists! Use -force to overwrite anyway.");
			}

			Console.WriteLine("Output filename: " + outFilename);


			// Set metadata if requested
			string setMetadataFile = GetArg("setMetadataFile",null);
			string SetMetadataStr = GetArg("setMetadata");

			// Make sure we have metadata if we're setting it
			bool bSkipUpdatingMetadata = false;
			if (csvStats.metaData == null && (setMetadataFile != null || SetMetadataStr != null) )
			{
				if (GetBoolArg("skipIfMetadataMissing"))
				{
					Console.WriteLine("Skipping writing metadata since it has none.");
					bSkipUpdatingMetadata = true;
				}
				else
				{
					Console.WriteLine("File has null metadata. Creating...");
					csvStats.metaData = new CsvMetadata();
					csvStats.GenerateCsvIDIfMissing();
				}
			}


			if (setMetadataFile != null && !bSkipUpdatingMetadata)
			{
				if (csvStats.metaData == null)
				{
					throw new Exception("File has no metadata, so can't set it");
				}
				string [] metadataLines=System.IO.File.ReadAllLines(setMetadataFile);
				foreach (string line in metadataLines)
				{
					int equalsIndex = line.IndexOf("=");
					if (equalsIndex != -1)
					{
						string Key = line.Substring(0, equalsIndex);
						string Value = line.Substring(equalsIndex + 1);
						Console.WriteLine("Setting metadata: " + Key + "=" + Value);
						csvStats.metaData.Values[Key] = Value;
					}
				}
			}

			if (SetMetadataStr != null && SetMetadataStr.Length > 0 && !bSkipUpdatingMetadata)
			{
				bool bModified = false;
				if (csvStats.metaData == null)
				{
					throw new Exception("File has no metadata, so can't set it");
				}
				string[] KeyValuePairs = SetMetadataStr.Split(';');
				foreach (string KeyValueStr in KeyValuePairs)
				{
					string[] KeyValue = KeyValueStr.Split('=');
					if (KeyValue.Length == 2)
					{
						string Key = KeyValue[0].ToLower().Trim();
						string Value = KeyValue[1].Trim();
						Console.WriteLine("Setting metadata: " + Key+"="+Value);
						csvStats.metaData.Values[Key] = Value;
						bModified = true;
					}
					else
					{
						Console.WriteLine("Bad metadata pair: " + KeyValueStr);
					}
				}
				if ( bModified )
				{
					csvStats.metaData.Values["metadatamodified"] = "1";
				}
			}

			Console.WriteLine("Writing output file...");
			if (bBinOut)
			{
				int binCompression = GetIntArg("binCompress", -1);

				// If compression is not set, determine it automatically
				if (binCompression == -1 )
				{
					binCompression = (int)inFileInfo.BinCompressionLevel;
				}

				if (binCompression < 0 || binCompression > 2)
				{
					throw new Exception("Bad compression level specified! Must be 0, 1 or 2");
				}
				csvStats.WriteBinFile(outFilename, (CSVStats.CsvBinCompressionLevel)binCompression);
			}
			else
			{
				csvStats.WriteToCSV(outFilename, bWriteMetadata);
			}


			if (GetBoolArg("verify"))
			{
				// TODO: if verify is specified, use a temp intermediate file?
				Console.WriteLine("Verifying output...");
				CsvStats csvStats2 = CsvStats.ReadCSVFile(outFilename,null);
				if (csvStats.SampleCount != csvStats2.SampleCount)
				{
					throw new Exception("Verify failed! Sample counts don't match");
				}
				double sum1 = 0.0;
				double sum2 = 0.0;
				foreach (StatSamples stat in csvStats.Stats.Values)
				{
					StatSamples stat2 = csvStats2.GetStat(stat.Name);

					if (stat2==null)
					{
						throw new Exception("Verify failed: missing stat: "+stat.Name);
					}
					for ( int i=0; i<csvStats.SampleCount; i++ )
					{
						sum1 += stat.samples[i];
						sum2 += stat2.samples[i];
					}
				}
				if (sum1 != sum2)
				{
					if (bBinOut == false)
					{
						// conversion to CSV is lossy. Just check the sums are within 0.25%
						double errorMargin = Math.Abs(sum1) * 0.0025;
						if (Math.Abs(sum1-sum2)> errorMargin)
						{
							throw new Exception("Verify failed: stat sums didn't match!");
						}
					}
					else
					{
						throw new Exception("Verify failed: stat sums didn't match!");
					}
				}
				// conversion to CSV is lossy. Hashes won't match because stat totals will be slightly different, so skip this check
				if (bBinOut)
				{
					if (csvStats.MakeCsvIdHash() != csvStats2.MakeCsvIdHash())
					{
						throw new Exception("Verify failed: hashes didn't match");
					}
				}
				Console.WriteLine("Verify success: Input and output files matched");
			}
		}
	}
}
