// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Diagnostics;
using CSVStats;

namespace CSVInfo
{
    class Version
    {
        private static string VersionString = "1.07";

        public static string Get() { return VersionString; }
    };

    class Program : CommandLineTool
    {
        static int Main(string[] args)
        {
            Program program = new Program();
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.Error.WriteLine("[ERROR] " + e.Message);
                    return 1;
                }
            }

            return 0;
        }

		string Quotify(string s)
		{
			return "\"" + s + "\"";
		}

		string Sanitize(string s)
		{
			return s.Replace("\\","\\\\").Replace("\"", "\\\"");
		}

		string ToJsonString(string s)
		{
			return Quotify(Sanitize(s));
		}

		string ToJsonStringList(List<string> list)
		{
			List<string> safeList = new List<string>();
			foreach(string s in list)
			{
				safeList.Add(ToJsonString(s));
			}
			return "[" + String.Join(",", safeList) + "]";
		}

        void Run(string[] args)
        {
            string formatString =
                "Format: \n" +
                "  <csvfilename>\n"+
				"  [-showaverages] Output stat averages.\n" +
				"  [-showmin] Output stat min values.\n" +
				"  [-showmax] Output stat max values.\n" +
				"  [-showTotals] Output stat totals.\n" +
				"  [-showAllStats] Output stat values from all frames.\n" +
				"  [-showEvents] Output CSV Events and the frame they occured on.\n" +
				"  [-forceFullRead] (always reads the full CSV)\n" +
				"  [-quiet] (no logging. Just throws returns a non-zero error code if the CSV is bad)\n" +
				"  [-toJson <filename>] Output to a json file instead of stdout.\n" +
				"  [-statFilters <stat list>] Comma separated list of stats to output values for. Wildcards are supported (eg. LLM/*)";

			// Read the command line
			if (args.Length < 1)
            {
				WriteLine("CsvInfo " + Version.Get());
                WriteLine(formatString);
                return;
            }

            string csvFilename = args[0];

            ReadCommandLine(args);

            bool showAverages = GetBoolArg("showAverages");
            bool showMin = GetBoolArg("showMin");
            bool showMax = GetBoolArg("showMax");
			bool showTotals= GetBoolArg("showTotals");
			bool showAllStats = GetBoolArg("showAllStats");
			bool showEvents = GetBoolArg("showEvents");
			string jsonFilename = GetArg("toJson",false);
			string statFilterString = GetArg("statFilters",false);
			bool bReadJustHeader = !GetBoolArg("forceFullRead") && !showAverages && !showTotals && !showMin && !showMax
			                       && (statFilterString.Length == 0);

			string[] statFilters = statFilterString.Length > 0 ? statFilterString.Split(',') : null;

			CSVStats.CsvFileInfo fileInfo = new CsvFileInfo();
			CsvStats csvStats = CsvStats.ReadCSVFile(csvFilename, statFilters, 0, false, fileInfo, bReadJustHeader);

			if ( GetBoolArg("quiet") )
			{
				return;
			}

			if (jsonFilename != "")
			{
				// We just write the lines raw, since this version of .Net doesn't have a json serializer. 
				// TODO: Fix this when we upgrade to .Net 5.0 and use System.Text.Json
				List<string> jsonLines = new List<string>();
				jsonLines.Add("{");
				jsonLines.Add("  \"sampleCount\":" + fileInfo.SampleCount+",");

				if (csvStats.metaData != null)
				{
					jsonLines.Add("  \"metadata\": {");
					Dictionary<string, string> metadata = csvStats.metaData.Values;
					int count = metadata.Count;
					int index = 0;
					foreach (string key in metadata.Keys)
					{
						string line = "    " + ToJsonString(key) + ":" + ToJsonString(metadata[key]);
						if (index < count-1)
						{
							line += ",";
						}
						jsonLines.Add(line);
						index++;
					}
					jsonLines.Add("  },");
				}

				if (showEvents)
				{
					jsonLines.Add("  \"events\": [");
					for (int i = 0; i < csvStats.Events.Count; ++i)
					{
						CsvEvent csvEvent = csvStats.Events[i];
						string line = $"		{{\"EventName\": \"{csvEvent.Name}\", \"Frame\": {csvEvent.Frame}}}";
						line += (i < csvStats.Events.Count - 1) ? "," : "";
						jsonLines.Add(line);
					}
					jsonLines.Add("  ],");
				}

				List<string> statLines = new List<string>();
				foreach (StatSamples stat in csvStats.Stats.Values.ToArray())
				{
					statLines.Add(stat.Name);
				}
				statLines.Sort();

				if (showTotals || showAverages || showMin || showMax || showAllStats)
				{
					jsonLines.Add("  \"stats\": {");
					for (int i=0; i<statLines.Count; i++)
					{
						string statName = statLines[i];
						List<string> entries = new List<string>();
						if ( showTotals )
						{
							entries.Add("\"total\":" + csvStats.GetStat(statName).total);
						}
						if (showAverages)
						{
							entries.Add("\"average\":" + csvStats.GetStat(statName).average);
						}
						if (showMin)
						{
							entries.Add("\"min\":" + csvStats.GetStat(statName).ComputeMinValue());
						}
						if (showMax)
						{
							entries.Add("\"max\":" + csvStats.GetStat(statName).ComputeMaxValue());
						}
						if (showAllStats)
						{
							entries.Add("\"all\": [" + String.Join(",", csvStats.GetStat(statName).samples) + "]");
						}
						string line = "    \"" + statName + "\": {" + String.Join(",", entries) + "}";
						if (i < statLines.Count-1)
							line += ",";
						jsonLines.Add(line);
					}
					jsonLines.Add("  }");
				}
				else
				{
					// Just output stats as an array if totals/averages were not requested
					jsonLines.Add("  \"stats\": " + ToJsonStringList(statLines));
				}

				jsonLines.Add("}");
				System.IO.File.WriteAllLines(jsonFilename,jsonLines);
				Console.Out.WriteLine("Wrote csv info to " + jsonFilename);
			}
			else
			{
				// Write out the sample count
				Console.Out.WriteLine("Sample Count: " + fileInfo.SampleCount);

				List<string> statLines = new List<string>();
				foreach (StatSamples stat in csvStats.Stats.Values.ToArray())
				{
					string statLine = stat.Name;
					if (showAverages)
					{
						statLine += " (" + stat.average.ToString() + ") ";
					}
					if (showTotals)
					{
						statLine += " (Total: " + stat.total.ToString()+") ";
					}
					if (showMin)
					{
						statLine += " (Min: " + stat.ComputeMinValue().ToString()+") ";
					}
					if (showMax)
					{
						statLine += " (Max: " + stat.ComputeMaxValue().ToString()+") ";
					}
					if (showAllStats)
					{
						statLine += " (All: [" + String.Join(",", stat.samples) + "]) ";
					}
					statLines.Add(statLine);
				}
				statLines.Sort();

				if (showEvents)
				{
					Console.Out.WriteLine("Events:");
					foreach (CsvEvent csvEvent in csvStats.Events)
					{
						Console.Out.WriteLine($"{csvEvent.Name}: {csvEvent.Frame}");
					}
				}

				// Write out the sorted stat names
				Console.Out.WriteLine("Stats:");
				foreach (string statLine in statLines)
				{
					Console.Out.WriteLine("  " + statLine);
				}

				if (csvStats.metaData != null)
				{
					// Write out the metadata, if it exists
					Console.Out.WriteLine("\nMetadata:");
					foreach (KeyValuePair<string, string> pair in csvStats.metaData.Values.ToArray())
					{
						string key = pair.Key.PadRight(20);
						Console.Out.WriteLine("  " + key + ": " + pair.Value);
					}
				}

			}
		}
    }
}
