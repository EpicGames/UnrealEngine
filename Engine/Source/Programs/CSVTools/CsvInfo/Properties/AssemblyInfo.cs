// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("5110c345-5449-40e2-9728-6197b23b5b37")]
