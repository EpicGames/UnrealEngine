<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <TargetFramework>netcoreapp2.0</TargetFramework>
    <RootNamespace>CsvStats</RootNamespace>
    <AssemblyName>CsvStats</AssemblyName>
    <RuntimeIdentifiers>win-x64;win-x86;linux-x64;osx-x64</RuntimeIdentifiers>
  </PropertyGroup>

  <ItemGroup>
    <Compile Remove="Properties\AssemblyInfo.cs" />
  </ItemGroup>

  <ItemGroup>
    <Folder Include="Properties\" />
  </ItemGroup>

</Project>
