// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using System.Diagnostics;
using System.Threading;

namespace CSVStats
{
	public class CommandLine
	{
		protected string commandLine = "";

		protected Dictionary<string, string> CommandLineArgs;

		public string GetCommandLine()
		{
			return commandLine;
		}

		public static bool IsParamName(string str)
		{
			if (str.Length == 0)
			{
				return false;
			}
			if (str[0] != '-')
			{
				return false;
			}
			if ( Double.TryParse(str, System.Globalization.NumberStyles.Any, System.Globalization.NumberFormatInfo.InvariantInfo, out _) )
			{
				return false;
			}
			return true;
		}

		private List<string> GetArgList(string inCommandLine)
		{
			List<string> args = new List<string>();
			bool bInQuotes = false;
			StringBuilder currentArgSb= new StringBuilder();
			for (int i=0; i< inCommandLine.Length; i++)
			{
				char c = inCommandLine[i];
				if (c=='"')
				{
					bInQuotes = !bInQuotes;
				}
				else
				{
					if (bInQuotes)
					{
						currentArgSb.Append(c);
					}
					else
					{ 
						if (Char.IsWhiteSpace(c))
						{
							if (currentArgSb.Length > 0)
							{
								args.Add(currentArgSb.ToString());
								currentArgSb.Clear();
							}
						}
						else
						{
							currentArgSb.Append(c);
						}

					}
				}
			}
			if (currentArgSb.Length > 0)
			{
				args.Add(currentArgSb.ToString());
			}
			return args;
		}

		public CommandLine(string[] args)
		{
			// Read commandline as a response file if specified (must be the only two args)
			if (args.Length==2 && args[0].ToLower()=="-response")
			{
				string [] lines = File.ReadAllLines(args[1]);
				List<string> argList = new List<string>();
				foreach (string line in lines)
				{
					argList.AddRange(GetArgList(line));
				}
				args = argList.ToArray();
			}

			// Write the commandline as a string for reporting purposes
			commandLine = "";
			foreach (string arg in args)
			{
				if (arg.Contains(' ') || arg.Contains('\t'))
				{
					commandLine += "\"" + arg + "\" ";
				}
				else
				{
					commandLine += arg + " ";
				}
			}

			// Parse the commandline
			CommandLineArgs = new Dictionary<string, string>();
			for (int i = 0; i < args.Length; i++)
			{
				string arg = args[i];
				if (IsParamName(arg))
				{
					string val = "1";

					// If there's a value, read it
					if (i < args.Length - 1 && !IsParamName(args[i+1]))
					{
						bool first = true;
						for (int j = i + 1; j < args.Length; j++)
						{
							string str = args[j];
							if (str.Length > 0 && IsParamName(str))
								break;
							if (first)
							{
								val = str;
								first = false;
							}
							else
							{
								val += ";" + str;
							}
						}
						i++;
					}

					// Decode negative values. This is necessary otherwise negative values are interpreted as separate args.
					val = val.Replace("&minus;", "-");

					string argKey = arg.Substring(1).ToLower();
					if (CommandLineArgs.ContainsKey(argKey))
					{
						Console.Out.WriteLine("Duplicate commandline argument found for " + arg + ". Overriding value from " + CommandLineArgs[argKey] + " to " + val);
						CommandLineArgs.Remove(argKey);
					}
					CommandLineArgs.Add(arg.Substring(1).ToLower(), val);
				}
			}
		}

		public int GetIntArg(string key, int defaultValue)
		{
			string val = GetArg(key, false);
			if (val != "") return Convert.ToInt32(val);
			return defaultValue;
		}

		public float GetFloatArg(string key, float defaultValue)
		{
			string val = GetArg(key, false);
			if (val != "") return (float)Convert.ToDouble(val, System.Globalization.CultureInfo.InvariantCulture);
			return defaultValue;
		}

		public bool GetBoolArg(string key, bool defaultValue)
		{
			if ( CommandLineArgs.ContainsKey(key.ToLower()) )
			{
				string value = CommandLineArgs[key.ToLower()];
				if (value == "1")
				{
					return true;
				}
				if (value == "0")
				{
					return false;
				}
			}
			return defaultValue;
		}

		public bool? GetOptionalBoolArg(string key)
		{
			if ( CommandLineArgs.ContainsKey(key.ToLower()) )
			{
				return CommandLineArgs[key.ToLower()] == "1";
			}
			return null;
		}

		public string GetArg(string key, string defaultValue)
		{
			string lowerKey = key.ToLower();

			if (CommandLineArgs.ContainsKey(lowerKey))
			{
				return CommandLineArgs[lowerKey];
			}
			return defaultValue;
		}

		public string GetArg(string key, bool mandatory = false)
		{
			string lowerKey = key.ToLower();

			if (CommandLineArgs.ContainsKey(lowerKey))
			{
				return CommandLineArgs[lowerKey];
			}
			if (mandatory)
			{
				Console.WriteLine("Missing parameter " + key);
			}
			return "";
		}

	}

	public class CommandLineTool
	{
		protected enum HostPlatform
		{
			Windows,
			Mac,
			Linux
		}

		static int initCounter = 0;

		public CommandLineTool()
		{
			bMainInstance = Interlocked.Increment(ref initCounter) == 1;
			System.Globalization.CultureInfo.DefaultThreadCurrentCulture = System.Globalization.CultureInfo.InvariantCulture;
		}

		// Accesses the commandline for the main instance. Caution: if using batch CommandLineTool instances (e.g via CsvToSvg -batchCommands), this will not return the commandline for the batch
		public static CommandLine GetGlobalCommandLine()
		{
			return globalCommandLine;
		}
		public CommandLine GetCommandLine()
		{
			return commandLine;
		}

		// It's possible to create instances of commandlineTool for batch processing. When this happens, bMainInstance will be false
		// globalCommandline is only valid for the main/first instance
		protected CommandLine commandLine;
		private static CommandLine globalCommandLine;
		private bool bMainInstance = true;

		private readonly static bool bIsMac = File.Exists("/System/Library/CoreServices/SystemVersion.plist");

		protected static HostPlatform Host
		{
			get 
			{
				PlatformID Platform = Environment.OSVersion.Platform;
				switch (Platform)
				{
					case PlatformID.Win32NT:
						return HostPlatform.Windows;
					case PlatformID.Unix:
						return bIsMac? HostPlatform.Mac : HostPlatform.Linux;
					case PlatformID.MacOSX:
						return HostPlatform.Mac;
					default:
						throw new Exception("Unhandled runtime platform " + Platform);
				}
			}
		}

		protected static string MakeShortFilename(string filename)
        {
            int index = filename.LastIndexOf(Path.DirectorySeparatorChar);
            if (index == -1)
            {
                return filename;
            }
            else
            {
                return filename.Substring(index + 1);
            }
        }
        protected int GetIntArg(string key, int defaultValue)
        {
			return commandLine.GetIntArg(key, defaultValue);
		}

		protected float GetFloatArg(string key, float defaultValue)
        {
			return commandLine.GetFloatArg(key, defaultValue);
		}

		protected bool GetBoolArg(string key, bool defaultValue=false)
        {
			return commandLine.GetBoolArg(key, defaultValue);
        }

		protected T GetEnumArg<T>(string key, T defaultValue) where T : System.Enum
		{
			string str = GetArg(key, null);
#nullable enable
			if ( str != null && Enum.TryParse(typeof(T), str, true, out object? result) )
#nullable disable
			{
				return (T)result;
			}
			return defaultValue;
		}

		protected bool? GetOptionalBoolArg(string key)
		{
			return commandLine.GetOptionalBoolArg(key);
		}


		protected string GetArg(string key, string defaultValue)
		{
			return commandLine.GetArg(key, defaultValue);
		}

		protected string GetArg(string key, bool mandatory = false)
        {
			return commandLine.GetArg(key, mandatory);
        }

		protected List<string> GetListArg(string key, char separator=';', bool convertToLowercase=false, bool mandatory=false)
		{
			string listStr=commandLine.GetArg(key, mandatory);
			if (listStr=="")
			{
				return new List<string>();
			}
			if (convertToLowercase)
			{
				listStr = listStr.ToLower();
			}
			return CsvStats.SplitStringListWithBracketGroups(listStr, separator);
		}

		protected void WriteLine(String message, params object[] args)
        {
            String formatted = String.Format(message, args);
            Console.WriteLine(formatted);
        }


        protected string[] ReadLinesFromFile(string filename)
        {
            StreamReader reader = new StreamReader(filename, true);
            List<string> lines = new List<string>();

            // Detect unicode
            string line = reader.ReadLine();

            bool bIsUnicode = false;
            for (int i = 0; i < line.Length - 1; i++)
            {
                if (line[i] == '\0')
                {
                    bIsUnicode = true;
                    break;
                }
            }
            if (bIsUnicode)
            {
                reader = new StreamReader(filename, Encoding.Unicode, true);
            }
            else
            {
                lines.Add(line);
            }

            while ((line = reader.ReadLine()) != null)
            {
                if (line.Trim().Length > 0)
                {
                    lines.Add(line);
                }
            }
            reader.Close();
            return lines.ToArray();
        }

		protected void ReadCommandLine(string[] args)
		{
			commandLine = new CommandLine(args);
			if (bMainInstance)
			{
				globalCommandLine = commandLine;
			}
		}
	}


	public class PerfLog
	{
		public PerfLog(bool inLoggingEnabled, string inLogPrefix=null)
		{
			stopWatch = Stopwatch.StartNew();
			previousTime = 0.0;
			loggingEnabled = inLoggingEnabled;
			if (inLogPrefix != null)
			{
				logPrefix = inLogPrefix + " - ";
			}
		}

		public double LogTiming(string description, bool newLine = false)
		{

			double currentTime = stopWatch.Elapsed.TotalSeconds;
			double elapsed = currentTime - previousTime;

			if (loggingEnabled)
			{
				Console.WriteLine("[PerfLog] " + logPrefix + String.Format("{0,-25} : {1,-10}", description, (elapsed * 1000.0).ToString("0.0") + "ms"), 70);
				if (newLine)
				{
					Console.WriteLine();
				}
			}
			previousTime = currentTime;
			return elapsed;
		}

		public double LogTotalTiming(bool seconds=true)
		{
			double elapsed = stopWatch.Elapsed.TotalSeconds;
			if (loggingEnabled)
			{
				if (seconds)
				{
					Console.WriteLine("[PerfLog] "+logPrefix+"TOTAL: " + elapsed.ToString("0.0") + "s\n");
				}
				else
				{
					Console.WriteLine("[PerfLog] " + logPrefix + "TOTAL: " + (elapsed*1000.0).ToString("0.0") + "ms\n");
				}
			}
			return elapsed;
		}
		Stopwatch stopWatch;
		double previousTime;
		bool loggingEnabled;
		string logPrefix = "";
	}
}
