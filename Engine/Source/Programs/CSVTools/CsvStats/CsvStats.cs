// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using System.Globalization;
using System.Collections;
using System.IO.Compression;
using System.Security.Cryptography;
using System.Data;

namespace CSVStats
{
	public enum CsvBinVersion
	{
		PreRelease = 1,
		InitialRelease,
		CompressionSupportAndFlags,
		EventTimestamps,

		COUNT,
		CURRENT = COUNT - 1
	};

	enum CsvBinFlags
	{
		HasMetadata = 0x00000001,
	};
	public enum CsvBinCompressionType
	{
		MsDeflate
	};

	public enum CsvBinCompressionLevel
	{
		None,
		Min,
		Max
	};


	public class CsvFileInfo
	{
		public bool bIsCsvBin;
		public CsvBinVersion BinVersion = CsvBinVersion.COUNT;
		public CsvBinCompressionLevel BinCompressionLevel = CsvBinCompressionLevel.None;
		public int SampleCount = 0;
	};


	public class CsvEvent
    {
		public CsvEvent(CsvEvent source)
		{
			Name = source.Name;
			Frame = source.Frame;
			Timestamp = source.Timestamp;
		}
		public CsvEvent(string NameIn, int FrameIn, double TimestampIn)
        {
            Name = NameIn;
            Frame = FrameIn;
			Timestamp = TimestampIn; 
        }
        public CsvEvent()
        {

        }

		public bool HasTimestamp() { return Timestamp != double.MaxValue; }

        public string Name;
        public int Frame;
		public double Timestamp = double.MaxValue;
    };

	public class DerivedStatException : Exception
	{
		public DerivedStatException(string message) : base(message) { }
	}

	public class DerivedStatMissingSourceException : DerivedStatException
	{
		public DerivedStatMissingSourceException(string message) : base(message) { }
	}


	public class CsvMetadata
    {
        public Dictionary<string, string> Values;

        public CsvMetadata Clone()
        {
            CsvMetadata rv = new CsvMetadata();
            rv.Values = Values;
            return rv;
        }

		public CsvMetadata(CsvMetadata source)
		{
			Values = new Dictionary<string, string>();
			List<KeyValuePair<string, string>> pairList = source.Values.ToList();
			foreach (KeyValuePair<string, string> pair in pairList)
			{
				Values.Add(pair.Key, pair.Value);
			}
		}

		public CsvMetadata()
		{
			Values = new Dictionary<string, string>();
		}

		public CsvMetadata(string[] csvLine)
        {
            Values = new Dictionary<string, string>();
            TextInfo ti = CultureInfo.CurrentCulture.TextInfo;

            // Last line is key/value pairs. Commandline is the last entry, and needs special casing
            bool bIsKey = false;
            string key = "";
            string value = "";
            for ( int i=0; i<csvLine.Length; i++ )
            {
                string entry = csvLine[i];
                if ( entry.StartsWith("[") && entry.EndsWith("]") && !bIsKey)
                {
                    key = entry.Substring(1, entry.Length - 2).ToLowerInvariant();
                    bIsKey = true;
                }
                else
                {
                    if (key == "commandline")
                    {
                        value = "";
                        bool bFirst = true;
                        for (int j = i; j < csvLine.Length; j++)
                        {
                            bool bLast = ( j == csvLine.Length - 1 );
                            string segment = csvLine[j];
                            if (bFirst)
                            {
                                if (segment.StartsWith("\""))
                                {
                                    segment = segment.Substring(1);
                                }
                                bFirst = false;
                            }
                            if (bLast && segment.EndsWith("\""))
                            {
                                segment = segment.Substring(0,segment.Length-1);
                            }
                            value += segment;
                            if ( j<csvLine.Length-1 )
                            {
                                value += ",";
                            }
                        }
                        // We're done
                        i = csvLine.Length;
                    }
                    else
                    {
                        value = entry;
                    }

					if (Values.ContainsKey(key))
					{
						// If we're still writing to the current key, append the data onto the end
						Values[key] += "," + value;
					}
					else
					{
						Values.Add(key, value);
					}
					bIsKey = false;
                }

            }
        }

		public void WriteToBinaryFile(BinaryWriter fileWriter)
		{
			// Write metadata
			fileWriter.Write(Values.Count);
			foreach (string key in Values.Keys)
			{
				fileWriter.Write(key);
				fileWriter.Write(Values[key]);
			}
		}
		public void ReadFromBinaryFile(BinaryReader fileReader)
		{
			// Write metadata
			int numValues=fileReader.ReadInt32();
			for (int i=0;i<numValues;i++)
			{
				string key = fileReader.ReadString();
				string value = fileReader.ReadString();
				Values[key] = value;
			}
		}

		public void CombineAndValidate(CsvMetadata comparisonMetadata)
        {
            List<string> valuesDontMatchKeys = new List<string>();
            foreach (KeyValuePair<string, string> pair in Values)
            {
				bool bMatch = false;
				if (comparisonMetadata != null)
				{
					if (comparisonMetadata.Values.ContainsKey(pair.Key))
					{
						if (comparisonMetadata.Values[pair.Key] == pair.Value)
						{
							bMatch = true;
						}
					}
				}
				if ( !bMatch )
                {
                    valuesDontMatchKeys.Add(pair.Key);
                    break;
                }
            }

            foreach (string key in valuesDontMatchKeys)
            {
                Values[key] = "["+ key + " doesn't match!]";
            }
        }

        public static bool Matches(CsvMetadata a, CsvMetadata b)
        {
            if ( a == null && b == null )
            {
                return true;
            }
            if ( a == null )
            {
                return false;
            }
            if ( b == null )
            {
                return false;
            }
            if (a.Values.Count != b.Values.Count)
            {
                return false;
            }

            foreach (KeyValuePair<string, string> pair in a.Values)
            {
                bool bMatch = false;
               // if (pair.Key != "deviceprofile")
                {
                    if (b.Values.ContainsKey(pair.Key))
                    {
                        if (b.Values[pair.Key] == pair.Value)
                        {
                            bMatch = true;
                        }
                    }
                    if (!bMatch)
                    {
                        return false;
                    }
                }
            }
            return true;
        }


        public string GetValue(string key, string defaultValue)
        {
            if ( Values.ContainsKey(key))
            {
                return Values[key];
            }
            return defaultValue;
        }
    };



    public class StatSamples : IComparable
    {
        public StatSamples(StatSamples source, bool copySamples=true)
        {
            Name = source.Name;
            samples = new List<float>();
			if (copySamples)
			{
				samples.AddRange(source.samples);
			}
			average = source.average;
            total = source.total;
            colour = source.colour;
            LegendName = source.LegendName;
        }
        public StatSamples(string name, int sampleListLength=0, bool bInIsDerived = false)
        {
            Name = name;
            LegendName = name;
            if (sampleListLength == 0)
            {
                samples = new List<float>();
            }
            else
            {
                samples = new List<float>(new float[sampleListLength]);
            }
            average = 0.0f;
            total = 0.0f;
            colour = new Colour(0, 0.0f);
			bIsDerived = bInIsDerived;
		}

        public float GetRatioOfFramesInBudget(float budget)
        {
            int countInBudget = 0;
            for (int i = 0; i < samples.Count; i++)
            {
                if (samples[i] <= budget)
                {
                    countInBudget++;
                }
            }
            return (float)(countInBudget) / (float)(samples.Count);
        }

		public float GetRatioOfFramesOverBudget(float budget)
		{
			return 1.0f - GetRatioOfFramesInBudget(budget);
		}

		public int GetCountOfFramesOverBudget(float budget, bool IgnoreFirstFrame = true, bool IgnoreLastFrame = true)
        {
            int countOverBudget = 0;
            for (int i = IgnoreFirstFrame ? 1 : 0; i < samples.Count - (IgnoreLastFrame ? 1 : 0); i++)
            {
                if (samples[i] > budget)
                {
                    countOverBudget++;
                }
            }
            return countOverBudget;
        }

        public int GetCountOfFramesAtOrOverBudget(float budget, bool IgnoreFirstFrame = true, bool IgnoreLastFrame = true)
        {
            int countOverBudget = 0;
            for (int i = IgnoreFirstFrame ? 1 : 0; i < samples.Count - (IgnoreLastFrame ? 1 : 0); i++)
            {
                if (samples[i] >= budget)
                {
                    countOverBudget++;
                }
            }
            return countOverBudget;
        }


        public int GetCountOfFramesInRangeOverBudget(float budget, int startindex, int endindex)
        {
            int countOverBudget = 0;
            if (endindex > samples.Count)
            {
                endindex = samples.Count;
            }
            for (int i = startindex; i < endindex; i++)
            {
                if (samples[i] > budget)
                {
                    countOverBudget++;
                }
            }
            return countOverBudget;
        }



        public int CompareTo(object obj)
        {
            if (obj is StatSamples)
            {
                float objAverage = (obj as StatSamples).average;
                if (average > objAverage) return -1;
                if (average < objAverage) return 1;
                return 0;
                //                return (int)((objAverage - average) * 100000000.0f);
            }
            throw new ArgumentException("Object is not a StatSamples");
        }


        public float ComputeAverage(int minSample=0, int maxSample=-1)
        {
            if (maxSample == -1)
            {
                maxSample = samples.Count;
            }
			double localTotal = 0.0;
            for (int i = minSample; i < maxSample; i++)
            {
				localTotal += (double)samples[i];
            }
			return (float)(localTotal / (double)(maxSample - minSample));
		}

		public void ComputeAverageAndTotal(int minSample = 0, int maxSample = -1)
        {
            if (maxSample == -1)
            {
                maxSample = samples.Count;
            }
            total = 0.0;
            for (int i = minSample; i < maxSample; i++)
            {
                total += (double)samples[i];
            }
            average = (float)(total / (double)(maxSample - minSample));
        }

        public float ComputeMaxValue(int minSample = 0, int maxSample = -1)
        {
            if (maxSample == -1)
            {
                maxSample = samples.Count;
            }
			else
			{
				maxSample = Math.Min(maxSample, samples.Count);
			}
            float maxValue = -float.MaxValue;
            for (int i = minSample; i < maxSample; i++)
            {
                maxValue = Math.Max(maxValue, samples[i]);
            }
            return maxValue;
        }
        public float ComputeMinValue(int minSample = 0, int maxSample = -1)
        {
            if (maxSample == -1)
            {
                maxSample = samples.Count;
            }
            float minValue = float.MaxValue;
            for (int i = minSample; i < maxSample; i++)
            {
                minValue = Math.Min(minValue, samples[i]);
            }
            return minValue;
        }

        public int GetNumSamples() { return samples.Count; }
        public string Name;
        public string LegendName;
        public List<float> samples;
        public float average;
        public double total;
        public Colour colour;
		public bool bIsDerived;
    };


	public class CsvStats
	{
		public CsvStats()
		{
			Stats = new Dictionary<string, StatSamples>();
			Events = new List<CsvEvent>();
		}
		public CsvStats(CsvStats sourceCsvStats, string[] statNamesToFilter = null)
		{
			Stats = new Dictionary<string, StatSamples>();
			Events = new List<CsvEvent>();
			if (statNamesToFilter != null)
			{
				foreach (string statName in statNamesToFilter)
				{
					List<StatSamples> stats = sourceCsvStats.GetStatsMatchingString(statName);
					foreach (StatSamples sourceStat in stats)
					{
						string key = sourceStat.Name.ToLower();
						if (!Stats.ContainsKey(key))
						{
							Stats.Add(key, new StatSamples(sourceStat));
						}
					}
				}
			}
			else
			{
				foreach (StatSamples sourceStat in sourceCsvStats.Stats.Values)
				{
					AddStat(new StatSamples(sourceStat));
				}
			}
			foreach (CsvEvent ev in sourceCsvStats.Events)
			{
				Events.Add(new CsvEvent(ev));
			}
			if (sourceCsvStats.metaData != null)
			{
				metaData = new CsvMetadata(sourceCsvStats.metaData);
			}
		}
		public StatSamples GetStat(string name)
		{
			name = name.ToLower();
			if (Stats.ContainsKey(name))
			{
				return Stats[name];
			}
			return null;
		}

		public string GetMetadataValue(string key)
		{
			if (metaData == null)
			{
				return null;
			}
			return metaData.GetValue(key, null);
		}

		public Tuple<CsvEvent, CsvEvent> FindEventPair(string patternBegin, string patternEnd, bool bFindLast = false)
		{
			IEnumerable<CsvEvent> eventsOrdered = bFindLast == false ? Events : Events.Reverse<CsvEvent>();
			CsvEvent beginEvent = patternBegin != null? FindEventInternal(eventsOrdered, patternBegin) : null;
			double startTime = (beginEvent != null) ? beginEvent.Timestamp : 0.0;

			CsvEvent endEvent = (patternEnd != null) ? FindEventInternal(eventsOrdered, patternEnd, startTime, double.MaxValue) : null;
			return Tuple.Create(beginEvent, endEvent);
		}

		public CsvEvent FindEvent(string pattern, double startTime = 0.0, double endTime = double.MaxValue)
		{
			return FindEventInternal(Events, pattern, startTime, endTime);
		}

		private CsvEvent FindEventInternal(IEnumerable<CsvEvent> InEvents, string pattern, double startTime = 0.0, double endTime = double.MaxValue)
		{
			foreach (CsvEvent ev in InEvents)
			{
				if (CsvStats.DoesSearchStringMatch(ev.Name, pattern) && ev.Timestamp >= startTime && ev.Timestamp <= endTime)
				{
					return ev;
				}
			}
			return null;
		}

		public List<CsvEvent> FindEvents(string pattern, double startTime = 0.0, double endTime = double.MaxValue)
		{
			return FindEventsInternal(Events, pattern, startTime, endTime);
		}

		private List<CsvEvent> FindEventsInternal(IEnumerable<CsvEvent> InEvents, string pattern, double startTime = 0.0, double endTime = double.MaxValue)
		{
			List<CsvEvent> eventList = new List<CsvEvent>();
			foreach (CsvEvent ev in InEvents)
			{
				if (CsvStats.DoesSearchStringMatch(ev.Name, pattern) && ev.Timestamp >= startTime && ev.Timestamp <= endTime)
				{
					eventList.Add(ev);
				}
			}
			return eventList;
		}

		// Matches 
		string ReadQuotedString(string str)
		{
			if (str.Length < 2 || str[0] != '\'')
			{
				return null;
			}
			int endIdx = str.IndexOf('\'',1);
			if (endIdx == -1)
			{
				return null;
			}
			return str[1..endIdx];
		}
		string ReadBracketString(string str)
		{
			if (str.Length < 2 || str[0] != '(')
			{
				return null;
			}
			string strOut = "";
			int bracketDepth = 1;
			bool bInQuotes = false;
			for (int i=1; i<str.Length; i++)
			{
				char c = str[i];
				if (c=='\'')
				{
					bInQuotes = !bInQuotes;
				}
				if (!bInQuotes)
				{
					if (c == '(')
					{
						bracketDepth++;
					}
					else if (c == ')')
					{
						bracketDepth--;
						if (bracketDepth == 0)
						{
							return strOut;
						}
					}
				}
				strOut += c;
			}
			return null;
		}

		enum BuiltinFunctionType { addstats, meta, eventcount, framedelta, cumulative, COUNT };
		string ReadNextToken(string expression, ref int index, Dictionary<string, StatSamples> statsOut, int tokenIndex)
		{
			// Skip leading whitespace
			while (index < expression.Length && Char.IsWhiteSpace(expression[index]))
			{
				index++;
			}

			if (index >= expression.Length)
			{
				// We're done
				return null;
			}

			// Match delimiters
			char[] delimiters = { '+', '-', '*', '(', ')', '/' };
			if (delimiters.Contains(expression[index]))
			{
				index++;
				return expression[index-1].ToString();
			}

			string remaining = expression[index..];
			int oldIndex = index;

			// Match our builtin functions. These are all of the form NAME('PARAM')
			for ( int i=0; i<(int)BuiltinFunctionType.COUNT; i++)
			{
				BuiltinFunctionType funcType = (BuiltinFunctionType)i;
				string funcName = funcType.ToString();
				if (remaining.Length > (funcName.Length + 4) && 
					remaining.StartsWith(funcName+"("))
				{
					string paramString = ReadBracketString(remaining[(funcName.Length)..]);
					if (paramString == null)
					{
						continue;
					}
					index += funcName.Length + paramString.Length + 2;

					if (funcType == BuiltinFunctionType.meta)
					{
						// Quotes are optional. They're supported for consistency with stats
						string key = ReadQuotedString(paramString);
						key = (key == null) ? paramString : key;
						string val = GetMetadataValue(key);
						if (val == null)
						{
							throw new DerivedStatMissingSourceException("Metadata not found: " + expression[oldIndex..index]);
						}
						if (!float.TryParse(val, out _))
						{
							throw new DerivedStatException("Failed to parse metadata : " + expression[oldIndex..index]);
						}
						return val.ToString();
					}
					else if (funcType == BuiltinFunctionType.eventcount)
					{
						// Quotes are optional. They're supported for consistency with stats
						string eventStr = ReadQuotedString(paramString);
						eventStr = (eventStr == null) ? paramString : eventStr;
						if (eventStr == null)
						{
							throw new DerivedStatException("Eventstr requires an arg: " + expression[oldIndex..index]);
						}
						string statName = "__STAT__EVENTCOUNT__" + tokenIndex.ToString();
						statsOut[statName] = MakeEventCountStat(statName, eventStr);
						return statName;
					}
					else if (funcType == BuiltinFunctionType.addstats)
					{
						// Quotes are optional. They're supported for consistency with other stats
						string statsStr = paramString.Replace("'", "");
						string[] statNames = statsStr.Split(",");
						List<StatSamples> foundStats = GetStatsMatchingStringList(statNames);
						if (foundStats.Count == 0)
						{
							// Fail gracefully if the stats weren't found. The sum of no stats is zero
							return "0";
						}
						// Sanitize the stat names and add the stats to statsOut
						List<string> safeStatNames = new List<string>();
						foreach (StatSamples stat in foundStats)
						{
							string safeName = stat.Name.Replace("/", "__SLASH__");
							safeStatNames.Add(safeName);
							statsOut[safeName] = stat;
						}
						// Return the stats added together
						return "("+String.Join("+", safeStatNames)+")";
					}
					else if (funcType == BuiltinFunctionType.framedelta)
					{
						// Quotes are optional. They're supported for consistency with stats
						string srcStatName = ReadQuotedString(paramString);
						srcStatName = (srcStatName == null) ? paramString : srcStatName;
						if (srcStatName == null)
						{
							throw new DerivedStatException("FrameDelta requires an arg: " + expression[oldIndex..index]);
						}

						StatSamples srcStat = GetStat(srcStatName);
						if (srcStat == null)
						{
							throw new DerivedStatMissingSourceException("Source stat not found: " + expression[oldIndex..index]);
						}
						string destStatName = "__STAT__FRAMEDELTA__" + tokenIndex.ToString();
						StatSamples newStat = new StatSamples(destStatName, srcStat.samples.Count);
						for (int j=1; j<srcStat.samples.Count; j++)
						{
							newStat.samples[j] = srcStat.samples[j] - srcStat.samples[j-1];
						}
						statsOut[destStatName] = newStat;
						return destStatName;
					}
					else if (funcType == BuiltinFunctionType.cumulative)
					{
						// Quotes are optional. They're supported for consistency with stats
						string srcStatName = ReadQuotedString(paramString);
						srcStatName = (srcStatName == null) ? paramString : srcStatName;
						if (srcStatName == null)
						{
							throw new DerivedStatException("Cumulative requires an arg: " + expression[oldIndex..index]);
						}

						StatSamples srcStat = GetStat(srcStatName);
						if (srcStat == null)
						{
							throw new DerivedStatMissingSourceException("Source stat not found: " + expression[oldIndex..index]);
						}
						string destStatName = "__STAT__CUMULATIVE__" + tokenIndex.ToString();
						StatSamples newStat = new StatSamples(destStatName, srcStat.samples.Count);
						double total = 0.0;
						for (int j = 0; j < srcStat.samples.Count; j++)
						{
							total += srcStat.samples[j];
							newStat.samples[j] = (float)total;
						}
						statsOut[destStatName] = newStat;
						return destStatName;
					}

				}
			}

			// Match stats
			if (remaining.StartsWith('\''))
			{
				// Read the literal stat name in quotes. This is needed for stat names that include slashes
				string statName = ReadQuotedString(remaining);
				index += statName.Length + 2;
				StatSamples stat = GetStat(statName.Trim());
				if (stat == null)
				{
					throw new DerivedStatMissingSourceException("Stat not found: " + statName);
				}
				string safeName = stat.Name.Replace("/", "__SLASH__");
				statsOut[safeName] = stat;
				return safeName;
			}
			// Read up to the next delimiter as the stat name
			int endIdx = remaining.Length;
			for (int i=0; i<remaining.Length; i++)
			{
				if (delimiters.Contains(remaining[i]))
				{
					endIdx = i;
					break;
				}
			}
			string nextToken = remaining[0..endIdx];
			index += nextToken.Length;
			
			// Check if this is a stat. If not, just return it unmodified
			StatSamples foundStat = GetStat(nextToken.Trim());
			if ( foundStat != null )
			{
				statsOut[foundStat.Name] = foundStat;
				return foundStat.Name;
			}
			return expression[oldIndex..index];
		}

		string PreProcessExpression(string expression, Dictionary<string,StatSamples> statsOut)
		{
			List<string> tokens = new List<string>();
			expression = expression.Trim();
			if (expression.Length == 0)
			{
				return null;
			}

			int index = 0;
			string nextToken;
			do 
			{
				nextToken = ReadNextToken(expression, ref index, statsOut, tokens.Count);
				if (nextToken != null)
				{
					tokens.Add(nextToken);
				}
			} while(nextToken != null);
			return String.Join("", tokens);
		}

		public StatSamples MakeDerivedStatFromExpression(string expression, bool bVerboseErrors = false)
		{
			if (expression == "")
			{
				return null;
			}

			expression = expression.Trim();

			// Remove outer brackets if they exist
			if (expression[0] == '(' && expression[^1]==')' )
			{
				expression = expression[1..^1];
			}

			// If we have an equals, to the left of it is the stat name
			int equalsIndex = expression.IndexOf('=');
			string statName;
			if (equalsIndex != -1)
			{
				statName = expression[..equalsIndex];
				expression = expression[(equalsIndex + 1)..];
			}
			else
			{
				statName = expression.Replace("'", " ");
			}

			// Check for collisions
			if (GetStat(statName) != null) 
			{
				Console.WriteLine("Derived expression stat name already exists: " + statName + ". Ignoring.");
				return null;
			}

			expression = expression.ToLower();

			Dictionary<string, StatSamples> referencedStats = new Dictionary<string, StatSamples>();
			string finalExpression;
			try
			{
				finalExpression = PreProcessExpression(expression, referencedStats);
			}
			catch (DerivedStatException ex)
			{
				if (!(ex is DerivedStatMissingSourceException) || bVerboseErrors)
				{
					Console.WriteLine("Error: derived stat expression \"" + expression + "\"could not be computed. Preprocessing error:" + ex.ToString());
				}
				return null;
			}

			// Construct the datatable with any referenced stats
			DataTable dataTable = new DataTable();
			foreach (KeyValuePair<string, StatSamples> entry in referencedStats)
			{
				dataTable.Columns.Add(entry.Key, typeof(float));
			}
			StatSamples[] statsList = referencedStats.Values.ToArray();
			for (int i=0; i<SampleCount; i++)
			{
				DataRow row = dataTable.NewRow();
				for (int j=0; j<statsList.Length; j++)
				{
					row[j] = statsList[j].samples[i];
				}
				dataTable.Rows.Add(row);
			}

			// Add the result column using the expression to compute
			int resultColumnIndex = dataTable.Columns.Count;
			DataColumn resultColumn = new DataColumn("__RESULT__", typeof(float));
			try
			{
				resultColumn.Expression = finalExpression;
				dataTable.Columns.Add(resultColumn);
				dataTable.AcceptChanges();
			}
			catch (Exception ex) 
			{ 
				Console.WriteLine("Error: derived stat expression \""+expression+"\"could not be computed."+ex.ToString());
				return null;
			}

			// Write out the derived stat
			StatSamples statSamples = new StatSamples(statName, dataTable.Rows.Count, true);
			for (int i=0; i<dataTable.Rows.Count; i++)
			{
				DataRow row = dataTable.Rows[i];
				statSamples.samples[i] = (float)row[resultColumnIndex];
			}
			statSamples.ComputeAverageAndTotal();
			return statSamples;
		}

		public StatSamples AddDerivedStatFromExpression(string expression, bool bVerboseErrors = false)
		{
			StatSamples derivedStat = MakeDerivedStatFromExpression(expression, bVerboseErrors);
			if (derivedStat != null)
			{
				AddStat(derivedStat);
			}
			return derivedStat;
		}

		// Splits a list of strings but preserves tokens inside brackets
		public static List<string> SplitStringListWithBracketGroups(string listStr, char separator)
		{
			if (!listStr.Contains('(') && !listStr.Contains('\''))
			{
				// If there are no brackets or quotes, we can just split
				return listStr.Split(separator).Select(token => token.Trim()).ToList();
			}
			int bracketDepth = 0;
			bool bInQuotes = false;
			string token = "";
			List<string> listOut = new List<string>();
			for (int i=0; i<listStr.Length; i++) 
			{ 
				char c = listStr[i];

				// If we're not in brackets then match quotes
				if (bracketDepth == 0 && c == '\'')
				{
					bInQuotes = !bInQuotes;
				}
				// If we're not in quotes then match brackets
				if (!bInQuotes)
				{
					if (c == '(')
					{
						bracketDepth++;
					}
					else if (c == ')')
					{
						bracketDepth--;
						bracketDepth = Math.Max(0, bracketDepth);
					}
				}

				if (!bInQuotes && bracketDepth==0 && c==separator)
				{
					listOut.Add(token.Trim());
					token = "";
				}
				else
				{
					token += c;
				}
			}
			if (token != "")
			{
				listOut.Add(token.Trim());
			}
			return listOut;
		}

		public StatSamples MakeEventCountStat(string statName, string eventStr)
		{
			StatSamples newStat = new StatSamples(statName, SampleCount);
			List<CsvEvent> matchingEvents = GetEventsMatchingString(eventStr);
			foreach (CsvEvent ev in matchingEvents)
			{
				if (ev.Frame < SampleCount)
				{
					newStat.samples[ev.Frame]++;
				}
			}
			newStat.ComputeAverageAndTotal();
			return newStat;
		}

		public List<CsvEvent> GetEventsMatchingString(string eventStr)
		{
			List<CsvEvent> eventsOut = new List<CsvEvent>();
			foreach (CsvEvent ev in Events)
			{
				if (CsvStats.DoesSearchStringMatch(ev.Name, eventStr))
				{
					eventsOut.Add(ev);
				}
			}
			return eventsOut;
		}


		public void AddStat(StatSamples stat)
		{
			Stats.Add(stat.Name.ToLower(), stat);
		}

		public void CropStats(int minFrame, int maxFrame = Int32.MaxValue)
		{
			if (maxFrame <= minFrame)
			{
				return;
			}
			if (minFrame == 0 && maxFrame >= SampleCount)
			{
				return;
			}

			// Remove stats outside the range
			foreach (StatSamples stat in Stats.Values.ToArray())
			{
				int start = Math.Max(minFrame, 0);
				int end = Math.Max(Math.Min(maxFrame, stat.samples.Count), 0);
				if (maxFrame == Int32.MaxValue)
				{
					end = stat.samples.Count;
				}
				int count = end - start;
				List<float> NewSamples = new List<float>(count);
				for (int i = 0; i < count; i++)
				{
					NewSamples.Add(stat.samples[i + start]);
				}
				stat.samples = NewSamples;
			}

			// Filter the filtered events
			List<CsvEvent> newEvents = new List<CsvEvent>();
			foreach (CsvEvent ev in Events)
			{
				if (ev.Frame >= minFrame && ev.Frame <= maxFrame)
				{
					ev.Frame -= minFrame; // Offset the frame based on the new start frame
					newEvents.Add(ev);
				}
			}

			ComputeAveragesAndTotal();

			Events = newEvents;
		}

		static string[] ReadLinesFromFile(string filename)
		{
			StreamReader reader = new StreamReader(filename, true);
			List<string> lines = new List<string>();

			// Detect unicode
			string line = reader.ReadLine();
			if (line == null)
			{
				throw new Exception(filename + " is empty. Aborting further processing to avoid unexpected results.");
			}

			bool bIsUnicode = false;
			for (int i = 0; i < line.Length - 1; i++)
			{
				if (line[i] == '\0')
				{
					bIsUnicode = true;
					break;
				}
			}
			if (bIsUnicode)
			{
				reader.Close();
				reader = new StreamReader(filename, Encoding.Unicode, true);
			}
			else
			{
				lines.Add(line);
			}

			while ((line = reader.ReadLine()) != null)
			{
				// Strip off empty lines
				if (line.Trim().Length > 0)
				{
					lines.Add(line);
				}
			}

			reader.Close();
			return lines.ToArray();
		}

		private static bool WildcardSubstringMatch(string strLower, string[] subSearchStrings)
		{
			if ( subSearchStrings.Length == 0 )
			{
				return false;
			}
			// Make sure the first and last substrings match the start and end
			if (!strLower.StartsWith(subSearchStrings[0]))
			{
				return false;
			}
			if (!strLower.EndsWith(subSearchStrings.Last()))
			{
				return false;
			}

			// Make sure we match all the substrings, in order
			string remainingString = strLower;
			foreach (string subString in subSearchStrings)
			{
				int pos = remainingString.IndexOf(subString);
				if (pos == -1)
				{
					return false;
				}
				remainingString = remainingString.Substring(pos + subString.Length);
			}
			return true;
		}

		public static bool DoesSearchStringMatch(string str, string searchString)
		{
			string searchStringLower = searchString.Trim().ToLower();
			if (searchStringLower.Contains("*"))
			{
				// Break the search string into substrings and check each of the substrings appears in order
				string[] subSearchStrings = searchStringLower.Split('*');
				return WildcardSubstringMatch(str.ToLower(), subSearchStrings);
			}
			else
			{
				return searchStringLower == str.ToLower();
			}
		}

		public static string GetColorFromString(string input, int minBrightness = 64, int maxBrightness = 200)
		{
			using (var md5 = MD5.Create())
			{
				byte[] hash = md5.ComputeHash(Encoding.UTF8.GetBytes(input));
				byte r = (byte)(hash[0] % (maxBrightness - minBrightness) + minBrightness);
				byte g = (byte)(hash[1] % (maxBrightness - minBrightness) + minBrightness);
				byte b = (byte)(hash[2] % (maxBrightness - minBrightness) + minBrightness);
				return $"#{r:X2}{g:X2}{b:X2}";
			}
		}

		public static List<string> WildcardMatchStringList(List<string> stringList, string searchString, bool bCaseSensitive = false, bool bListIsSorted = false)
		{
			searchString = searchString.Trim();
			if (!bCaseSensitive)
			{
				searchString = searchString.ToLower();
			}
			int startIndex = 0;
			int endIndex = stringList.Count;

			string[] wildcardSearchSubStrings = null;
			if (searchString.Contains('*'))
			{
				wildcardSearchSubStrings = searchString.Split('*');
			}

			// If the list is sorted and there's no leading wildcard, we can binary search to limit the search range
			if (bListIsSorted && !searchString.StartsWith('*'))
			{
				// In the wildcard case we're searching for the first term
				string searchPrefix = wildcardSearchSubStrings == null ? searchString : wildcardSearchSubStrings[0];
				int searchEndIndex = endIndex;

				// Binary search for the first matching element
				int prevMidIndex = -1;
				while (true)
				{
					int midIndex = (startIndex + searchEndIndex) / 2;
					if ( prevMidIndex == midIndex )
					{
						// No matches! Early out
						return new List<string>();
					}
					string thisStr = bCaseSensitive ? stringList[midIndex] : stringList[midIndex].ToLower();
					if (thisStr.StartsWith(searchPrefix))
					{
						string prevStr = midIndex > 0 ? (bCaseSensitive ? stringList[midIndex - 1] : stringList[midIndex - 1].ToLower()) : null;
						if (prevStr == null || !prevStr.StartsWith(searchPrefix) )
						{
							// Exact find. Early out
							startIndex = midIndex;
							break;
						}
						else
						{
							searchEndIndex = midIndex;
						}
					}
					else if (searchPrefix.CompareTo(thisStr) < 0)
					{
						searchEndIndex = midIndex;
					}
					else
					{
						startIndex = midIndex;
					}
					prevMidIndex = midIndex;
				}


				// Linear search to find the end of the region
				for ( int i=startIndex+1; i<endIndex; i++ )
				{
					string thisStr = bCaseSensitive ? stringList[i] : stringList[i].ToLower();
					if (!thisStr.StartsWith(searchPrefix))
					{
						endIndex = i;
						break;
					}
				}

				// We don't need a subsequent search if there's no wildcard or this is a single RHS wildcard
				if (wildcardSearchSubStrings == null || (wildcardSearchSubStrings.Length == 2 && wildcardSearchSubStrings[1].Length == 0))
				{
					return stringList.GetRange(startIndex, endIndex-startIndex);
				}
			}


			List<string> results = new List<string>();
			if (wildcardSearchSubStrings != null)
			{
				for (int i = startIndex; i<endIndex; i++)
				{
					if (WildcardSubstringMatch(bCaseSensitive ? stringList[i] : stringList[i].ToLower(), wildcardSearchSubStrings))
					{
						results.Add(stringList[i]);
					}
				}
			}
			else
			{
				for (int i = startIndex; i < endIndex; i++)
				{
					if ((bCaseSensitive ? stringList[i] : stringList[i].ToLower()) == searchString)
					{
						results.Add(stringList[i]);
					}
				}
			}
			return results;
		}

		public HashSet<StatSamples> GetStatsMatchingStringList_Set(string[] statNames)
		{
			HashSet<StatSamples> uniqueStats = new HashSet<StatSamples>();
			foreach (string statStr in statNames)
			{
				List<StatSamples> statsMatching = GetStatsMatchingString(statStr);
				foreach (StatSamples stat in statsMatching)
				{
					if (!uniqueStats.Contains(stat))
					{
						uniqueStats.Add(stat);
					}
				}
			}
			return uniqueStats;
		}
		public List<StatSamples> GetStatsMatchingStringList(string[] statNames)
		{
			return GetStatsMatchingStringList_Set(statNames).ToList();
		}

		public HashSet<string> GetStatNamesMatchingStringList_Set(string[] statNames)
		{
			HashSet<StatSamples> uniqueStats = GetStatsMatchingStringList_Set(statNames);
			HashSet<string> statNamesOut = new HashSet<string>();
			foreach (StatSamples stat in uniqueStats)
			{
				statNamesOut.Add(stat.Name);
			}
			return statNamesOut;
		}

		public List<string> GetStatNamesMatchingStringList(string[] statNames)
		{
			return GetStatNamesMatchingStringList_Set(statNames).ToList();
		}

		public List<StatSamples> GetStatsMatchingString(string statString)
		{
			statString = statString.Trim().ToLower();
			string[] wildcardSearchSubStrings = null;
			if (statString.Contains("*"))
			{
				wildcardSearchSubStrings = statString.Split('*');
			}
			List<StatSamples> statList = new List<StatSamples>();
			if (wildcardSearchSubStrings != null)
			{
				foreach (StatSamples stat in Stats.Values)
				{
					if (WildcardSubstringMatch(stat.Name.ToLower(), wildcardSearchSubStrings))
					{
						statList.Add(stat);
					}
				}
			}
			else
			{
				foreach (StatSamples stat in Stats.Values)
				{
					if (stat.Name.ToLower() == statString)
					{
						statList.Add(stat);
					}
				}

			}
			return statList;
		}

		private static bool GetFlag(uint flags, CsvBinFlags flag)
		{
			return (flags & (uint)flag) != 0;
		}

		private static void SetFlag(ref uint flags, CsvBinFlags flag, bool value)
		{
			if (value)
			{
				flags |= (uint)flag;
			}
			else
			{
				flags &= ~(uint)flag;
			}
		}

		private System.IO.Compression.CompressionLevel GetIOCompressionLevel(CsvBinCompressionLevel compressionLevel)
		{
			switch ( compressionLevel )
			{
				case CsvBinCompressionLevel.None:
					return System.IO.Compression.CompressionLevel.NoCompression;
				case CsvBinCompressionLevel.Min:
					return System.IO.Compression.CompressionLevel.Fastest;
				case CsvBinCompressionLevel.Max:
					return System.IO.Compression.CompressionLevel.Optimal;
			}
			throw new Exception("CompressionLevel not found!");
		}



		public void WriteBinFile(string filename, CsvBinCompressionLevel compressionLevel = CsvBinCompressionLevel.None)
		{
			System.IO.FileStream fileStream = new FileStream(filename, FileMode.Create);
			System.IO.BinaryWriter fileWriter = new System.IO.BinaryWriter(fileStream);

			bool bCompress = (compressionLevel != CsvBinCompressionLevel.None);

			// Write the header
			fileWriter.Write("CSVBIN");
			fileWriter.Write((int)CsvBinVersion.CURRENT);

			bool bHasMetadata = (metaData != null);
			uint flags = 0;
			SetFlag(ref flags, CsvBinFlags.HasMetadata, bHasMetadata);
			fileWriter.Write(flags);

			fileWriter.Write((byte)compressionLevel);
			if (bCompress)
			{
				fileWriter.Write((byte)CsvBinCompressionType.MsDeflate);
			}
			if (bHasMetadata)
			{
				metaData.WriteToBinaryFile(fileWriter);
			}
			fileWriter.Write(Events.Count);
			fileWriter.Write(SampleCount);
			fileWriter.Write(Stats.Count);
			foreach (StatSamples stat in Stats.Values)
			{
				fileWriter.Write(stat.Name);
			}

			// Write the stats
			foreach (StatSamples stat in Stats.Values)
			{
				fileWriter.Write(stat.Name);
				if (stat.samples.Count != SampleCount)
				{
					throw new Exception("Sample count doesn't match!");
				}
				fileWriter.Write(stat.average);
				fileWriter.Write(stat.total);

				int statDataSizeBytes;
				long statStartOffset = fileWriter.BaseStream.Position+4;
				if (bCompress)
				{
					byte[] valuesBuffer = new byte[SampleCount * 4];
					Buffer.BlockCopy(stat.samples.ToArray(), 0, valuesBuffer, 0, valuesBuffer.Length);
					byte[] compressedValuesBuffer = null;
					using (MemoryStream memoryStream = new MemoryStream())
					{
						using (DeflateStream compressionStream = new DeflateStream(memoryStream, GetIOCompressionLevel(compressionLevel)))
						{
							compressionStream.Write(valuesBuffer, 0, valuesBuffer.Length);
						}
						compressedValuesBuffer = memoryStream.ToArray();
					}
					statDataSizeBytes = compressedValuesBuffer.Length + sizeof(int);
					fileWriter.Write(statDataSizeBytes);
					fileWriter.Write(compressedValuesBuffer.Length);
					fileWriter.Write(compressedValuesBuffer);
				}
				else
				{ 
					List<float> uniqueValues = new List<float>();
					FileEfficientBitArray statUniqueMask = new FileEfficientBitArray(SampleCount);
					float oldVal = float.NegativeInfinity;
					for (int i = 0; i < stat.samples.Count; i++)
					{
						float val = stat.samples[i];
						if (val != oldVal || i == 0)
						{
							uniqueValues.Add(val);
							statUniqueMask.Set(i);
						}
						oldVal = val;
					}
					// convert the unique values to a bytearray and compress if necessary
					byte[] uniqueValuesBuffer = new byte[uniqueValues.Count * 4];
					Buffer.BlockCopy(uniqueValues.ToArray(), 0, uniqueValuesBuffer, 0, uniqueValuesBuffer.Length);
					statDataSizeBytes = statUniqueMask.GetSizeBytes() + sizeof(int) + uniqueValuesBuffer.Length;
					fileWriter.Write(statDataSizeBytes);
					statUniqueMask.WriteToFile(fileWriter);
					fileWriter.Write(uniqueValues.Count);
					fileWriter.Write(uniqueValuesBuffer);
				}

				// Check the stat data size matches what we wrote
				int measuredStatDataSize = (int)(fileWriter.BaseStream.Position-statStartOffset);
				if (statDataSizeBytes != measuredStatDataSize)
				{
					throw new Exception("Stat data size is wrong!");
				}

			}
			// Write the events (no point in compressing these - they're tiny and we might want to access them quickly)
			foreach (CsvEvent ev in Events)
			{
				fileWriter.Write(ev.Frame);
				fileWriter.Write(ev.Name);
				fileWriter.Write(ev.Timestamp);
			}
			fileWriter.Close();
		}


		public static CsvStats ReadBinFile(string filename, string[] statNamesToRead=null, int numRowsToSkip=0, bool justHeader=false, CsvFileInfo FileInfoOut=null)
		{
			System.IO.FileStream fileStream = new FileStream(filename, FileMode.Open, FileAccess.Read);
			System.IO.BinaryReader fileReader = new System.IO.BinaryReader(fileStream);

			// Read the header
			if ( fileReader.ReadString() != "CSVBIN" )
			{
				throw new Exception("Failed to read "+filename+". Bad format");
			}
			int version=fileReader.ReadInt32();
			if (version < (int)CsvBinVersion.InitialRelease )
			{
				throw new Exception("Failed to read "+filename+". Version mismatch. Version is "+version.ToString()+". Expected: "+((int)CsvBinVersion.InitialRelease).ToString()+" or later");
			}

			// Read flags
			uint flags = 0;
			CsvBinCompressionLevel compressionLevel = 0;
			bool bCompressed = false;
			if (version>=(int)CsvBinVersion.CompressionSupportAndFlags)
			{
				flags = fileReader.ReadUInt32();
				compressionLevel = (CsvBinCompressionLevel)fileReader.ReadByte();
				bCompressed = compressionLevel != CsvBinCompressionLevel.None;

				if (bCompressed)
				{
					if (fileReader.ReadByte() != (byte)CsvBinCompressionType.MsDeflate)
					{
						throw new Exception("Bad compression type found!");
					}
				}
			}
			else
			{
				bool bHasMetadata = fileReader.ReadBoolean();
				SetFlag(ref flags, CsvBinFlags.HasMetadata, bHasMetadata);
			}

			CsvStats csvStatsOut = new CsvStats();
			if (GetFlag(flags,CsvBinFlags.HasMetadata))
			{
				csvStatsOut.metaData = new CsvMetadata();
				csvStatsOut.metaData.ReadFromBinaryFile(fileReader);
			}
			int eventCount = fileReader.ReadInt32();
			int sampleCount = fileReader.ReadInt32();
			int statCount = fileReader.ReadInt32();

			List<string> statNames = new List<string>();
			for (int s = 0; s < statCount; s++)
			{
				string statName = fileReader.ReadString();
				StatSamples stat = new StatSamples(statName);
				csvStatsOut.AddStat(stat);
				statNames.Add(statName);
			}

			// Get the list of all stats
			HashSet<string> statNamesToReadSet = null;
			if (statNamesToRead != null)
			{
				statNamesToReadSet = csvStatsOut.GetStatNamesMatchingStringList_Set(statNamesToRead);
			}

			// Fill in the file info
			if (FileInfoOut != null)
			{
				FileInfoOut.bIsCsvBin = true;
				FileInfoOut.BinVersion = (CsvBinVersion)version;
				FileInfoOut.BinCompressionLevel = compressionLevel;
				FileInfoOut.SampleCount = sampleCount;
			}

			if (justHeader)
			{
				fileReader.Close();
				return csvStatsOut;
			}

			// Read the stats
			foreach (string headerStatName in statNames)
			{
				string statName = fileReader.ReadString();
				if (statName != headerStatName)
				{
					throw new Exception("Failed to read " + filename + ". Stat name doesn't match header!");
				}
				float statAverage = fileReader.ReadSingle();
				double statTotal = fileReader.ReadDouble();

				int statSizeBytes = fileReader.ReadInt32();
				bool readThisStat = statNamesToReadSet == null || statNamesToReadSet.Contains(statName);
				if (readThisStat == false)
				{
					// If we're skipping this stat then just read the bytes and remove it from the stat list
					byte[] bytes = fileReader.ReadBytes(statSizeBytes);
					if (!csvStatsOut.Stats.Remove(statName.ToLowerInvariant()))
					{
						throw new Exception("Unexpected error. Stat " + statName + " wasn't found!");
					}
					continue;
				}
				StatSamples statOut = csvStatsOut.GetStat(statName);

				statOut.total = statTotal;
				statOut.average = statAverage;

				if ( bCompressed )
				{
					int compressedBufferLength = fileReader.ReadInt32();
					byte[] compressedBuffer = fileReader.ReadBytes(compressedBufferLength);
					int uncompressedBufferLength = sizeof(float) * sampleCount;

					byte[] uncompressedValuesBuffer = new byte[uncompressedBufferLength];
					using (MemoryStream memoryStream = new MemoryStream(compressedBuffer))
					{
						using (DeflateStream decompressionStream = new DeflateStream(memoryStream, CompressionMode.Decompress))
						{
							int offset = 0;
							while(offset<uncompressedBufferLength)
							{
								int bytesRead = decompressionStream.Read(uncompressedValuesBuffer, offset, uncompressedBufferLength - offset);
								if (bytesRead == 0)
								{
									throw new Exception("Decompression error!");
								}
								offset += bytesRead;
							}
							if (offset != uncompressedBufferLength)
							{
								throw new Exception("Decompression error!");
							}
						}
					}

					float[] samples = new float[sampleCount];
					Buffer.BlockCopy(uncompressedValuesBuffer, 0, samples, 0, uncompressedValuesBuffer.Length);
					statOut.samples = samples.ToList();
				}
				else
				{
					// Read the mask
					FileEfficientBitArray statUniqueMask = new FileEfficientBitArray();
					statUniqueMask.ReadFromFile(fileReader);

					int uniqueValueCount = fileReader.ReadInt32();
					float[] uniqueValues = new float[uniqueValueCount];
					byte[] uniqueValuesBuffer = fileReader.ReadBytes(uniqueValueCount * 4);
					Buffer.BlockCopy(uniqueValuesBuffer, 0, uniqueValues, 0, uniqueValuesBuffer.Length);

					// Decode the samples from the mask and unique list
					if (statUniqueMask.Count != sampleCount)
					{
						throw new Exception("Failed to read " + filename + ". Stat sample count doesn't match header!");
					}
					float currentVal = 0;
					int uniqueListIndex = 0;
					statOut.samples.Capacity = statUniqueMask.Count;
					for (int i = 0; i < statUniqueMask.Count; i++)
					{
						if (statUniqueMask.Get(i))
						{
							currentVal = uniqueValues[uniqueListIndex];
							uniqueListIndex++;
						}
						statOut.samples.Add(currentVal);
					}
				}

				if (numRowsToSkip > 0)
				{
					statOut.samples.RemoveRange(0, numRowsToSkip);
				}
			}
			// Read the events
			for (int i = 0; i < eventCount; i++)
			{
				int frame = fileReader.ReadInt32();
				string name = fileReader.ReadString();
				double timestamp = double.MaxValue;
				if (version >= (int)CsvBinVersion.EventTimestamps)
				{
					timestamp = fileReader.ReadDouble();
				}
				if (frame >= numRowsToSkip)
				{
					CsvEvent ev = new CsvEvent(name, frame, timestamp);
					csvStatsOut.Events.Add(ev);
				}
			}
			fileReader.Close();
			return csvStatsOut;
		}

		public void WriteToCSV(string filename, bool bWriteMetadataRow = true)
        {
            System.IO.StreamWriter csvOutFile;
            csvOutFile = new System.IO.StreamWriter(filename);

            // Store the event text in a dictionary so we can lookup by frame
            Dictionary<int, string> eventsLookup = new Dictionary<int, string>();
            foreach (CsvEvent ev in Events)
            {
				string eventText = ev.HasTimestamp() ? ev.Name + "##" + ev.Timestamp.ToString(CultureInfo.InvariantCulture) : ev.Name;
                if (!eventsLookup.ContainsKey(ev.Frame))
                {
                    eventsLookup.Add(ev.Frame, eventText);
                }
                else
                {
                    eventsLookup[ev.Frame] += ";" + eventText;
                }
            }

			// ensure there's an events stat
			bool bAddedEventsStat = false;
            if (Events.Count > 0)
            {
                if (!Stats.ContainsKey("events"))
                {
                    Stats.Add("events", new StatSamples("events"));
					bAddedEventsStat = true;
				}
            }

            // Write the headings
            int sampleCount = Int32.MaxValue;
            bool first = true;
            StringBuilder sb = new StringBuilder();
            int statIndex = 0;
            foreach (StatSamples srcStat in Stats.Values.ToArray())
            {
                if (first)
                {
                    sampleCount = srcStat.samples.Count;
                }

                if (sampleCount > 0 || srcStat.Name.ToLower() == "events")
                {
                    if (!first)
                    {
                        sb.Append(',');
                    }
                    sb.Append(srcStat.Name);
                    first = false;
                }
                statIndex++;
            }
            csvOutFile.WriteLine(sb);
            sb.Clear();

            // Write the stats
            if (sampleCount == Int32.MaxValue)
            {
                sampleCount = 0;
            }
            for (int i = 0; i < sampleCount; i++)
            {
                first = true;
                foreach (StatSamples srcStat in Stats.Values.ToArray())
                {
                    if (!first)
                    {
                        sb.Append(',');
                    }
                    if (srcStat.samples.Count == 0 && srcStat.Name.ToLower() == "events")
                    {
                        if (eventsLookup.ContainsKey(i))
                        {
                            sb.Append(eventsLookup[i]);
                        }
                    }
                    else
                    {
                        if (i < srcStat.samples.Count)
                        {
                            sb.Append(srcStat.samples[i]);
                        }
                        else
                        {
                            sb.Append("0");
                        }
                    }
                    first = false;
                }
                csvOutFile.WriteLine(sb);
                sb.Clear();
            }

            if (metaData != null && bWriteMetadataRow )
            {
                int index = 0;
				bool bFoundCommandline = false;
				KeyValuePair<string, string> commandlinePair = new KeyValuePair<string,string>();

				foreach (KeyValuePair<string, string> pair in metaData.Values)
                {
                    if (index > 0)
                    {
                        sb.Append(",");
                    }
					string value = pair.Value;

					// Override header row at end metadata, because it's not
					if (pair.Key.ToLower() == "hasheaderrowatend")
					{
						value = "0";
					}
					if (pair.Key.ToLower() == "commandline")
					{
						// Defer commandline to last
						bFoundCommandline = true;
						commandlinePair = pair;
						continue;
					}
					sb.Append("["+pair.Key + "]," + value);
                    index++;
                }

				// Force the commandline to be output last
				if (bFoundCommandline)
				{
					if (index > 0)
					{
						sb.Append(",");
					}
					string value = "\"" + commandlinePair.Value + "\"";
					sb.Append("[" + commandlinePair.Key + "]," + value);
				}
			}
            csvOutFile.WriteLine(sb);
            csvOutFile.Close();

			// Remove the temporary events stat if we added it
			if (bAddedEventsStat)
			{
				Stats.Remove("events");
			}
			// Write the metadata
		}

		// Pad the stats
		public void ComputeAveragesAndTotal()
        {
            foreach (StatSamples stat in Stats.Values.ToArray())
            {
                stat.ComputeAverageAndTotal();
            }
        }

        public void Combine(CsvStats srcStats, bool sumRows = false, bool computeAverage = true)
        {
            if (metaData != null)
            {
                metaData.CombineAndValidate(srcStats.metaData);
            }

            // Collate the stats, removing ones which don't exist in both
            int EventFrameOffset = 0;
            Dictionary<string, StatSamples> newStats = new Dictionary<string, StatSamples>();
            foreach (StatSamples srcStat in srcStats.Stats.Values.ToArray())
            {
                if (Stats.ContainsKey(srcStat.Name.ToLower()))
                {
                    StatSamples destStat = GetStat(srcStat.Name);
					if (sumRows)
					{
						int n = Math.Min(srcStat.samples.Count, destStat.samples.Count);
						// Combine initial stats
						for (int i=0;i<n; i++)
						{
							destStat.samples[i] += srcStat.samples[i];
						}
						// Add the remaining stats
						for (int i=n; i< srcStat.samples.Count; i++)
						{
							destStat.samples.Add(srcStat.samples[i]);
						}
					}
					else
					{
						EventFrameOffset = Math.Max(EventFrameOffset, destStat.samples.Count);
						foreach (float sample in srcStat.samples)
						{
							destStat.samples.Add(sample);
						};
					}
                    newStats.Add(srcStat.Name.ToLower(), destStat);
                }
            }

            if (computeAverage)
            {
                // Compute the new average
                foreach (StatSamples stat in newStats.Values.ToArray())
                {
                    stat.ComputeAverageAndTotal();
                }
            }

			if ( sumRows)
			{
				// just remove events. Averaging is complicated/doesn't really make sense
				Events.Clear();
			}
			else
			{
				// Add the events, offsetting the frame numbers
				foreach (CsvEvent ev in srcStats.Events)
				{
					ev.Frame += EventFrameOffset;
					Events.Add(ev);
				}
			}
			
            Stats = newStats;
        }


		// This will average only those stats that exist in all the stat sets
		// Stats are averaged per frame, and the the final length is be equal to the longest CSV
		// If CSVs are of varying length, this means later frames will be averaged over fewer samples than earlier frames
		public static CsvStats AverageByFrame(CsvStats[] statsToAvg, bool bStatsAvarage = false)
        {
            CsvStats avgStats = new CsvStats();
            if (statsToAvg.Length > 0)
            {
                // We need to have all the frame per each stat in the file
                int maxFrames = 0;

                // Use the first as stat name basis
                string[] statKeys = statsToAvg[0].Stats.Keys.ToArray();
                foreach (string statName in statKeys)
                {
                    int maxSamples = 0;
                    int statCount = 0;
                    foreach (CsvStats stats in statsToAvg)
                    {
                        // Remove from the set of new stats if
                        // it doesn't exist in one of the set.
                        if (stats.Stats.ContainsKey(statName))
                        {
                            maxSamples = Math.Max(stats.Stats[statName].samples.Count, maxSamples);
                            statCount++;
                        }
                    }

                    if (statCount == statsToAvg.Length && maxSamples > 0)
                    {
                        avgStats.AddStat(new StatSamples(statName));
                        maxFrames = Math.Max(maxFrames, maxSamples);
                    }
                }

                // Copy meta data
                avgStats.metaData = statsToAvg[0].metaData;
                if (avgStats.metaData != null)
                {
                    foreach (CsvStats stats in statsToAvg)
                    {
                        avgStats.metaData.CombineAndValidate(stats.metaData);
                    }
                }

                foreach (string statName in avgStats.Stats.Keys)
                {
                    // This should always exist
                    StatSamples avgSamples = avgStats.GetStat(statName);
                    if (avgSamples != null)
                    {
						List<int> sampleCounts = new List<int>();
						sampleCounts.AddRange(Enumerable.Repeat(0, maxFrames));

						// Initialise sample to 0.0
						avgSamples.samples.AddRange(Enumerable.Repeat(0.0f, maxFrames));

                        // Add samples from other stats
                        foreach (CsvStats stats in statsToAvg)
                        {
                            StatSamples statSamples = stats.GetStat(statName);
                            if ((statSamples != null) && (avgSamples.samples.Count >= statSamples.samples.Count))
                            {
								// This should always be true: avgSamples.samples.Count >= statSamples.samples.Count
								for (int i = 0; i < statSamples.samples.Count; i++)
								{
									avgSamples.samples[i] += statSamples.samples[i];
									sampleCounts[i] += 1;
                                }
                            }
                        }

                        // Average the samples
                        for (int i = 0; i < avgSamples.samples.Count; i++)
                        {
                            avgSamples.samples[i] /= (float)sampleCounts[i];
                        }
                    }

                    if (bStatsAvarage)
                    {
                        avgSamples.ComputeAverageAndTotal();
                    }
                }
            }

            return avgStats;
        }


        public static CsvMetadata ReadCSVMetadata(string csvFilename)
        {
            string lastLine = File.ReadLines(csvFilename).Last();
            
            if (LineIsMetadata(lastLine))
            {
                string[] values = lastLine.Split(',');
                return new CsvMetadata(values);
            }
            return null;
        }
        public static CsvMetadata ReadCSVMetadataFromLines(string [] csvFileLines)
        {
            string lastLine = csvFileLines.Last();
            if (LineIsMetadata(lastLine))
            {
                string[] values = lastLine.Split(',');
                return new CsvMetadata(values);
            }
            return null;
        }

        public void GetEventFrameIndexDelimiters(string startString, string endString, out List<int> startIndices, out List<int> endIndices)
        {
            bool startIsWild = false;
            bool endIsWild = false;

			if (startString != null)
			{
				startString = startString.ToLower().Trim();
				if (startString.EndsWith("*"))
				{
					startIsWild = true;
					startString = startString.TrimEnd('*');
				}
			}
			if (endString != null)
            {
                endString = endString.ToLower().Trim();
                if (endString.EndsWith("*"))
                {
                    endIsWild = true;
                    endString = endString.TrimEnd('*');
                }
            }

			bool insidePair = false;
            startIndices = new List<int>();
            endIndices = new List<int>();

			// If we only have an endstring, we need to check if it actually appears
			if (startString == null && endString != null)
			{
				// Find the frame of the last end event
				int lastEndEventFrame = -1;
				for (int i = 0; i < Events.Count; i++)
				{
					CsvEvent csvEvent = Events[i];
					string evName = csvEvent.Name.ToLower();
					if (endIsWild && evName.StartsWith(endString) || evName == endString)
					{
						lastEndEventFrame = csvEvent.Frame;
					}
				}
				if (lastEndEventFrame >= 0)
				{
					// If we found the end event, strip everything before it
					startIndices.Add(0);
					endIndices.Add(lastEndEventFrame);
				}
			}
			else
			{
				// Normal case: we have a begin event. End event is optional
				for (int i = 0; i < Events.Count; i++)
				{
					CsvEvent csvEvent = Events[i];
					string evName = csvEvent.Name.ToLower();
					string strToMatch = insidePair ? endString : startString;
					bool isWild = insidePair ? endIsWild : startIsWild;

					bool found = false;
					if (strToMatch != null)
					{
						if (isWild)
						{
							if (evName.StartsWith(strToMatch))
							{
								found = true;
							}
						}
						else if (evName == strToMatch)
						{
							found = true;
						}
					}

					if (found)
					{
						if (insidePair)
						{
							endIndices.Add(csvEvent.Frame);
						}
						else
						{
							startIndices.Add(csvEvent.Frame);
						}
						insidePair = !insidePair;
					}
				}

				// If the end event was missing, add it at the end
				if (endIndices.Count == startIndices.Count - 1)
				{
					endIndices.Add(SampleCount);
				}
			}
        }

		public void ComputeEventStripSampleMask(string startString, string endString, ref BitArray sampleMask)
		{
			List<int> startIndices = null;
			List<int> endIndices = null;
			GetEventFrameIndexDelimiters(startString, endString, out startIndices, out endIndices);
			if (startIndices.Count == 0)
			{
				return;
			}
			if (sampleMask == null)
			{
				sampleMask = new BitArray(SampleCount);
				sampleMask.SetAll(true);
			}

			for (int i=0; i<startIndices.Count; i++)
			{
				int startIndex = startIndices[i];
				int endIndex = Math.Min(endIndices[i],SampleCount-1);
				for (int j=startIndex; j<=endIndex; j++)
				{
					sampleMask.Set(j, false);
				}
			}
		}

		StatSamples ApplySampleMaskToSamples(StatSamples sourceStat, BitArray sampleMask)
		{
			StatSamples destStat = new StatSamples(sourceStat, false);
			destStat.samples = new List<float>(SampleCount);
			for (int i = 0; i < sourceStat.samples.Count; i++)
			{
				if (sampleMask.Get(i))
				{
					destStat.samples.Add(sourceStat.samples[i]);
				}
			}
			destStat.ComputeAverageAndTotal();
			return destStat;
		}

		public CsvStats ApplySampleMask(BitArray sampleMask, bool parallel=false)
		{
			CsvStats newStats = new CsvStats();
			newStats.metaData = metaData;

			if (parallel)
			{
				StatSamples[] destStatsArray = new StatSamples[Stats.Values.Count];
				Parallel.For(0, Stats.Values.Count, s =>
				{
					StatSamples srcStat = Stats.Values.ElementAt(s);
					destStatsArray[s] = ApplySampleMaskToSamples(srcStat, sampleMask);
				});
				foreach (StatSamples stat in destStatsArray)
				{
					newStats.AddStat(stat);
				}
			}
			else
			{
				// Strip out samples and recompute averages
				foreach (StatSamples srcStat in Stats.Values)
				{
					StatSamples destStat = ApplySampleMaskToSamples(srcStat, sampleMask);
					newStats.AddStat(destStat);
				}
			}

			// Strip and offset events
			if (Events.Count > 0)
			{
				List<CsvEvent> newEvents=new List<CsvEvent>(Events.Count);
				int eventIndex = 0;
				int skippedFrameCount = 0;
				for (int i = 0; i < SampleCount; i++)
				{
					bool bFrameExists = sampleMask.Get(i);
					// Loop through all the events on this frame
					while (eventIndex<Events.Count && Events[eventIndex].Frame==i)
					{
						CsvEvent currentEvent=Events[eventIndex];
						if (bFrameExists)
						{
							newEvents.Add(new CsvEvent(currentEvent.Name, currentEvent.Frame-skippedFrameCount, currentEvent.Timestamp));
						}
						eventIndex++;
					}
					if (!bFrameExists)
					{
						skippedFrameCount++;
					}
				}
				newStats.Events = newEvents;
			}
			return newStats;
		}

        public int SampleCount
        {
            get
            {
                if ( Stats.Count == 0 )
                {
                    return 0;
                }
                else
                {
                    return Stats.First().Value.samples.Count;
                }
            }
        }

		public static bool DoesMetadataMatchFilter(CsvMetadata metadata, string metadataFilterString)
		{
			QueryExpression expressionTree = MetadataQueryBuilder.BuildQueryExpressionTree(metadataFilterString);
			return expressionTree.Evaluate(metadata);
		}

		public static bool IsDerivedStatExpression(string stat)
		{
			return stat.StartsWith('(') && stat.EndsWith(')') || stat.Contains('=');
		}


		public static CsvStats ReadCSVFile(string csvFilename, string[] statNames=null, int numRowsToSkip = 0, bool bGenerateCsvIdIfMissing=false, CsvFileInfo FileInfoOut=null, bool bJustHeader=false)
        {
			CsvStats statsOut;
			if (csvFilename.EndsWith(".csv.bin"))
			{
				statsOut = ReadBinFile(csvFilename, statNames, numRowsToSkip, bJustHeader, FileInfoOut);
			}
			else
			{
				string[] lines = ReadLinesFromFile(csvFilename);
				statsOut = ReadCSVFromLines(lines, statNames, numRowsToSkip, bJustHeader, FileInfoOut);
			}
			if (bGenerateCsvIdIfMissing)
			{
				if (statsOut.metaData != null)
				{
					statsOut.GenerateCsvIDIfMissing();
				}
				else
				{
					Console.WriteLine("Warning: Csv file has no metadata. Skipping adding a CsvID");
				}
			}
			return statsOut;
		}

		public bool GenerateCsvIDIfMissing()
		{
			if (metaData == null)
			{
				throw new Exception("GenerateCsvIDIfMissing called for a CSV with no metadata!");
			}
			if (metaData.Values.ContainsKey("csvid"))
			{
				// Nothing to do 
				return false;
			}
			metaData.Values["csvid"] = MakeCsvIdHash();
			metaData.Values["hastoolgeneratedid"] = "1";
			return true;
		}

		public string MakeCsvIdHash()
		{
			string newId;
			using (MemoryStream memStream = new MemoryStream())
			{
				using (BinaryWriter writer = new BinaryWriter(memStream))
				{
					writer.Write(SampleCount);
					writer.Write(Stats.Values.Count);
					writer.Write(Events.Count);
					foreach (StatSamples stat in Stats.Values)
					{
						writer.Write(stat.Name);
						writer.Write(stat.total);
					}
					foreach ( CsvEvent ev in Events)
					{
						writer.Write(ev.Name);
						writer.Write(ev.Frame);
					}
					if (metaData != null)
					{
						foreach (string key in metaData.Values.Keys)
						{
							if (key != "csvid" && key != "hastoolgeneratedid" && key != "hasheaderrowatend")
							{
								writer.Write(key);
								writer.Write(metaData.Values[key]);
							}
						}
					}
					byte[] hashInputBuffer = memStream.ToArray();
					using (SHA256 sha256 = SHA256.Create())
					{
						byte[] hash = sha256.ComputeHash(hashInputBuffer);
						newId = BitConverter.ToString(hash).Replace("-", string.Empty).Substring(0,32);
					}
				}
			}
			return newId;
		}


		static bool LineIsMetadata(string line)
        {
            if (line.Trim().StartsWith("["))
            {
                return true;
            }
            return false;
        }

        public static CsvStats ReadCSVFromLines(string[] linesArray, string[] statNames=null, int numRowsToSkip = 0, bool skipReadingData=false, CsvFileInfo fileInfo=null)
        {
            List<string> lines = linesArray.ToList();

            // Check if we have any metadata
            bool bHasMetaData = LineIsMetadata(lines.Last());

			CsvMetadata metaData = null;

			// Add the metadata to the stats collection
			if (bHasMetaData)
			{
				string[] lastLine = lines.Last().Split(',');
				metaData = new CsvMetadata(lastLine);

				// New CSVs from the csv profiler have a header row at the end of the file,
				// since the profiler writes out the file incrementally.
				if ("1" == metaData.GetValue("hasheaderrowatend", null))
				{
					// Swap the header row for the one at the end of the file,
					// then remove the end one.
					lines[0] = lines[lines.Count - 2];
					lines.RemoveAt(lines.Count - 2);
				}
			}

			if ( numRowsToSkip > 0)
            {
                lines.RemoveRange(1, numRowsToSkip);
            }
			string[] headings = lines[0].Split(',');

            if (lines.Count > 2 && lines[lines.Count - 1] == "\"" )
            {
                lines[lines.Count - 2] += lines[lines.Count - 1];
                lines.RemoveAt(lines.Count - 1);
            }

			// First line is headings, last line contains build info 
			int numSamples = Math.Max(lines.Count - (bHasMetaData ? 2 : 1), 0);
			if (fileInfo != null)
			{
				fileInfo.bIsCsvBin = false;
				fileInfo.SampleCount = numSamples;
			}

			if (skipReadingData)
            {
				lines.RemoveRange(1, numSamples);
				numSamples = 0;
            }

			// Get the list of lower case stat names, expanding wildcards
			string[] statNamesLowercase = null;
            if (statNames != null)
            {
                statNamesLowercase = statNames.Select(s => s.ToLowerInvariant()).ToArray();
				string [] headingsLowercase = headings.Select(s => s.ToLowerInvariant().Trim()).ToArray();
				// Expand the list of stat names based on the wildcards and the headers. We do this here to make sorting simpler
				HashSet<string> newStatNamesLowercase = new HashSet<string>();
                foreach (string statname in statNamesLowercase)
                {
					if (statname.Contains("*"))
                    {
                        foreach (string headingStat in headingsLowercase)
                        {
							if ( DoesSearchStringMatch(headingStat, statname) )
							{
								newStatNamesLowercase.Add(headingStat.ToLower());
							}
                        }
                    }
                    else
                    {
                        newStatNamesLowercase.Add(statname);
                    }
                }
                statNamesLowercase = newStatNamesLowercase.ToArray();
            }

            // Create the stats
            int eventHeadingIndex = -1;
            StatSamples[] stats = new StatSamples[headings.Length];
            for (int i = 0; i < headings.Length; i++)
            {
                string heading = headings[i].Trim();
				if ( heading == "")
				{
					continue;
				}
                // find the events column (if there is one)
                else if (heading.ToLower() == "events")
                {
                    eventHeadingIndex = i;
                }
                else if (statNamesLowercase == null || statNamesLowercase.Contains(heading.ToLower()))
                {
                    stats[i] = new StatSamples(heading, numSamples);
                }
            }


            List<CsvEvent> FilteredEvents = new List<CsvEvent>();

            if (!skipReadingData)
            {
                string[] eventStrings = new string[numSamples];

//                for (int i = 1; i < numSamples + 1; i++)
                Parallel.For(1, numSamples + 1,
                                     i =>
                {
                    int sampleIndex = i - 1;
                    int statIndex = 0;
                    string line = lines[i] + "\n";
                    for (int j = 0; j < line.Length; j++)
                    {
                        // Note: we check statIndex<stats.length here in case of truncated CSVs
                        if (statIndex < stats.Length && stats[statIndex] != null)
                        {
                            // Read the stat
                            float value = 0.0f;

                            // Skip whitespace
                            while (line[j] == ' ')
                            {
                                j++;
                            }

                            bool negative = false;
                            if ( line[j] == '-' )
                            {
                                negative = true;
                                j++;
                            }

                            // Read the nonfractional part of the number
                            int num = 0;
                            while (line[j] >= '0' && line[j] <= '9')
                            {
                                num *= 10;
                                num += line[j] - '0';
                                j++;
                            }
                            value = (float)num;

                            if (line[j] == '.')
                            {
                                // read fractional part
                                num = 0;
                                j++;
                                float multiplier = 0.1f;
                                while (line[j] >= '0' && line[j] <= '9')
                                {
                                    value += (float)(line[j] - '0') * multiplier;
                                    j++;
                                    multiplier *= 0.1f;
                                }
                            }

                            if ( negative )
                            {
                                value = -value;
                            }

                            stats[statIndex].samples[sampleIndex] = value;

                            // Skip everything else until the next newline or comma
                            while (line[j] != ',' && line[j] != '\n')
                            {
                                j++;
                            }
                        }
                        else
                        {
                            // Skip parsing
                            int startJ = j;
                            while (line[j] != ',' && line[j] != '\n')
                            {
                                j++;
                            }
                            if (statIndex == eventHeadingIndex)
                            {
                                eventStrings[sampleIndex] = line.Substring(startJ, j - startJ);
                            }
                        }
                        statIndex++;
                    }
                }
                ); // Needed by parallel for

                // Read events
                for (int i = 0; i < eventStrings.Length; i++)
                {
                    string eventString = eventStrings[i];
                    if (!string.IsNullOrEmpty(eventString))
                    {
                        string[] Events = eventString.Split(';');
                        foreach (string EventString in Events)
                        {
                            if (EventString.Length > 0)
                            {
								double timestamp = double.MaxValue;
								string name = EventString;

								// Check if there's a timestamp embedded
								int suffixIndex = EventString.LastIndexOf("##");
								if (suffixIndex != -1)
								{
									name = EventString.Substring(0, suffixIndex);
									if ( double.TryParse(EventString.Substring(suffixIndex + 2), NumberStyles.Float, CultureInfo.InvariantCulture, out double result) )
									{
										timestamp = result;
									}
									
								}

								CsvEvent ev = new CsvEvent(name, i, timestamp);
                                FilteredEvents.Add(ev);
                            }
                        }
                    }
                }
            }

            // Make sure the stat ordering matches the order they're passed in
            CsvStats csvStats = new CsvStats();

            if (statNamesLowercase != null)
            {
                CsvStats unorderedCsvStats = new CsvStats();
                foreach (StatSamples statSamples in stats)
                {
                    if (statSamples != null)
                    {
                        // Combine stats if we find a duplicate
                        if (unorderedCsvStats.Stats.ContainsKey(statSamples.Name.ToLower()))
                        {
                            StatSamples existingStat = unorderedCsvStats.GetStat(statSamples.Name);
                            for (int i = 0; i < statSamples.samples.Count; i++)
                            {
                                existingStat.samples[i] += statSamples.samples[i];
                            }
                        }
                        else
                        {
                            unorderedCsvStats.AddStat(statSamples);
                        }
                    }
                }

                foreach (string statName in statNamesLowercase)
                {
                    StatSamples stat = unorderedCsvStats.GetStat(statName);
                    if (stat != null)
                    {
                        csvStats.AddStat(stat);
                    }
                }
            }
            else
            {
                int c = 0;
                foreach (StatSamples statSamples in stats)
                {
                    c++;
                    if (statSamples != null)
                    {
                        if (csvStats.Stats.ContainsKey(statSamples.Name.ToLower()))
                        {
                            // Combine stats if we find a duplicate                
                            StatSamples existingStat = csvStats.GetStat(statSamples.Name);
                            for (int i = 0; i < statSamples.samples.Count; i++)
                            {
                                existingStat.samples[i] += statSamples.samples[i];
                            }
                        }
                        else
                        {
                            csvStats.AddStat(statSamples);
                        }
                    }
                }
            }

            // Compute averages
            foreach (StatSamples stat in csvStats.Stats.Values.ToArray())
            {
                stat.ComputeAverageAndTotal();
            }

			csvStats.metaData = metaData;

            csvStats.Events = FilteredEvents;
            return csvStats;
        }


        public Dictionary<string, StatSamples> Stats;
        public List<CsvEvent> Events;
        public CsvMetadata metaData;
    };


    public class Colour
    {
		public Colour(byte rIn, byte gIn, byte bIn, float alphaIn = 1.0f) { r = rIn; g = gIn; b = bIn; alpha = alphaIn; }
        public Colour(uint hex, float alphaIn = 1.0f)
        {
            r = (byte)((hex >> 16) & 0xff);
            g = (byte)((hex >> 8) & 0xff);
            b = (byte)((hex >> 0) & 0xff);
            alpha = alphaIn;
        }
        public Colour(Colour colourIn) { r = colourIn.r; g = colourIn.g; b = colourIn.b; alpha = colourIn.alpha; }

		public Colour(string str)
		{
			string hexStr = str.TrimStart('#');
			int hex = Convert.ToInt32(hexStr, 16);
			r = (byte)((hex >> 16) & 0xff);
			g = (byte)((hex >> 8) & 0xff);
			b = (byte)((hex >> 0) & 0xff);
			alpha = 1.0f;
		}

		public string SVGString()
        {
            return "'" + SVGStringNoQuotes() + "'";
        }
        public string SVGStringNoQuotes()
        {
            if (alpha >= 1.0f)
            {
                return "rgb(" + r + ", " + g + ", " + b + ")";
            }
            else
            {
                return "rgba(" + r + ", " + g + ", " + b + ", " + alpha + ")";
            }
        }

        public static Colour White = new Colour(255, 255, 255);
        public static Colour Black = new Colour(0, 0, 0);

        public byte r, g, b;
        public float alpha;
    };

	// BitArray doesn't allow us to access actual words, which is horribly slow when you need to read/write one bit at a time
	class FileEfficientBitArray
	{
		public FileEfficientBitArray()
		{
			bitCount = 0;
		}
		public FileEfficientBitArray(int inBitCount)
		{
			bitCount = inBitCount;
			int wordCount = ((bitCount + 31) / 32);
			words = new uint[wordCount];
		}

		public bool Get(int bitIndex)
		{
			int wordIndex = bitIndex >> 5;
			int b = bitIndex & 0x0000001F;
			return (words[wordIndex] & (1 << b)) != 0;
		}
		public void Set(int bitIndex)
		{
			int wordIndex = bitIndex >> 5;
			int b = bitIndex & 0x0000001F;
			words[wordIndex] |= (uint)(1 << b);
		}

		public void WriteToFile(BinaryWriter fileWriter)
		{
			fileWriter.Write(bitCount);
			foreach(uint word in words)
			{
				fileWriter.Write(word);
			}
		}

		public void ReadFromFile(BinaryReader fileReader)
		{
			bitCount = fileReader.ReadInt32();
			int wordCount = ((bitCount + 31) / 32);
			words = new uint[wordCount];

			byte[] wordsBuffer= fileReader.ReadBytes(wordCount * 4);
			Buffer.BlockCopy(wordsBuffer, 0, words, 0, wordsBuffer.Length);

			//for (int i=0;i<words.Length;i++)
			//{
			//	words[i]=fileReader.ReadUInt32();
			//}
		}
		public int GetSizeBytes()
		{
			return words.Length * sizeof(uint)+sizeof(int);
		}

		public int Count 
		{
			get { return bitCount; }
		}
		uint[] words;
		int bitCount;
	}

}
