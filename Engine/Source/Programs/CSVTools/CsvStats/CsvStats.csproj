<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Library</OutputType>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
</Project>