using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;

namespace CSVStats
{
	public enum EvalOp
	{
		And,
		Or,
		None,

		COUNT=None
	};

	public enum ComparisonOp
	{
		Equal,
		NotEqual,
		GreaterThan,
		LessThan,
		GreaterThanOrEqual,
		LessThanOrEqual,
		Contains,

		COUNT
	};

	public abstract class QueryExpression
	{
		public abstract bool Evaluate(CsvMetadata metadata);

		public bool bNegate;
	}

	public class QueryComparisonExpression : QueryExpression
	{
		public override bool Evaluate(CsvMetadata metadata)
		{

			bool matches = false;
			bool negate = bNegate;

			double doubleVal;
			if (!Double.TryParse(Value, NumberStyles.Float, CultureInfo.InvariantCulture, out doubleVal))
			{
				doubleVal = double.MaxValue;
			}

			string metaDataValueStr = "NULL";
			double doubleMetadataVal = double.MaxValue;
			bool bNumericComparison = false;
			if (metadata.Values.TryGetValue(Key, out string metaDataValueCased))
			{
				metaDataValueStr = metaDataValueCased.ToLower();
				// Attempt to parse both values as numeric and determine if this is a numeric comparison
				if (!Double.TryParse(metaDataValueStr, NumberStyles.Float, CultureInfo.InvariantCulture, out doubleMetadataVal))
				{
					doubleMetadataVal = double.MaxValue; 
				}
				if(doubleVal != double.MaxValue && doubleMetadataVal != double.MaxValue)
				{
					bNumericComparison = true;
				}
			}

			if (Op == ComparisonOp.Equal || Op == ComparisonOp.NotEqual)
			{
				// Check if the value actually matches (allow wildcards)
				if (bNumericComparison)
				{
					matches = doubleVal == doubleMetadataVal;
				}
				else
				{
					// Do a string comparison (taking wildcards into account)
					matches = CsvStats.DoesSearchStringMatch(metaDataValueStr, Value.ToLower());
				}
				if (Op == ComparisonOp.NotEqual)
				{
					negate = !negate;
				}
			}
			else if (Op == ComparisonOp.LessThan || Op == ComparisonOp.GreaterThan || Op == ComparisonOp.LessThanOrEqual || Op == ComparisonOp.GreaterThanOrEqual)
			{
				if (bNumericComparison)
				{
					switch (Op)
					{
						case ComparisonOp.LessThan:
							matches = doubleMetadataVal < doubleVal;
							break;
						case ComparisonOp.GreaterThan:
							matches = doubleMetadataVal > doubleVal;
							break;
						case ComparisonOp.LessThanOrEqual:
							matches = doubleMetadataVal <= doubleVal;
							break;
						case ComparisonOp.GreaterThanOrEqual:
							matches = doubleMetadataVal >= doubleVal;
							break;
					}
				}
				else
				{
					if (doubleVal == double.MaxValue)
					{
						throw new Exception("Metadata operators expect numeric values!");
					}
					if (doubleMetadataVal == double.MaxValue)
					{
						// Metadata value wasn't numeric, so just return false. This will be the case with NULL
						matches = false;
					}
				}
			}
			else if (Op == ComparisonOp.Contains)
			{
				matches = metaDataValueStr.Contains(Value);
			}
			return negate ? !matches : matches;
		}
		public string Key;
		public string Value;
		public ComparisonOp Op;
	};

	public class QueryLogicOpExpression : QueryExpression
	{
		public QueryLogicOpExpression()
		{
			First = null;
			Second = null;
			Op = EvalOp.None;
		}
		public QueryLogicOpExpression(QueryExpression first, QueryExpression second, EvalOp op)
		{
			First = first;
			Second = second;
			Op = op;
		}
		public QueryExpression First;
		public QueryExpression Second;
		public EvalOp Op;

		public override bool Evaluate(CsvMetadata metadata)
		{
			bool rv = false;
			if (Op == EvalOp.None)
			{
				rv=First.Evaluate(metadata);
			}
			else if (Op == EvalOp.And)
			{
				rv=First.Evaluate(metadata) && Second.Evaluate(metadata);
			}
			else if (Op == EvalOp.Or)
			{
				rv=First.Evaluate(metadata) || Second.Evaluate(metadata);
			}
			return bNegate ? !rv : rv;
		}
	};

	public class MetadataQueryBuilder
	{
		private static QueryComparisonExpression ConsumeTerm(ref List<string> tokenList)
		{
			QueryComparisonExpression termOut = new QueryComparisonExpression();
			if (tokenList.Count < 3)
			{
				return null;
			}
			ComparisonOp op = ComparisonOp.COUNT;
			switch (tokenList[1])
			{
				case "=":
					op = ComparisonOp.Equal;
					break;
				case "!=":
					op = ComparisonOp.NotEqual;
					break;
				case "<":
					op = ComparisonOp.LessThan;
					break;
				case ">":
					op = ComparisonOp.GreaterThan;
					break;
				case "<=":
					op = ComparisonOp.LessThanOrEqual;
					break;
				case ">=":
					op = ComparisonOp.GreaterThanOrEqual;
					break;
				case "~=":
					op = ComparisonOp.Contains;
					break;
				default:
					break;
			}
			if ( op != ComparisonOp.COUNT )
			{
				termOut.Key = tokenList[0];
				termOut.Value = tokenList[2];
				termOut.Op = op;
				tokenList.RemoveRange(0, 3);
				return termOut;
			}
			return null;
		}


		private static QueryExpression ConsumeBracketExpression(ref List<string> tokenList)
		{
			if (FirstToken(tokenList)=="(")
			{
				int bracketDepth = 1;
				for (int i=1;i<tokenList.Count;i++)
				{
					string token = tokenList[i];
					if (token=="(")
					{
						bracketDepth++;
					}
					else if (token == ")")
					{
						bracketDepth--;
						if (bracketDepth == 0)
						{
							List<string> innerTokenList = tokenList.GetRange(1,i-1);
							tokenList.RemoveRange(0, i+1);
							return ConsumeExpression(ref innerTokenList);
						}
					}

				}
			}
			return null;
		}

		private static EvalOp ConsumeOp(ref List<string> tokenList)
		{
			string firstToken = FirstToken(tokenList);
			if (firstToken == null)
			{
				return EvalOp.None;
			}
			if (firstToken=="and")
			{
				tokenList.RemoveAt(0);
				return EvalOp.And;
			}
			if (firstToken == "or")
			{
				tokenList.RemoveAt(0);
				return EvalOp.Or;
			}
			throw new Exception("Error reading term");
		}

		private static bool ConsumeNot(ref List<string> tokenList)
		{
			if (FirstToken(tokenList)=="not")
			{
				tokenList.RemoveAt(0);
				return true;
			}
			return false;
		}

		static string FirstToken(List<string> tokenList)
		{
			if (tokenList.Count > 0)
			{
				return tokenList[0];
			}
			return null;
		}

		private static QueryExpression ConsumeExpression(ref List<string> tokenList)
		{
			List<QueryExpression> expressions = new List<QueryExpression>();
			List<EvalOp> ops = new List<EvalOp>();
			while (true)
			{
				bool bNegate = ConsumeNot(ref tokenList);
				QueryExpression expression = ConsumeBracketExpression(ref tokenList);
				if (expression == null)
				{
					expression = ConsumeTerm(ref tokenList);
					if (expression == null)
					{
						throw new Exception("Error parsing metadata filter expression: "+string.Join("",tokenList));
					}
				}
				expression.bNegate = bNegate;
				expressions.Add(expression);
				EvalOp op = ConsumeOp(ref tokenList);
				if (op == EvalOp.None)
				{
					break;
				}
				ops.Add(op);
			}
			if (expressions.Count == 1)
			{
				return expressions[0];
			}

			// Merge each of the expressions in order of operator precedence
			for ( int opIndex=0; opIndex<(int)EvalOp.COUNT; opIndex++)
			{
				EvalOp op = (EvalOp)opIndex;
				for (int i = 0; i < ops.Count; i++)
				{
					if (ops[i] == op)
					{
						expressions[i] = new QueryLogicOpExpression(expressions[i], expressions[i + 1], op);
						expressions.RemoveAt(i + 1);
						ops.RemoveAt(i);
						i--;
					}
				}
			}
			if (expressions.Count == 1)
			{
				return expressions[0];
			}
			throw new Exception("Filter string parsing error");
		}

		enum CharType
		{
			Standard,
			Operator,
			Whitespace,
		}

		private static string ConsumeNextToken(ref string remainingString)
		{
			string[] operatorTokenArray = { "!=", "<=", ">=", "(", ")", "=", "<", ">", "~=" };
			string operatorChars = "!=<>()~";

			// Consume leading whitespace
			remainingString = remainingString.TrimStart();
			if (remainingString.Length == 0)
			{
				return null;
			}

			// Consume operators
			foreach (string opString in operatorTokenArray)
			{
				if (remainingString.StartsWith(opString))
				{
					remainingString = remainingString.Substring(opString.Length);
					return opString;
				}
			}
			// Not an operator or whitespace
			string token = "";
			for (int i = 0; i < remainingString.Length; i++)
			{
				char c = remainingString[i];
				if ( i>0 && ( Char.IsWhiteSpace(c) || operatorChars.IndexOf(c) >= 0 ))
				{
					break;
				}
				token += c;
			}
			remainingString = remainingString.Substring(token.Length);
			return token;
		}

		private static List<string> Tokenize(string str)
		{
			// Tokenize the string
			string remainingString = str;
			List<string> tokens = new List<string>();
			while (remainingString.Length > 0)
			{
				string token = ConsumeNextToken(ref remainingString);
				if (token != null)
				{
					tokens.Add(token);
				}
			}
			return tokens;
		}

		public static QueryExpression BuildQueryExpressionTree(string metadataFilterString)
		{
			// Fix up old-style comma separated expressions
			metadataFilterString = metadataFilterString.Replace(",", " and ").ToLower();

			List<string> tokenList = Tokenize(metadataFilterString);

			return ConsumeExpression(ref tokenList);
		}

	}

}