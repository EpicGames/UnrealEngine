// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("54811cbd-f2b0-427f-859a-2f93318baab1")]
