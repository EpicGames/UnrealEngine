using System;
using System.Collections.Generic;
using System.Linq;
using CSVStats;
using System.Threading.Tasks;
using System.Text;
using System.ComponentModel;
using System.Diagnostics;
using System.Security.Cryptography;
using System.Xml.Linq;
using System.Collections;

namespace CSVTools
{

	public class CsvToSvgLibVersion
	{
		private static string VersionString = "3.70";

		public static string Get() { return VersionString; }
	};

	public enum ShowEventTextMode
	{
		Hide = 0,
		ShowAuto = 1,
		ShowAll = 2
	};

	public class GraphParams
	{
		public int width = 1800;
		public int height = 550;
		public string title = "";
		public List<string> statNames = new List<string>();
		public List<string> ignoreStats = new List<string>();
		public int colorOffset = 0;

		// Events
		public List<string> showEventNames = new List<string>();
		public ShowEventTextMode showEventNameTextMode = ShowEventTextMode.ShowAuto;
		public List<string> highlightEventRegions = new List<string>();

		// Smoothing
		public bool smooth = false;
		public float smoothKernelPercent = 2.0f;
		public int smoothKernelSize = -1;

		// Other flags
		public bool showMetadata = true; //noMetadata
		public bool graphOnly = false;
		public bool interactive = false;
		public bool snapToPeaks = true; // Interactive mode only
		public int maxHierarchyDepth = -1;
		public char hierarchySeparator = '/';
		public List<string> hideStatPrefixes = new List<string>();

		// Compression
		public float compression = 0.0f;
		public bool bFixedPointGraphs = true;
		public float fixedPointPrecisionScale = 2.0f; // Scale applied to fixed point graphs. Values greater than one allow for subpixel accuracy
		public int lineDecimalPlaces = 3; // Legacy: only used if bFixedPointGraphs is 0

		// Stacking
		public bool stacked = false;
		public string stackTotalStat = "";
		public bool stackedUnsorted = false;

		// Percentiles
		public bool percentileTop90 = false;
		public bool percentileTop99 = false;
		public bool percentile = false;

		// X/Y range
		public float minX = Range.Auto;
		public float maxX = Range.Auto;
		public float minY = Range.Auto;
		public float maxY = Range.Auto;

		// start graphing from this event. Note: this will cause frame numbers to start at zero (which is necessary for multiple graphs)
		public string startEvent;
		public int startEventOffset = 0;
		public string endEvent;
		public int endEventOffset = 0;

		// Max auto Y range. Set to 0 to disable
		public float maxAutoMaxY = 0.0f;

		public float budget = float.MaxValue;
		public float lineThickness = 1.0f;

		// Thresholds
		public float threshold = -float.MaxValue;
		public float averageThreshold = -float.MaxValue;

		public string embedText = "Created with CSVtoSVG "+ CsvToSvgLibVersion.Get();
		public string uniqueId = "ID";

		// Legend
		public bool showAverages = false;
		public bool showTotals = false;
		public bool forceLegendSort = false;
		public float legendAverageThreshold = -float.MaxValue;
		public List<string> customLegendNames = new List<string>();
		public List<string> csvColors = new List<string>();

		// Process options (doesn't affect output)
		public bool bSmoothMultithreaded = true;
		public bool bPerfLog = false;

		// Advanced params
		public string themeName = "";
		public int frameOffset = 0;
		public float statMultiplier = 1.0f;
		public string minFilterStatName = null;
		public float minFilterStatValue = -float.MaxValue;
		public bool filterOutZeros = false;
		public bool discardLastFrame = true;

		private bool bFinalized = false;

		public void FinalizeSettings()
		{
			if (bFinalized)
			{
				return;
			}

			if (percentile && (stacked || interactive))
			{
				throw new Exception("Warning: percentile graph not compatible with stacked & interactive");
			}

			// Make sure stack total stat is in the stat list. This may result in a duplicate, but that's fine
			if (stacked)
			{
				if (stackTotalStat != "")
				{
					stackTotalStat = stackTotalStat.ToLower();
					statNames.Add(stackTotalStat);
				}
			}

			if ( percentileTop90 || percentileTop99 )
			{
				percentile = true;
			}

			themeName = themeName.ToLower();

			if (maxX <= 0.0f)
			{
				maxX = Range.Auto;
			}
			if (maxY <= 0.0f)
			{
				maxY = Range.Auto;
			}
			if (maxAutoMaxY <= 0.0f)
			{
				// Clamp to 8m to prevent craziness
				maxAutoMaxY = 8000000;
			}
			if (smoothKernelSize>0.0f)
			{
				smoothKernelPercent = 0.0f;
			}
			bFinalized = true;
		}


		public void DebugPrintProperties()
		{
			foreach (PropertyDescriptor descriptor in TypeDescriptor.GetProperties(this))
			{
				string name = descriptor.Name;
				object value = descriptor.GetValue(this);
				Console.WriteLine("{0}={1}", name, value);
			}
		}
	};

	public class CsvInfo
	{
		public CsvInfo(CsvStats csvStatsIn, string csvFilenameIn)
		{
			stats = csvStatsIn;
			filename = csvFilenameIn;
		}
		public CsvStats stats;
		public string filename;
	};

	class SvgFile
	{
		public SvgFile(string filename, string inUniqueId=null)
		{
			file = new System.IO.StreamWriter(filename);
			idSuffix = "_" + inUniqueId;
			sb = new StringBuilder();
		}

		public void WriteFast(string str)
		{
			sb.Append(str);
		}

		public void Write(string str)
		{
			sb.Append(str.Replace("<UNIQUE>", idSuffix));
		}

		public void WriteLineFast(string str)
		{
			sb.Append(str+"\n");
		}

		public void WriteLine(string str)
		{
			sb.Append(str.Replace("<UNIQUE>", idSuffix)+"\n");
			Flush();
		}

		public void Flush()
		{
			file.Write(sb.ToString());
			sb.Clear();
		}

		public void Close()
		{
			Flush();
			file.Close();
			file = null;
		}

		readonly string idSuffix;
		System.IO.StreamWriter file = null;
		StringBuilder sb = null;
		//int bufferLineCount = 0;
	}



	class Rect
	{
		public Rect(float xIn, float yIn, float widthIn, float heightIn)
		{
			x = xIn; y = yIn; width = widthIn; height = heightIn;
		}
		public float x, y;
		public float width, height;
	};

	class Range
	{
		public const float Auto = -100000.0f;

		public Range() { }
		public Range(float minx, float maxx, float miny, float maxy) { MinX = minx; MaxX = maxx; MinY = miny; MaxY = maxy; }
		public float MinX, MaxX;
		public float MinY, MaxY;
	};

	class Theme
	{
		// TODO: make this data driven
		public Theme(string Name)
		{
			GraphColours = new Colour[32];
			uint[] GraphColoursInt = null;
			if (Name == "light")
			{
				BackgroundColour = new Colour(255, 255, 255);
				BackgroundColourCentre = new Colour(255, 255, 255);
				LineColour = new Colour(0, 0, 0);

				GraphColoursInt = new uint[16]
				{
					0x0000C0, 0x8000C0, 0xFF4000, 0xC00000,
					0x4040A0, 0x008080, 0x200080, 0x408060,
					0x008040, 0x00008C, 0x60A880, 0x325000,
					0xA040A0, 0x808000, 0x005050, 0x606060,
				};

				TextColour = new Colour(0, 0, 0);
				MediumTextColour = new Colour(64, 64, 64);
				MinorTextColour = new Colour(128, 128, 128);
				AxisLineColour = new Colour(128, 128, 128);
				MajorGridlineColour = new Colour(128, 128, 128);
				MinorGridlineColour = new Colour(160, 160, 160);
				BudgetLineColour = new Colour(0, 196, 0);
				EventTextColour = new Colour(0);
				BudgetLineThickness = 1.0f;
			}
			else if (Name == "pink")
			{
				BackgroundColour = new Colour(255, 128, 128);
				BackgroundColourCentre = new Colour(255, 255, 255);
				LineColour = new Colour(0, 0, 0);
				GraphColoursInt = new uint[16]
				{
					0x8080FF, 0xFF8C8C, 0xFFFF8C, 0x20C0C0,
					0x808000, 0xFF8C8C, 0x20FF8C, 0x408060,
					0xFF8040, 0xFFFF8C, 0x60008C, 0x3250FF,
					0x008000, 0x8C8CFF, 0xFF5050, 0x606060,
				};

				TextColour = new Colour(0, 0, 0);
				MediumTextColour = new Colour(192, 192, 192);
				MinorTextColour = new Colour(128, 128, 128);
				AxisLineColour = new Colour(128, 128, 128);
				MajorGridlineColour = new Colour(128, 128, 128);
				MinorGridlineColour = new Colour(160, 160, 160);
				BudgetLineColour = new Colour(0, 196, 0);
				EventTextColour = new Colour(0);
				BudgetLineThickness = 1.0f;
			}
			else // "dark"
			{
				BackgroundColour = new Colour(16, 16, 16);
				BackgroundColourCentre = new Colour(80, 80, 80);
				LineColour = new Colour(255, 255, 255);

				GraphColoursInt = new uint[16]
				{//0x11bbbb
                    0x0080FF, 0x66cdFF, 0xFF6600, 0xFFFF8C,
					0x60f060, 0xFFFF00, 0x99CC00, 0xCC6600,
					0xCC3300, 0xCCFF66, 0x60008C, 0x3250FF,
					0x008000, 0x11bbbb, 0xFF5050, 0x606060,
				};
				for (int i = 0; i < 16; i++)
					GraphColours[i] = new Colour(GraphColoursInt[i]);

				TextColour = new Colour(255, 255, 255);
				MediumTextColour = new Colour(192, 192, 192);
				MinorTextColour = new Colour(128, 128, 128);
				AxisLineColour = new Colour(128, 128, 128);
				MajorGridlineColour = new Colour(128, 128, 128);
				MinorGridlineColour = new Colour(96, 96, 96);
				BudgetLineColour = new Colour(128, 255, 128);
				EventTextColour = new Colour(255, 255, 255, 0.75f);
				BudgetLineThickness = 0.5f;
			}

			for (int i = 0; i < GraphColours.Length; i++)
			{
				int repeat = i / GraphColoursInt.Length;
				float alpha = 1.0f - (float)repeat * 0.25f;
				GraphColours[i] = new Colour(GraphColoursInt[i % GraphColoursInt.Length], alpha);

			}

		}
		public Colour BackgroundColour;
		public Colour BackgroundColourCentre;
		public Colour LineColour;
		public Colour TextColour;
		public Colour MinorTextColour;
		public Colour MediumTextColour;
		public Colour AxisLineColour;
		public Colour MinorGridlineColour;
		public Colour MajorGridlineColour;
		public Colour BudgetLineColour;
		public Colour EventTextColour;
		public float BudgetLineThickness;
		public Colour[] GraphColours;
	};


	public class GraphGenerator
	{
		List<CsvInfo> csvList;

		public GraphGenerator(List<CsvInfo> csvsIn)
		{
			csvList = csvsIn;
		}

		public GraphGenerator(CsvStats csv, string filename)
		{
			csvList = new List<CsvInfo>();
			csvList.Add(new CsvInfo(csv, filename));
		}

		public Task MakeGraphAsync(GraphParams graphParams, string svgFilename, bool bWriteErrorsToSvg, bool rethrowExceptions = true)
		{
			Action action = () =>
			{
				MakeGraph(graphParams, svgFilename, bWriteErrorsToSvg, rethrowExceptions);
			};
			return Task.Factory.StartNew(action);
		}

		public void MakeGraph(GraphParams graphParams, string svgFilename, bool bWriteErrorsToSvg, bool rethrowExceptions = true)
		{
			SvgFile svg = new SvgFile(svgFilename, graphParams.uniqueId);
			try
			{
				graphParams.FinalizeSettings();
				MakeGraphInternal(graphParams, svg);
			}
			catch (System.Exception e)
			{
				// Write the error to the SVG
				string errorString = e.ToString();
				if (bWriteErrorsToSvg)
				{
					errorString = errorString.Replace(" at", " at<br/>\n");
					errorString += "<br/><br/>"+ graphParams.embedText.Replace("\n","<br/>");
					float MessageWidth = graphParams.width - 20;
					float MessageHeight = graphParams.height - 20;

					svg.WriteLine("<switch>");
					svg.WriteLine("<foreignObject x='10' y='10' color='#ffffff' font-size='12' width='" + MessageWidth + "' height='" + MessageHeight + "'><p xmlns='http://www.w3.org/1999/xhtml'>" + errorString + "</p></foreignObject>'");
					svg.WriteLine("<text x='10' y='10' fill='rgb(255, 255, 255)' font-size='10' font-family='Helvetica' > ERROR: " + errorString + "</text>");
					svg.WriteLine("</switch>");
					svg.WriteLine("</svg>");
				}
				svg.Close();
				if (rethrowExceptions)
				{
					throw;
				}
			}
			svg.Close();
		}

		void MakeGraphInternal(GraphParams graphParams, SvgFile svg)
		{
			PerfLog perfLog = new PerfLog(graphParams.bPerfLog, "Graph:"+graphParams.title);

			Rect dimensions = new Rect(0, 0, graphParams.width, graphParams.height);
			Rect graphRect = new Rect(50, 42, dimensions.width - 100, dimensions.height - 115);
			Theme theme = new Theme(graphParams.themeName);
			List<string> statNames = new List<string>(graphParams.statNames);
			string stackTotalStat = null;
			bool stackTotalStatIsAutomatic = false;

			if (graphParams.stacked)
			{
				stackTotalStat = graphParams.stackTotalStat;
			}

			svg.Write("<svg width='" + dimensions.width + "' height='" + dimensions.height + "' viewPort='0 0 " + dimensions.height + " " + dimensions.width + "' version='1.1' xmlns='http://www.w3.org/2000/svg'");
			if (graphParams.interactive)
			{
				svg.Write(" onLoad='OnLoaded<UNIQUE>(evt)'");
			}
			svg.WriteLine(">");

			if (graphParams.embedText != null)
			{
				svg.WriteLine("<![CDATA[ \n"+graphParams.embedText);
				svg.WriteLine("]]>");
			}

			// Write defs
			svg.WriteLine("<defs>");
			svg.WriteLine("<radialGradient id='radialGradient<UNIQUE>'");
			svg.WriteLine("fx='50%' fy='50%' r='65%'");
			svg.WriteLine("spreadMethod='pad'>");
			svg.WriteLine("<stop offset='0%'   stop-color=" + theme.BackgroundColourCentre.SVGString() + " stop-opacity='1'/>");
			svg.WriteLine("<stop offset='100%' stop-color=" + theme.BackgroundColour.SVGString() + " stop-opacity='1' />");
			svg.WriteLine("</radialGradient>");

			svg.WriteLine("<linearGradient id = 'linearGradient<UNIQUE>' x1 = '0%' y1 = '0%' x2 = '100%' y2 = '100%'>");
			svg.WriteLine("<stop stop-color = 'black' offset = '0%'/>");
			svg.WriteLine("<stop stop-color = 'white' offset = '100%'/>");
			svg.WriteLine("</linearGradient>");

			// Clip rect
			svg.WriteLine("<clipPath id='graphClipRect<UNIQUE>'>");
			// If we're using a fixed point scale for graphs, we need to apply it to the clip rect
			float clipRectScale = (graphParams.bFixedPointGraphs && !graphParams.percentile) ? graphParams.fixedPointPrecisionScale : 1.0f;
			svg.WriteLine("<rect  x='" + graphRect.x*clipRectScale + "' y='" + graphRect.y*clipRectScale + "' width='" + graphRect.width*clipRectScale + "' height='" + graphRect.height*clipRectScale + "'/>");
			svg.WriteLine("</clipPath>");

			svg.WriteLine("<filter id='dropShadowFilter<UNIQUE>' x='-20%' width='130%' height='130%'>");
			svg.WriteLine("<feOffset result='offOut' in='SourceAlpha' dx='-2' dy='2' />");
			svg.WriteLine("<feGaussianBlur result='blurOut' in='offOut' stdDeviation='1.1' />");
			svg.WriteLine("<feBlend in='SourceGraphic' in2='blurOut' mode='normal' />");
			svg.WriteLine("</filter>");

			svg.WriteLine("</defs>");

			
			DrawGraphArea(svg, theme, theme.BackgroundColour, dimensions, true);
			perfLog.LogTiming("Initialization");

			// Generate the list of processed, filtered CSV stats for graphing
			List<CsvStats> csvStatsList = new List<CsvStats>();
			int currentColorOffset = graphParams.colorOffset;
			int currentCustomLabelIndex = 0;

			FrameRange frameRange = new FrameRange();
			for ( int i=0; i<csvList.Count; i++ )
			{
				CsvInfo csvInfo = csvList[i];
				FrameRange csvFrameRange;
				CsvStats newCsvStats = ProcessCsvStats(csvInfo.stats, graphParams, out csvFrameRange);
				Colour csvColorOverride = null;

				// Do we have specific csv color overrides?
				if(graphParams.csvColors.Count > 0)
				{
					int csvColorIndex = (i % graphParams.csvColors.Count);
					// assign a color override for this CSV
					csvColorOverride = new Colour(graphParams.csvColors[csvColorIndex]);
				}

				if ( csvFrameRange.isLimited() )
				{
					// If we have multiple CSVs and we're applying startEvent limiting then just truncate the start of the CSVs at the event so the frames line up
					// This has the effect that frame numbers start from zero, but there is no good alternative given the CSV frame numbers will differ
					if (csvList.Count > 1 )
					{
						if (csvFrameRange.start > 0)
						{
							newCsvStats.CropStats(csvFrameRange.start, int.MaxValue);
						}
						// Adjust the global frameRange end such that we're using the largest range for all CSVs
						if (csvFrameRange.end != int.MaxValue)
						{
							// Offset end index because we're truncating the start
							csvFrameRange.end -= csvFrameRange.start;
						}
						frameRange.end = (i == 0) ? csvFrameRange.end : Math.Max(frameRange.end, csvFrameRange.end);
					}
					// Otherwise set frameRange to use the first (only) csv's range and we'll modify minX/maxX, which keeps the axis numbers accurate
					else
					{
						frameRange = csvFrameRange;
					}
				}

				if (graphParams.stacked && stackTotalStat == "")
				{
					// Make a total stat by summing each frame
					StatSamples totalStat = new StatSamples("Total");
					totalStat.samples.Capacity = newCsvStats.SampleCount;
					for (int j = 0; j < newCsvStats.SampleCount; j++)
					{
						float totalValue = 0.0f;
						foreach (StatSamples stat in newCsvStats.Stats.Values)
						{
							totalValue += stat.samples[j];
						}
						totalStat.samples.Add(totalValue);
					}
					totalStat.ComputeAverageAndTotal();
					totalStat.colour = new Colour(0x6E6E6E);
					newCsvStats.AddStat(totalStat);
					stackTotalStat = "total";
					stackTotalStatIsAutomatic = true;
				}

				SetLegend(newCsvStats, csvInfo.filename, graphParams, csvList.Count > 1, ref currentCustomLabelIndex);
				if (!graphParams.stacked)
				{
					currentColorOffset = AssignColours(newCsvStats, theme, false, currentColorOffset, csvColorOverride);
				}
				csvStatsList.Add(newCsvStats);
			}
			perfLog.LogTiming("ProcessCsvStats");

			Range range = new Range(graphParams.minX, graphParams.maxX, graphParams.minY, graphParams.maxY);

			// Apply the frameRange if necessary
			if (frameRange.isLimited())
			{
				// MinX and MaxX are both relative to frameRange.start
				range.MinX = (range.MinX == Range.Auto) ? frameRange.start : frameRange.start + range.MinX;
				if (range.MaxX != Range.Auto)
				{
					range.MaxX += frameRange.start;
				}
				if (frameRange.end != int.MaxValue)
				{
					range.MaxX = (range.MaxX == Range.Auto) ? frameRange.end : Math.Min(range.MaxX, frameRange.end); 
				}
			}

			if (graphParams.smooth)
			{
				for (int i=0; i<csvStatsList.Count; i++)
				{ 
					int kernelSize = graphParams.smoothKernelSize;
					if (graphParams.smoothKernelSize == -1)
					{
						float percent = graphParams.smoothKernelPercent;
						percent = Math.Min(percent, 100.0f);
						percent = Math.Max(percent, 0.0f);
						float kernelSizeF = percent * 0.01f * (float)csvStatsList[i].SampleCount + 0.5f;
						kernelSize = (int)kernelSizeF;
					}
					csvStatsList[i] = SmoothStats(csvStatsList[i], kernelSize, graphParams);
				}
				perfLog.LogTiming("SmoothStats");
			}

			// Compute the X range 
			range = ComputeAdjustedXRange(range, graphRect, csvStatsList);

			// Recompute the averages based on the X range. This is necessary for the legend and accurate sorting
			for (int i = 0; i < csvStatsList.Count; i++)
			{
				RecomputeStatAveragesForRange(csvStatsList[i], range);
			}
			perfLog.LogTiming("RecomputeStatAverages");


			// Handle stacking. Note that if we're not stacking, unstackedCsvStats will be a copy of csvStats
			List<CsvStats> unstackedCsvStats = new List<CsvStats>();
			for (int i = 0; i < csvStatsList.Count; i++)
			{
				unstackedCsvStats.Add(csvStatsList[i]);

				if (graphParams.stacked)
				{
					csvStatsList[i] = StackStats(csvStatsList[i], range, graphParams.stackedUnsorted, stackTotalStat, stackTotalStatIsAutomatic);
					AssignColours(csvStatsList[i], theme, true, graphParams.colorOffset, null);

					// Copy the colours to the unstacked stats
					foreach (StatSamples samples in unstackedCsvStats[i].Stats.Values)
					{
						string statName = samples.Name;
						if (stackTotalStatIsAutomatic == false && stackTotalStat != null && samples.Name.ToLower() == stackTotalStat.ToLower())
						{
							statName = "other";
						}
						samples.colour = csvStatsList[i].GetStat(statName).colour;
					}
				}
			}

			if (graphParams.stacked) perfLog.LogTiming("StackStats");

			// Adjust range
			range = ComputeAdjustedYRange(range, graphRect, csvStatsList, graphParams.maxAutoMaxY);

			if (graphParams.graphOnly)
			{
				graphRect = dimensions;
			}

			// Adjust thickness depending on sample density, smoothness etc
			float thickness = graphParams.smooth ? 0.33f : 0.1f;
			float thicknessMultiplier = (12000.0f / (range.MaxX - range.MinX)) * graphParams.lineThickness;
			thicknessMultiplier *= (graphRect.width / 400.0f);
			thickness *= thicknessMultiplier;
			thickness = Math.Max(Math.Min(thickness, 1.5f), 0.11f);

			string graphTitle = graphParams.title;

			// Get the title
			if (graphTitle.Length == 0 && statNames.Count == 1 && csvStatsList.Count > 0 && !statNames[0].Contains("*"))
			{
				// Use the stat name if we can find it, otherwise just take the param as-is
				StatSamples stat = csvStatsList[0].GetStat(statNames[0]);
				if (stat == null)
				{
					int idx = statNames[0].IndexOf('=');
					if (idx > 0)
					{
						// If this is a stat expression and we've defined the name via the = operator then use that
						graphTitle = statNames[0][..idx].Replace("(","").Replace(")","");
					}
					else
					{
						graphTitle = statNames[0];
					}					
				}
				else
				{
					graphTitle = stat.Name;
				}
			}

			// Combine and validate metadata
			// Assign metadata based on the first CSV with metadata
			CsvMetadata metaData = null;
			
			if (graphParams.showMetadata && !graphParams.graphOnly)
			{
				foreach (CsvStats stats in csvStatsList)
				{
					if (stats.metaData != null)
					{
						metaData = stats.metaData.Clone();
						break;
					}
				}
				if (metaData != null)
				{
					// Combine all metadata
					foreach (CsvStats stats in csvStatsList)
					{
						metaData.CombineAndValidate(stats.metaData);
					}
				}
			}
			perfLog.LogTiming("AdjustRangesAndMetadata");

			// Draw the graphs
			int csvIndex = 0;
			svg.WriteLine("<g id='graphArea<UNIQUE>'>");
			if (graphParams.percentile)
			{
				range.MinX = graphParams.percentileTop99 ? 99 : (graphParams.percentileTop90 ? 90 : 0);
				range.MaxX = 100;
				DrawGridLines(svg, theme, graphRect, range, graphParams, !graphParams.graphOnly, 1.0f, true);
				csvIndex = 0;
				foreach (CsvStats csvStat in csvStatsList)
				{
					foreach (StatSamples stat in csvStat.Stats.Values)
					{
						string statID = "Stat_" + csvIndex + "_" + stat.Name + "<UNIQUE>";
						DrawPercentileGraph(svg, stat.samples, stat.colour, graphRect, range, statID);
					}
					csvIndex++;
				}
			}
			else
			{
				DrawGridLines(svg, theme, graphRect, range, graphParams, !graphParams.graphOnly, 1.0f, graphParams.stacked);
				csvIndex = 0;
				foreach (CsvStats csvStat in csvStatsList)
				{
					DrawEventLines(svg, theme, csvStat.Events, graphRect, range);
					DrawEventHighlightRegions(svg, csvStat, graphRect, range, graphParams.highlightEventRegions);

					int MaxSegments = 1;
					if (graphParams.stacked)
					{
						MaxSegments = 4;
						if (csvStat.Stats.Values.Count > 15)
						{
							MaxSegments = 8;
							if (csvStat.Stats.Values.Count > 30)
							{
								MaxSegments = 32;
							}
						}
					}

					foreach (StatSamples stat in csvStat.Stats.Values)
					{
						string statID = "Stat_" + csvIndex + "_" + stat.Name + "<UNIQUE>";
						DrawGraph(svg, stat.samples, stat.colour, graphRect, range, thickness, statID, graphParams, MaxSegments);
					}

					if (graphParams.showEventNameTextMode != ShowEventTextMode.Hide)
					{
						Colour eventColour = theme.EventTextColour;
						DrawEventText(svg, csvStat.Events, eventColour, graphRect, range, graphParams);
					}

					csvIndex++;
				}
			}
			perfLog.LogTiming("DrawGraphs");


			svg.WriteLine("</g>");

			if (graphParams.stacked)
			{
				// If we're stacked, we need to redraw the grid lines without axis text
				DrawGridLines(svg, theme, graphRect, range, graphParams, false, 0.75f, false);
			}

			// Draw legend, metadata and title
			if (!graphParams.graphOnly)
			{
				DrawLegend(svg, theme, csvStatsList, graphParams, dimensions);
				if (graphParams.smooth)
				{
					DrawText(svg, "(smoothed)", 50.0f, 33, 10.0f, dimensions, theme.MinorTextColour, "start");
				}
				DrawTitle(svg, theme, graphTitle, dimensions);
			}


			if (metaData != null)
			{
				DrawMetadata(svg, metaData, dimensions, theme.TextColour);
			}

			DrawText(svg, "CSVToSVG " + CsvToSvgLibVersion.Get(), dimensions.width - 102, 15, 7.0f, dimensions, theme.MinorTextColour);

			perfLog.LogTiming("DrawText");

			// Draw the interactive elements
			if (graphParams.interactive)
			{
				AddInteractiveScripting(svg, theme, graphRect, range, unstackedCsvStats, graphParams);
				perfLog.LogTiming("AddInteractiveScripting");
			}

			svg.WriteLine("</svg>");
			perfLog.LogTotalTiming(false);
		}

		Range ComputeAdjustedYRange(Range range, Rect rect, List<CsvStats> csvStats, float maxAutoMaxY)
		{
			Range newRange = new Range(range.MinX, range.MaxX, range.MinY, range.MaxY);

			if (range.MinY == Range.Auto)
			{
				newRange.MinY = 0.0f;
			}

			if (range.MaxY == Range.Auto)
			{
				float maxSample = -10000000.0f;
				foreach (CsvStats stats in csvStats)
				{
					foreach (StatSamples samples in stats.Stats.Values)
					{
						maxSample = Math.Max(maxSample, samples.ComputeMaxValue(range.MinX == Range.Auto ? 0 : (int)range.MinX, range.MaxX == Range.Auto ? -1 : (int)range.MaxX));
					}
				}
				newRange.MaxY = Math.Min(maxSample * 1.05f, maxAutoMaxY);
			}

			// Quantise based on yincrement
			if (rect != null)
			{
				float yInc = GetYAxisIncrement(rect, newRange);
				float difY = newRange.MaxY - newRange.MinY;
				float newDifY = (int)(0.9999 + difY / yInc) * yInc;
				newRange.MaxY = newRange.MinY + newDifY;
			}

			return newRange;
		}

		Range ComputeAdjustedXRange(Range range, Rect rect, List<CsvStats> csvStats)
		{
			Range newRange = new Range(range.MinX, range.MaxX, range.MinY, range.MaxY);
			int maxNumSamples = 0;
			foreach (CsvStats stats in csvStats)
			{
				foreach (StatSamples samples in stats.Stats.Values)
				{
					maxNumSamples = Math.Max(maxNumSamples, samples.samples.Count);
				}
			}

			if (range.MinX == Range.Auto) newRange.MinX = 0;
			if (range.MaxX == Range.Auto) newRange.MaxX = maxNumSamples;

			// Quantize MinX and MaxX based on xincrement
			if (rect != null)
			{
				float xInc = GetXAxisIncrement(rect, newRange);
				if (newRange.MinX > 0)
				{
					newRange.MinX = (int)(range.MinX / xInc) * xInc;
				}
				float difX = newRange.MaxX - newRange.MinX;
				float newDifX = (int)(0.9999 + difX / xInc) * xInc;
				newRange.MaxX = newRange.MinX + newDifX;
			}

			return newRange;
		}

		class SmoothKernel
		{
			public SmoothKernel(int inKernelSize, float maxDownsampleMultiplier = 4.0f)
			{
				int maxDownsampleLevel = Math.Min(4, Math.Max(0,(int)Math.Log(maxDownsampleMultiplier, 2.0f)));
				downsampleLevel = Math.Max( Math.Min( (int)Math.Log((double)inKernelSize / 20.0, 2.0), maxDownsampleLevel), 0);

				kernelSize = inKernelSize >> downsampleLevel;
				weights = new float[kernelSize];
				float totalWeight = 0.0f;
				for (int i = 0; i < kernelSize; i++)
				{
					double df = (double)i + 0.5;
					double b = (double)kernelSize / 2;
					double c = (double)kernelSize / 8;
					double weight = Math.Exp(-(Math.Pow(df - b, 2.0) / Math.Pow(2 * c, 2.0)));
					weights[i] = (float)weight;
					totalWeight += (float)weight;
				}
				float oneOverTotalWeight = 1.0f / totalWeight;
				for (int i = 0; i < kernelSize; i++)
				{
					weights[i] *= oneOverTotalWeight;
				}
			}
			public StatSamples SmoothStatSamples(StatSamples sourceStatSamples)
			{
				StatSamples destStatSamples = new StatSamples(sourceStatSamples,false);
				destStatSamples.samples.Capacity = sourceStatSamples.samples.Count;

				// Downsample the source samples to the specified size
				List<float> downsampledSamples = sourceStatSamples.samples;
				for ( int i=0; i< downsampleLevel; i++)
				{
					int currentSampleCount = downsampledSamples.Count / 2;
					List<float> nextLevelSamples = new List<float>(currentSampleCount);
					for (int j=0; j< currentSampleCount; j++)
					{
						int prevLevelIndex = j * 2;
						nextLevelSamples.Add( ( downsampledSamples[prevLevelIndex] + downsampledSamples[prevLevelIndex + 1] ) * 0.5f );
					}
					downsampledSamples = nextLevelSamples;
				}
				int downsampleScale = 1 << downsampleLevel;

				int kernelSizeClamped = Math.Min(kernelSize, downsampledSamples.Count);

				for (int i = 0; i < sourceStatSamples.samples.Count; i++)
				{
					float sum = 0.0f;
					int startIndex = (i >> downsampleLevel) - kernelSizeClamped / 2;
					int endIndex = (startIndex + kernelSizeClamped);
					int startIndexClamped = Math.Max(0, startIndex);
					int endIndexClamped = Math.Min(downsampledSamples.Count, endIndex);
					int kernelStartIndex = startIndexClamped - startIndex;
					int kernelIndex = kernelStartIndex;

					// Smooth the in-range samples
					for (int j = startIndexClamped; j < endIndexClamped; j++)
					{
						sum += downsampledSamples[j] * weights[kernelIndex];
						kernelIndex++;
					}

					// Mirror the out of range samples < 0 
					if (startIndex < 0)
					{
						kernelIndex = kernelStartIndex - 1;
						endIndexClamped = Math.Min(downsampledSamples.Count, -startIndex);
						for (int j = startIndexClamped; j < endIndexClamped; j++)
						{
							sum += downsampledSamples[j] * weights[kernelIndex];
							kernelIndex--;
						}
					}
					// Mirror the out of range samples >= n
					if (endIndex > downsampledSamples.Count)
					{
						kernelIndex = kernelSizeClamped - 1;
						startIndexClamped = Math.Max(0, downsampledSamples.Count - endIndex + downsampledSamples.Count);
						for (int j = startIndexClamped; j < endIndexClamped; j++)
						{
							sum += downsampledSamples[j] * weights[kernelIndex];
							kernelIndex--;
						}
					}

					destStatSamples.samples.Add(sum);
				}
				return destStatSamples;
			}

			int kernelSize;
			int downsampleLevel;
			float [] weights;
		};






		CsvStats SmoothStats(CsvStats stats, int KernelSize, GraphParams graphParams)
		{
			bool bMultithreaded = graphParams.bSmoothMultithreaded;

			// Compute the displayed sample count
			int minX = Math.Max( (int)graphParams.minX, 0 );
			int maxX = stats.SampleCount;
			if (graphParams.maxX > 0)
			{
				maxX = Math.Min((int)graphParams.maxX, stats.SampleCount);
			}
			int sampleCount = maxX - minX;

			// Compute a max downsample level based on the sample count
			float maxDownsampleMultiplier = (float)sampleCount/graphParams.width;

			// Compute Gaussian Weights
			SmoothKernel kernel = new SmoothKernel(KernelSize, maxDownsampleMultiplier);

			List<StatSamples> statSampleList = stats.Stats.Values.ToList();
			// Add the stats to smoothstats before the parallel for, so we can preserve the order

			StatSamples[] smoothSamplesArray = new StatSamples[stats.Stats.Count];
			if (bMultithreaded)
			{
				int numThreads = Environment.ProcessorCount/2;
				Parallel.For(0, stats.Stats.Values.Count, new ParallelOptions { MaxDegreeOfParallelism = numThreads }, i =>
				{
					smoothSamplesArray[i] = kernel.SmoothStatSamples(statSampleList[i]);
				});
			}
			else
			{
				for (int i=0; i< stats.Stats.Values.Count; i++)
				{
					smoothSamplesArray[i] = kernel.SmoothStatSamples(statSampleList[i]);
				}
			}

			CsvStats smoothStats = new CsvStats();
			foreach (StatSamples stat in smoothSamplesArray)
			{
				smoothStats.AddStat(stat);
			}

			smoothStats.metaData = stats.metaData;
			smoothStats.Events = stats.Events;
			return smoothStats;
		}

		void SetLegend(CsvStats csvStats, string csvFilename, GraphParams graphParams, bool UseFilename, ref int currentCustomLabelIndex)
		{
			foreach (StatSamples stat in csvStats.Stats.Values)
			{
				stat.LegendName = stat.Name;
				if (UseFilename)
				{
					stat.LegendName = System.IO.Path.GetFileName(csvFilename);
				}
				if (graphParams.customLegendNames.Count > 0 && currentCustomLabelIndex < graphParams.customLegendNames.Count)
				{
					stat.LegendName = graphParams.customLegendNames[currentCustomLabelIndex];
					currentCustomLabelIndex++;
				}
				foreach (string hideStatPrefix in graphParams.hideStatPrefixes)
				{
					if (hideStatPrefix.Length > 0)
					{
						if (stat.LegendName.ToLower().StartsWith(hideStatPrefix.ToLower()))
						{
							stat.LegendName = stat.LegendName.Substring(hideStatPrefix.Length);
						}
					}
				}
			}
		}

		int AssignColours(CsvStats stats, Theme theme, bool reverseOrder, int inColorOffset, Colour csvColorOverride)
		{
			int colorOffset = inColorOffset;
			if (reverseOrder)
			{
				colorOffset = stats.Stats.Values.Count - 1;
			}

			foreach (StatSamples stat in stats.Stats.Values)
			{
				if(csvColorOverride == null)
				{
					if (stat.colour.alpha == 0.0f)
					{
						stat.colour = theme.GraphColours[colorOffset % theme.GraphColours.Length];
					}
					if (reverseOrder)
					{
						colorOffset--;
					}
					else
					{
						colorOffset++;
					}
				}
				else
				{
					stat.colour = csvColorOverride;
				}
			}
			return colorOffset;
		}

		CsvStats StackStats(CsvStats stats, Range range, bool stackedUnsorted, string stackTotalStat, bool bStackTotalStatIsAutomatic)
		{
			// Find our stack total stat
			StatSamples totalStat = null;
			string stackTotalStatLower = stackTotalStat.ToLower();
			if (stackTotalStat.Length > 0)
			{
				foreach (StatSamples stat in stats.Stats.Values)
				{
					if (stat.Name.ToLower() == stackTotalStatLower)
					{
						totalStat = stat;
						break;
					}
				}
			}

			// sort largest->smallest (based on average)
			List<StatSamples> StatSamplesSorted = new List<StatSamples>();
			foreach (StatSamples stat in stats.Stats.Values)
			{
				if (stat != totalStat)
				{
					StatSamplesSorted.Add(stat);
				}
			}

			if (!stackedUnsorted)
			{
				StatSamplesSorted.Sort();
			}

			StatSamples OtherStat = null;
			StatSamples OtherUnstacked = null;
			int rangeStart = Math.Max(0, (int)range.MinX);
			int rangeEnd = (int)range.MaxX;

			if (!bStackTotalStatIsAutomatic && totalStat != null)
			{
				OtherStat = new StatSamples("Other");
				OtherStat.colour = new Colour(0x6E6E6E);
				StatSamplesSorted.Add(OtherStat);
				OtherStat.samples.AddRange(totalStat.samples);
				OtherUnstacked = new StatSamples(OtherStat, false);
				rangeEnd = Math.Min(OtherStat.samples.Count, (int)range.MaxX);
			}

			// Get the max count for all stats
			int maxCount = 0;
			foreach (StatSamples statSamples in StatSamplesSorted)
			{
				maxCount = Math.Max(statSamples.samples.Count, maxCount);
			}

			// Make the stats cumulative
			float[] SumList = new float[maxCount];
			List<StatSamples> StatSamplesList = new List<StatSamples>();
			foreach (StatSamples srcStatSamples in StatSamplesSorted)
			{
				StatSamples destStatSamples = new StatSamples(srcStatSamples, false);
				if (srcStatSamples == OtherStat)
				{
					for (int j = 0; j < srcStatSamples.samples.Count; j++)
					{
						// The previous value of value is the total. If there are no other stats, then the total is "other"
						float value = Math.Max(srcStatSamples.samples[j] - SumList[j], 0.0f);
						OtherUnstacked.samples.Add(value);
						SumList[j] += value;
						destStatSamples.samples.Add(SumList[j]);
					}
					// If this is the "other" stat, recompute the average based on the unstacked value and apply that to the dest stat (this is used in the legend)
					OtherUnstacked.ComputeAverageAndTotal(rangeStart, rangeEnd);
					destStatSamples.average = OtherUnstacked.average;
				}
				else 
				{
					for (int j = 0; j < srcStatSamples.samples.Count; j++)
					{
						SumList[j] += srcStatSamples.samples[j];
						destStatSamples.samples.Add(SumList[j]);
					}
				}
				StatSamplesList.Add(destStatSamples);
			}

			// Copy out the list in reverse order (so they get rendered front->back)
			CsvStats stackedStats = new CsvStats();
			if (bStackTotalStatIsAutomatic)
			{
				stackedStats.AddStat(totalStat);
			}

			for (int i = StatSamplesList.Count - 1; i >= 0; i--)
			{
				stackedStats.AddStat(StatSamplesList[i]);
			}

			stackedStats.metaData = stats.metaData;
			stackedStats.Events = stats.Events;
			return stackedStats;
		}

		float GetXAxisIncrement(Rect rect, Range range)
		{
			float[] xIncrements = { 0.1f, 1.0f, 5.0f, 10.0f, 20.0f, 50.0f, 100.0f, 200.0f, 500.0f, 1000.0f, 2000.0f, 5000.0f, 10000.0f };
			float xIncrement = xIncrements[0];

			for (int i = 0; i < xIncrements.Length - 1; i++)
			{
				float gap = ToSvgXScale(xIncrement, rect, range);
				if (gap < 25)
				{
					xIncrement = xIncrements[i + 1];
				}
				else
				{
					break;
				}

			}
			int xRange = (int)range.MaxX - (int)range.MinX;
			//xIncrement = xIncrement - (xRange % (int)xIncrement);
			return xIncrement;
		}

		float GetYAxisIncrement(Rect rect, Range range)
		{
			float[] yIncrements = { 0.5f, 1.0f, 2.0f, 5.0f, 10.0f, 20.0f, 50.0f, 100.0f, 200.0f, 500.0f, 1000.0f, 2000, 5000, 10000, 20000, 50000, 100000, 200000, 500000, 1000000, 2000000, 5000000, 10000000 };

			// Find a good increment
			float yIncrement = yIncrements[0];
			for (int i = 0; i < yIncrements.Length - 1; i++)
			{
				float gap = ToSvgYScale(yIncrement, rect, range);
				if (gap < 25)
				{
					yIncrement = yIncrements[i + 1];
				}
				else
				{
					break;
				}
			}
			return yIncrement;
		}

		void DrawGridLines(SvgFile svg, Theme theme, Rect rect, Range range, GraphParams graphParams, bool bDrawAxisText, float alpha = 1.0f, bool extendLines = false)
		{
			float xIncrement = GetXAxisIncrement(rect, range);
			float yIncrement = GetYAxisIncrement(rect, range);
			float yScale = 1.0f;

			// Draw 1ms grid lines
			float minorYAxisIncrement = 1.0f;
			if ((range.MaxY - range.MinY) > 100) minorYAxisIncrement = yIncrement / 10.0f;
			minorYAxisIncrement = Math.Max(minorYAxisIncrement, yIncrement / 5.0f);

			Colour colour = new Colour(theme.MinorGridlineColour);
			colour.alpha = alpha;

			// TODO: replace with <pattern>
			svg.WriteLine("<g id='gridLines<UNIQUE>'>");
			for (int i = 0; i < 5000; i++)
			{
				float y = range.MinY + (float)i * minorYAxisIncrement;
				DrawHorizLine(svg, y, colour, rect, range);
				if (y >= range.MaxY) break;
			}

			colour = new Colour(theme.MajorGridlineColour);
			colour.alpha = (alpha + 3.0f) / 4.0f;
			for (int i = 0; i < 2000; i++)
			{
				float y = range.MinY + (float)i * yIncrement;
				DrawHorizLine(svg, y, colour, rect, range, false, 0.25f, false);
				yScale += yIncrement;
				if (y >= range.MaxY) break;
			}
			svg.WriteLine("</g>");

			// Draw the axis text
			if (bDrawAxisText)
			{
				svg.WriteLine("<g id='xAxis<UNIQUE>'>");
				for (int i = 0; i < 2000; i++)
				{
					float x = range.MinX + i * xIncrement;
					if (x > (range.MaxX)) break;
					int displayFrame = (int)Math.Floor(x) + graphParams.frameOffset;
					DrawHorizontalAxisText(svg, displayFrame.ToString(), x, theme.TextColour, rect, range);
				}
				svg.WriteLine("</g>");

				svg.WriteLine("<g id='yAxis<UNIQUE>'>");
				for (int i = 0; i < 2000; i++)
				{
					float y = range.MinY + (float)i * yIncrement;
					// FIXME: This text should use theme.LineColour - using LineColour for now for consistency/historical reasons. Need to test 
					DrawVerticalAxisText(svg, y.ToString(), y, theme.LineColour, rect, range);
					DrawVerticalAxisNotch(svg, y, theme.LineColour, rect, range);
					if (y >= range.MaxY) break;
				}
				svg.WriteLine("</g>");
			}

			DrawVerticalLine(svg, 0, theme.AxisLineColour, rect, range, 0.5f);
			DrawHorizLine(svg, 0.0f, theme.AxisLineColour, rect, range, false, 0.5f);

			// Draw the budget line if there is one
			if (graphParams.budget != float.MaxValue)
			{
				float budgetLineThickness = theme.BudgetLineThickness;
				Colour budgetLineColour = new Colour(theme.BudgetLineColour);
				bool dropShadow = false;
				if (alpha != 1.0f)
				{
					dropShadow = true;
					budgetLineThickness *= 2.0f;
					budgetLineColour.r = (byte)((float)budgetLineColour.r * 0.9f);
					budgetLineColour.g = (byte)((float)budgetLineColour.g * 0.9f);
					budgetLineColour.b = (byte)((float)budgetLineColour.b * 0.9f);
				}
				DrawHorizLine(svg, graphParams.budget, budgetLineColour, rect, range, true, budgetLineThickness, dropShadow);
			}
		}

		void DrawText(SvgFile svg, string text, float x, float y, float size, Rect rect, Colour colour, string anchor = "start", string font = "Helvetica", string id = "", bool dropShadow = false)
		{
			svg.WriteLine("<text x='" + (rect.x + x) + "' y='" + (rect.y + y) + "' fill=" + colour.SVGString() +
					  (id.Length > 0 ? " id='" + id + "'" : "") +
					  (dropShadow ? " filter='url(#dropShadowFilter)'" : "") +
					  " font-size='" + size + "' font-family='" + font + "' style='text-anchor: " + anchor + "'>" + text + "</text> >");
		}

		void DrawTitle(SvgFile svg, Theme theme, string title, Rect rect)
		{
			DrawText(svg, title, (float)(rect.x + rect.width * 0.5), 34, 14, rect, theme.TextColour, "middle", "Helvetica");
		}


		void DrawGraphArea(SvgFile svg, Theme theme, Colour colour, Rect rect, bool gradient, bool interactive = false)
		{
			//svg.WriteLine("<rect x='0' y='0' width='" + (dimensions.width) + "' height='" + (dimensions.height) + "' fill=" + colour.SVGString() + " stroke-width='1' stroke=" + theme.BackgroundColour.SVGString());
			svg.WriteLine("<rect x='0' y='0' width='" + (rect.width) + "' height='" + (rect.height) + "' fill=" + colour.SVGString() + " stroke-width='1' stroke=" + theme.BackgroundColour.SVGString());

			if (gradient)
			{
				svg.Write("style='fill:url(#radialGradient<UNIQUE>);' ");
			}
			if (interactive)
			{
				svg.Write(" onmousemove='OnGraphAreaClicked<UNIQUE>(evt)'");
				//svg.Write(" onmousewheel='OnGraphAreaWheeled<UNIQUE>(evt)'");
			}
			svg.Write("/>");
		}

		float ToSvgX(float graphX, Rect rect, Range range)
		{
			float scaleX = rect.width / (range.MaxX - range.MinX);
			return rect.x + ((float)graphX - range.MinX) * scaleX;
		}

		float ToSvgY(float graphY, Rect rect, Range range)
		{
			float svgY = rect.height - (graphY - range.MinY) / (range.MaxY - range.MinY) * rect.height + rect.y;

			float scaleY = rect.height / (range.MaxY - range.MinY);
			float svgY2 = rect.y + rect.height - (graphY - range.MinY) * scaleY;
			return svgY;
		}

		float ToSvgXScale(float graphX, Rect rect, Range range)
		{
			float scaleX = rect.width / (range.MaxX - range.MinX);
			return graphX * scaleX;
		}

		float ToSvgYScale(float graphY, Rect rect, Range range)
		{
			float scaleY = rect.height / (range.MaxY - range.MinY);
			return graphY * scaleY;
		}

		float ToCsvXScale(float graphX, Rect rect, Range range)
		{
			float scaleX = rect.width / (range.MaxX - range.MinX);
			return graphX / scaleX;
		}

		float ToCsvYScale(float graphY, Rect rect, Range range)
		{
			float scaleY = rect.height / (range.MaxY - range.MinY);
			return graphY / scaleY;
		}

		float ToCsvX(float svgX, Rect rect, Range range)
		{
			return (svgX - rect.x) * (range.MaxX - range.MinX) / rect.width + range.MinX;
		}

		float ToCsvY(float svgY, Rect rect, Range range)
		{
			return (svgY - rect.y) * (range.MaxY - range.MinY) / rect.height + range.MinY;
		}

		struct Vec2
		{
			public Vec2(float inX, float inY) { X = inX; Y = inY; }
			public float X;
			public float Y;
		};

		class PointInfo
		{
			public void AddPoint(Vec2 pt)
			{
				Count++;
				Max.X = Math.Max(pt.X, Max.X);
				Max.Y = Math.Max(pt.Y, Max.Y);
				Min.X = Math.Min(pt.X, Min.X);
				Min.Y = Math.Min(pt.Y, Min.Y);
				Total.X += pt.X;
				Total.Y += pt.Y;
			}
			public Vec2 ComputeAvg()
			{
				Vec2 avg;
				avg.X = Total.X / (float)Count;
				avg.Y = Total.Y / (float)Count;
				return avg;
			}
			public Vec2 Max = new Vec2(float.MinValue, float.MinValue);
			public Vec2 Min = new Vec2(float.MaxValue, float.MaxValue);
			public Vec2 Total = new Vec2(0, 0);
			public int Count = 0;
		};

		// TODO: this would be better if merged the best points first, ie the ones where the error value is close to zero
		// Also, the approach of using only the current pass points error to compound. Ideally we want it to be computed from the original points
		Vec2[] RemoveRedundantPoints(Vec2[] RawPoints, float compressionThreshold, int passIndex)
		{
			//return RawPoints;
			//compressionThreshold = 0;
			if (RawPoints.Length == 0)
			{
				return RawPoints;
			}
			List<Vec2> StrippedPoints = new List<Vec2>(RawPoints.Length);

			int offset = passIndex % 2;

			// Add the first points
			int i = 0;
			for (i = 0; i <= offset; i++)
			{
				StrippedPoints.Add(RawPoints[i]);
			}
			for (; i < RawPoints.Length - 1; i += 2)
			{
				Vec2 p1 = RawPoints[i - 1];
				Vec2 pCentre = RawPoints[i];
				Vec2 p2 = RawPoints[i + 1];

				// Interpolate to find where the line would be if we removed this point
				float dX = p2.X - p1.X;
				float centreDX = pCentre.X - p1.X;
				float lerpValue = centreDX / dX;
				float interpX = p1.X * (1.0f - lerpValue) + (p2.X * lerpValue);
				float interpY = p1.Y * (1.0f - lerpValue) + (p2.Y * lerpValue);

				//float distMh = Math.Abs(interpX - pCentre.X) + Math.Abs(interpY - pCentre.Y);
				float distMh = Math.Abs(interpY - pCentre.Y);
				if (distMh >= compressionThreshold)
				{
					StrippedPoints.Add(pCentre);
				}
				StrippedPoints.Add(p2);
			}
			// Add the last points
			for (; i < RawPoints.Length; i++)
			{
				StrippedPoints.Add(RawPoints[i]);
			}
			return StrippedPoints.ToArray();
		}

		List<Vec2> CompressPoints(List<Vec2> RawPoints, float pixelsPerPoint, Rect rect, Range range)
		{
			if (RawPoints.Count < 3)
			{
				return RawPoints;
			}
			int numPointsAfterCompression = (int)((float)rect.width / pixelsPerPoint);
			if (numPointsAfterCompression >= RawPoints.Count)
			{
				return RawPoints;
			}
			float NumOutputPointsPerInputPoint = (float)numPointsAfterCompression / (float)RawPoints.Count;

			PointInfo[] pointInfoList = new PointInfo[numPointsAfterCompression];
			for (int i = 0; i < numPointsAfterCompression; i++) pointInfoList[i] = new PointInfo();

			// Compute average, min, max for every point
			for (int i = 0; i < RawPoints.Count; i++)
			{
				Vec2 point = RawPoints[i];
				float outPointX = (float)i * NumOutputPointsPerInputPoint;
				int outPointIndex = Math.Min((int)(outPointX), numPointsAfterCompression - 1);
				pointInfoList[outPointIndex].AddPoint(point);
			}

			Vec2[] CompressedPoints = new Vec2[numPointsAfterCompression];
			CompressedPoints[0] = pointInfoList[0].ComputeAvg();
			for (int i = 1; i < pointInfoList.Length - 1; i++)
			{
				PointInfo info = pointInfoList[i];
				PointInfo last = pointInfoList[i - 1];
				PointInfo next = pointInfoList[i + 1];
				if (info.Min.Y <= last.Min.Y && info.Min.Y < next.Min.Y)
				{
					CompressedPoints[i] = info.Min;
				}
				else if (info.Max.Y >= last.Max.Y && info.Max.Y > next.Max.Y)
				{
					CompressedPoints[i] = info.Max;
				}
				else
				{
					CompressedPoints[i] = info.ComputeAvg();
				}
			}
			CompressedPoints[pointInfoList.Length - 1]=pointInfoList.Last().ComputeAvg();
			float compressionThreshold = pixelsPerPoint;
			for (int i = 0; i < 8; i++)
			{
				CompressedPoints = RemoveRedundantPoints(CompressedPoints, compressionThreshold, i);
			}
			return CompressedPoints.ToList();
		}

		void DrawGraph(SvgFile svg, List<float> samples, Colour colour, Rect rect, Range range, float thickness, string id, GraphParams graphParams, int MaxSegments=1)
		{
			int n = Math.Min(samples.Count, (int)(range.MaxX + 0.5));
			int start = (int)(range.MinX + 0.5);

			int numDecimalPlaces = graphParams.lineDecimalPlaces;

			if (start + 1 < n)
			{
				List<Vec2> RawPoints = new List<Vec2>(n);
				for (int i = start; i < n; i++)
				{
					float sample = samples[i];
					float x = ToSvgX((float)i, rect, range);
					float y = ToSvgY(sample, rect, range);

					RawPoints.Add(new Vec2(x, y));
				}

				if (graphParams.compression > 0.0f)
				{
					RawPoints = CompressPoints(RawPoints, graphParams.compression, rect, range);
				}

				string idString = id.Length > 0 ? "id='" + id + "'" : "";

				string fillString = "none";
				if (graphParams.stacked)
				{
					thickness = Math.Min(thickness, 0.01f);
					colour.alpha = 1.0f;
					fillString = colour.SVGStringNoQuotes();
				}

				float graphScale = 1.0f;
				if (graphParams.bFixedPointGraphs)
				{
					// Fixed point graphs are smaller because they don't include the decimal point, but we need to apply a scale to get subpixel accuracy
					// We scale the polyline by the inverse of the scaling that we're applying to the points
					graphScale = graphParams.fixedPointPrecisionScale;
					float oneOverScale = 1.0f / graphScale;

					// Divide the graph into segments to make it faster to render
					float originSvgY = ToSvgY(0.0f, rect, range);

					int numSegments = Math.Max( Math.Min(RawPoints.Count / 500, 1), MaxSegments);
					int pointsPerSegment = RawPoints.Count / numSegments;
					for ( int i=0;i<numSegments; i++)
					{
						int startIndex = i * pointsPerSegment;
						int endIndex = startIndex + pointsPerSegment + 1;
						if (i == numSegments - 1)
						{
							endIndex = RawPoints.Count;
						}

						string segmentIdString = id.Length > 0 ? "id='" + id + "_Seg-"+i+ "'" : "";
						svg.WriteLine("<polyline " + segmentIdString + " transform='scale(" + oneOverScale + "," + oneOverScale + ")' points='");

						for (int pointIndex=startIndex ; pointIndex < endIndex; pointIndex++)
						{
							Vec2 point = RawPoints[pointIndex];
							int x = (int)Math.Round(point.X * graphScale, 0);
							int y = (int)Math.Round(point.Y * graphScale, 0);
							svg.WriteFast(" " + x + "," + y);
						}

						if (graphParams.stacked)
						{
							int firstX = (int)Math.Round(RawPoints[startIndex].X * graphScale);
							int lastX = (int)Math.Round(RawPoints[endIndex - 1].X * graphScale);
							int originY = (int)Math.Round(ToSvgY(0.0f, rect, range) * graphScale);
							svg.WriteFast(" " + lastX + ","+ originY);
							svg.WriteFast(" " + firstX + ","+ originY);
						}
						svg.WriteLine("' style='fill:" + fillString + ";stroke-width:" + thickness * graphScale + "; clip-path: url(#graphClipRect<UNIQUE>)' stroke=" + colour.SVGString() + "/>");
					}
				}
				else
				{
					string formatStr = "0";
					if (numDecimalPlaces > 0)
					{
						formatStr += ".";
						for (int i = 0; i < numDecimalPlaces; i++)
						{
							formatStr += "0";
						}
					}
					svg.WriteLine("<polyline " + idString + " points='");
					foreach (Vec2 point in RawPoints)
					{
						float x = point.X;
						float y = point.Y;
						svg.WriteFast(" " + x.ToString(formatStr) + "," + y.ToString(formatStr));
					}

					if (graphParams.stacked)
					{
						float lastSample = samples[n - 1];
						svg.WriteFast(" " + ToSvgX((float)n + 20, rect, range) * graphScale + "," + ToSvgY(lastSample, rect, range) * graphScale);
						svg.WriteFast(" " + ToSvgX((float)n + 20, rect, range) * graphScale + "," + ToSvgY(0.0f, rect, range) * graphScale);
						svg.WriteFast(" " + ToSvgX((float)n - 20, rect, range) * graphScale + "," + ToSvgY(0.0f, rect, range) * graphScale);
						svg.WriteFast(" " + ToSvgX(start, rect, range) * graphScale + "," + ToSvgY(0.0f, rect, range) * graphScale );
					}
					svg.WriteLine("' style='fill:" + fillString + ";stroke-width:" + thickness * graphScale + "; clip-path: url(#graphClipRect<UNIQUE>)' stroke=" + colour.SVGString() + "/>");
				}
			}
		}
		void DrawPercentileGraph(SvgFile svg, List<float> samples, Colour colour, Rect rect, Range range, string id)
		{
			samples.Sort();
			List<Vec2> RawPoints = new List<Vec2>();
			const int NUM_SAMPLES = 100;
			int Begin = (int)range.MinX;
			int End = (int)range.MaxX;
			int Delta = End - Begin;
			float Percentile, xbase, x, y;
			int Count = samples.Count;
			for (int i = NUM_SAMPLES * Begin; i < NUM_SAMPLES * End; i += Delta)
			{
				int sampleIndex = Count * i / (NUM_SAMPLES * 100);
				Percentile = samples[sampleIndex];
				xbase = (float)i / NUM_SAMPLES;
				x = ToSvgX(xbase, rect, range);
				y = ToSvgY(Percentile, rect, range);
				RawPoints.Add(new Vec2(x, y));
			}

			Percentile = samples[Count - 1];
			xbase = 100.0F;
			x = ToSvgX(xbase, rect, range);
			y = ToSvgY(Percentile, rect, range);
			RawPoints.Add(new Vec2(x, y));
			string idString = id.Length > 0 ? "id='" + id + "'" : "";
			svg.WriteLine("<polyline " + idString + " points='");
			foreach (Vec2 point in RawPoints)
			{
				x = point.X;
				y = point.Y;

				svg.WriteFast(" " + x + "," + y);
			}
			svg.WriteLine("' style='fill:none;stroke-width:1.3; clip-path: url(#graphClipRect<UNIQUE>)' stroke=" + colour.SVGString() + "/>");

		}


		class CsvGraphEvent : CsvEvent
		{
			public CsvGraphEvent(CsvEvent ev, int inPriority)
			{
				base.Frame = ev.Frame;
				base.Name = ev.Name;
				base.Timestamp = ev.Timestamp;
				count = 1;
				priority = inPriority;
			}
			public int count;
			public int priority;
		};

		private string truncateTextIfNeeded(string text, int maxLength)
		{
			if (text.Length <= maxLength)
			{
				return text;
			}
			return text.Substring(0, maxLength) + "...";
		}

		void DrawEventText(SvgFile svg, List<CsvEvent> events, Colour colour, Rect rect, Range range, GraphParams graphParams )
		{
			float LastEventX = -100000.0f;
			int lastFrame = 0;

			bool bAutoShowMode = graphParams.showEventNameTextMode == ShowEventTextMode.ShowAuto;
			float mergeThreshold = bAutoShowMode ? 12.0f : 8.5f;

			// Make a filtered list of events to display, grouping duplicates which are close together
			List<CsvGraphEvent> filteredEvents = new List<CsvGraphEvent>();
			CsvGraphEvent currentDisplayEvent = null;
			foreach (CsvEvent ev in events)
			{
				// Only draw events which are in the range
				int eventPriority = GetEventPriority(ev.Name, graphParams);
				if (ev.Frame >= range.MinX && ev.Frame <= range.MaxX)
				{
					// Merge with the current display event?
					bool bMerge = false;
					if (currentDisplayEvent != null && ( ev.Name == currentDisplayEvent.Name || bAutoShowMode))
					{
						float DistToLastEvent = ToSvgX(ev.Frame, rect, range) - ToSvgX(currentDisplayEvent.Frame, rect, range);
						if (DistToLastEvent <= mergeThreshold)
						{
							bMerge = true;
						}
					}

					if (bMerge)
					{
						if ( bAutoShowMode )
						{
							// If this event is actually higher (numerically lower) priority then replace the current event
							if (eventPriority < currentDisplayEvent.priority )
							{
								currentDisplayEvent.priority = eventPriority;
								currentDisplayEvent.Name = ev.Name;
								currentDisplayEvent.Frame = ev.Frame;
							}
						}
						currentDisplayEvent.count++;
					}
					else
					{
						currentDisplayEvent = new CsvGraphEvent(ev, eventPriority);
						filteredEvents.Add(currentDisplayEvent);
					}
				}
			}

			if (graphParams.showEventNameTextMode == ShowEventTextMode.ShowAuto)
			{
				foreach (CsvGraphEvent ev in filteredEvents)
				{
					float eventX = ToSvgX(ev.Frame, rect, range);
					string name = truncateTextIfNeeded(ev.Name,32);

					if (ev.count > 1)
					{
						name += " +" + (ev.count-1);
					}

					float csvTextX = ToCsvX(eventX + 7, rect, range);
					DrawHorizontalAxisText(svg, name, csvTextX, colour, rect, range, 10, true);
				}
			}
			else
			{
				foreach (CsvGraphEvent ev in filteredEvents)
				{
					float eventX = ToSvgX(ev.Frame, rect, range);
					string name = ev.Name;

					if (ev.count > 1)
					{
						name += " &#x00D7; " + ev.count;
					}

					// Space out the events (allow at least 8 pixels between them)
					if (eventX - LastEventX <= 8.5f)
					{
						// Add an arrow to indicate events were spaced out 
						name = "&#x21b7; " + name;
						if (ev.count == 1)
						{
							name += " (+" + (ev.Frame - lastFrame) + ")";
						}
						eventX = LastEventX + 9.0f;
					}
					float csvTextX = ToCsvX(eventX + 7, rect, range);

					DrawHorizontalAxisText(svg, name, csvTextX, colour, rect, range, 10, true);
					LastEventX = eventX;
					lastFrame = ev.Frame;
				}
			}

		}

		void DrawEventLines(SvgFile svg, Theme theme, List<CsvEvent> events, Rect rect, Range range)
		{
			float LastEventX = -100000.0f;
			float LastDisplayedEventX = -100000.0f;
			int i = 0;
			foreach (CsvEvent ev in events)
			{
				// Only draw events which are in the range
				if (ev.Frame >= range.MinX && ev.Frame <= range.MaxX)
				{
					float eventX = ToSvgX(ev.Frame, rect, range);
					// Space out the events (allow at least 2 pixels between them)
					if (eventX - LastDisplayedEventX > 3)
					{
						float alpha = (eventX - LastEventX < 1.0f) ? 0.5f : 1.0f;
						DrawVerticalLine(svg, ev.Frame, theme.MajorGridlineColour, rect, range, alpha, true, true);
						LastDisplayedEventX = eventX;
					}
					LastEventX = eventX;
				}
				i++;
			}
		}


		void DrawEventHighlightRegions(SvgFile svg, CsvStats csvStats, Rect rect, Range range, List<string> highlightEventRegions )
		{
			// Draw event regions if requested
			if (highlightEventRegions != null)
			{
				Colour highlightColour = new Colour(0xffffff, 0.1f);
				float y0 = rect.y;
				float y1 = rect.y + rect.height;
				float height = y1 - y0;

				int numPairs = highlightEventRegions.Count / 2;
				if (highlightEventRegions.Count >= 2 && highlightEventRegions.Count % 2 != 2)
				{
					for (int i = 0; i < numPairs; i++)
					{
						string startName = highlightEventRegions[i * 2].ToLower().Trim();
						string endName = highlightEventRegions[i * 2 + 1].ToLower().Trim();

						if (endName == "{null}")
						{
							endName = null;
						}
						if (startName == "{null}")
						{
							startName = null;
						}

						List<int> startIndices = null;
						List<int> endIndices = null;
						csvStats.GetEventFrameIndexDelimiters(startName, endName, out startIndices, out endIndices);
						for (int j = 0; j < startIndices.Count; j++)
						{
							float x0 = ToSvgX(startIndices[j], rect, range);
							float x1 = ToSvgX(endIndices[j], rect, range);
							float width = x1 - x0;

							svg.WriteLine("<rect x='" + x0 + "' y='" + y0 + "' width='" + width + "' height='" + height + "' fill=" + highlightColour.SVGString() + "/>");
						}
					}
				}
			}
		}
		void DrawVerticalAxisNotch(SvgFile svg, float sample, Colour colour, Rect rect, Range range, float thickness = 0.25f)
		{
			if (sample > range.MaxY) return;
			if (sample < range.MinY) return;

			float y = ToSvgY(sample, rect, range);

			float x1 = rect.x - 2.5f;
			float x2 = rect.x;

			svg.WriteLineFast("<line x1='" + x1 + "' y1='" + y + "' x2='" + x2 + "' y2='" + y + "' style='fill:none;stroke-width:" + thickness + "'"
				+ " stroke=" + colour.SVGString() + "/>");
		}

		void DrawHorizLine(SvgFile svg, float sample, Colour colour, Rect rect, Range range, bool dashed = false, float thickness = 0.25f, bool dropShadow = false)
		{
			if (sample > range.MaxY) return;
			if (sample < range.MinY) return;

			float y = ToSvgY(sample, rect, range);

			float x1 = rect.x;
			float x2 = rect.x + rect.width;

			if (dropShadow)
			{
				Colour shadowColour = new Colour(0, 0, 0, 0.33f);
				svg.WriteLineFast("<line x1='" + (x1 - 1) + "' y1='" + (y + 1) + "' x2='" + (x2 - 1) + "' y2='" + (y + 1) + "' style='fill:none;stroke-width:" + thickness * 1.5f + "'"
					+ " stroke=" + shadowColour.SVGString() + ""
					+ (dashed ? " stroke-dasharray='4,4' d='M5 10 l215 0' " : "")
					+ "/>" );
			}

			svg.WriteLineFast("<line x1='" + x1 + "' y1='" + y + "' x2='" + x2 + "' y2='" + y + "' style='fill:none;stroke-width:" + thickness + "'"
				+ " stroke=" + colour.SVGString() + ""
				+ (dashed ? " stroke-dasharray='4,4' d='M5 10 l215 0' " : "")
				//   + (dropShadow ? "filter='url(#dropShadowFilter)'>" : "" )
				+ "/>" );
		}

		void DrawVerticalAxisText(SvgFile svg, string text, float textY, Colour colour, Rect rect, Range range, float size = 10.0f)
		{
			float y = ToSvgY(textY, rect, range) + 4;
			float x = rect.x - 10.0f;
			svg.WriteLine("<text x='" + x + "' y='" + y + "' fill=" + colour.SVGString() + " font-size='" + size + "' font-family='Helvetica' style='text-anchor: end'>" + text + "</text> >");
		}


		void DrawHorizontalAxisText(SvgFile svg, string text, float textX, Colour colour, Rect rect, Range range, float size = 10.0f, bool top = false)
		{
			float scaleX = rect.width / (float)(range.MaxX - range.MinX - 1);
			float x = ToSvgX(textX, rect, range) - 4;
			float y = rect.y + rect.height + 10;

			if (top)
			{
				y = rect.y + 5;
			}

			svg.WriteLine("<text x='" + y + "' y='" + (-x) + "' fill=" + colour.SVGString() + " font-size='" + size + "' font-family='Helvetica' transform='rotate(90)' >" + text + "</text>");
		}

		void DrawLegend(SvgFile svg, Theme theme, List<CsvStats> csvStats, GraphParams graphParams, Rect rect)
		{
			List<StatSamples> statSamples = new List<StatSamples>();

			foreach (CsvStats csvStat in csvStats)
			{
				foreach (StatSamples stat in csvStat.Stats.Values)
				{
					statSamples.Add(stat);
				}
			}

			// Stacked stats are already sorted in the right order
			if (graphParams.forceLegendSort || ((graphParams.showAverages || graphParams.showTotals) && !graphParams.stacked))
			{
				statSamples.Sort();
			}


			float x = rect.width + rect.x - 60;
			if (graphParams.showAverages || graphParams.showTotals)
			{
				x -= 30;
			}
			float y = 30; // rect.height - 25;

			svg.WriteLine("<g id='LegendPanel<UNIQUE>' fill='rgba(255,0,0,0.3)'>");// transform='translate(5) rotate(45 50 50)'> ");
			svg.WriteLine("<rect x='"+x+"' y='"+y+"' width='0' height='100' fill='rgba(0,0,0,0.3)' blend='1' id='legendPanelRect<UNIQUE>' rx='5' ry='5'/>");

			// Check if the total is a fraction
			bool legendValueIsWholeNumber = false;
			if (graphParams.showTotals)
			{
				legendValueIsWholeNumber = true;
				foreach (StatSamples stat in statSamples)
				{
					if (stat.total != Math.Floor(stat.total))
					{
						legendValueIsWholeNumber = false;
						break;
					}
				}
			}

			foreach (StatSamples stat in statSamples)
			{
				if (stat.average > graphParams.legendAverageThreshold)
				{
					Colour colour = stat.colour;
					svg.WriteLine("<text x='" + (x - 5) + "' y='" + y + "' fill=" + theme.TextColour.SVGString() + " font-size='10' font-family='Helvetica' style='text-anchor: end' filter='url(#dropShadowFilter)'>" + stat.LegendName + "</text>");
					svg.WriteLine("<rect x='" + (x + 0) + "' y='" + (y - 8) + "' width='10' height='10' fill=" + colour.SVGString() + " filter='url(#dropShadowFilter)' stroke-width='1' stroke='" + theme.LineColour + "' />");

					if (graphParams.showAverages || graphParams.showTotals)
					{
						float legendValue = graphParams.showTotals ? (float)stat.total : stat.average;
						string formatString = "0.00";
						if (graphParams.showTotals && legendValueIsWholeNumber)
						{
							formatString = "0";
						}
						svg.WriteLine("<text x='" + (x + 15) + "' y='" + y + "' fill=" + theme.TextColour.SVGString() + " font-size='10' font-family='Helvetica' style='text-anchor: start' filter='url(#dropShadowFilter)'>" + legendValue.ToString(formatString) + "</text>");
					}

					y += 16;
				}
			}

			if (graphParams.showTotals || graphParams.showAverages)
			{
				string legendValueTypeString = graphParams.showTotals ? "(sum)" : "(avg)";
				svg.WriteLine("<text x='" + (x + 15) + "' y='" + y + "' fill=" + theme.TextColour.SVGString() + " font-size='10' font-family='Helvetica' style='text-anchor: start' filter='url(#dropShadowFilter)'>" + legendValueTypeString + "</text>");
			}

			svg.WriteLine("</g>");
		}


		void DrawVerticalLine(SvgFile svg, float sampleX, Colour colour, Rect rect, Range range, float thickness = 0.25f, bool dashed = false, bool dropShadow = false, string id = "", bool noTranslate = false)
		{
			float x = noTranslate ? sampleX : ToSvgX(sampleX, rect, range);
			float y1 = rect.y;
			float y2 = rect.y + rect.height;

			if (dropShadow)
			{
				Colour shadowColour = new Colour(0, 0, 0, 0.33f);
				svg.WriteLine("<line x1='" + (x - 1) + "' y1='" + (y1 + 1) + "' x2='" + (x - 1) + "' y2='" + (y2 + 1) + "' style='fill:none;stroke-width:" + thickness * 1.5f + "' "
					  + " stroke=" + shadowColour.SVGString() + ""
					  + (dashed ? " stroke-dasharray='4,4' d='M5 10 l215 0' " : "")
					  + (id.Length > 0 ? " id='" + id + "_dropShadow'" : "")
					  + "/>");
			}

			svg.WriteLine("<line x1='" + x + "' y1='" + y1 + "' x2='" + x + "' y2='" + y2 + "' style='fill:none;stroke-width:" + thickness + "' "
				+ " stroke=" + colour.SVGString() + ""
				+ (dashed ? " stroke-dasharray='4,4' d='M5 10 l215 0' " : "")
				+ (id.Length > 0 ? " id='" + id + "'" : "")
				+ "/>");

		}
		float frac(float value)
		{
			return value - (float)Math.Truncate(value);
		}

		string GetStringHash8Char(string name)
		{
			using (SHA256 sha256 = SHA256.Create())
			{
				Encoding enc = Encoding.UTF8;
				byte[] hash = sha256.ComputeHash(enc.GetBytes(name));
				return BitConverter.ToString(hash).Replace("-", string.Empty).Substring(0, 8);
			}
		}

		string GetJSStatName(string statName)
		{
			string newString = "_";
			foreach (char c in statName)
			{
				if ((c >= 'A' && c <= 'Z') || (c >= 'a' && c <= 'z'))
				{
					newString += c;
				}
				else
				{
					newString += "_";
				}
			}
			newString += "_" + GetStringHash8Char(statName);
			return newString;
		}

		class InteractiveStatInfo
		{
			public string friendlyName;
			public string jsVarName;
			public string jsTextElementId;
			public string jsGroupElementId;
			public string graphID;
			public Colour colour;
			public StatSamples statSamples;
			public StatSamples originalStatSamples;
		};

		void AddInteractiveScripting(SvgFile svg, Theme theme, Rect rect, Range range, List<CsvStats> csvStats, GraphParams graphParams )
		{
			bool bSnapToPeaks = graphParams.snapToPeaks && !graphParams.smooth;

			// Todo: separate values, quantized frame pos, pass in X axis name
			List<InteractiveStatInfo> interactiveStats = new List<InteractiveStatInfo>();

			// Add the frame number stat
			{
				InteractiveStatInfo statInfo = new InteractiveStatInfo();
				statInfo.friendlyName = "Frame";
				statInfo.jsTextElementId = "csvFrameText<UNIQUE>";
				statInfo.jsGroupElementId = "csvFrameGroup<UNIQUE>";
				statInfo.statSamples = null;
				interactiveStats.Add(statInfo);
			}

			for (int i = 0; i < csvStats.Count; i++)
			{
				foreach (StatSamples stat in csvStats[i].Stats.Values)
				{
					InteractiveStatInfo statInfo = new InteractiveStatInfo();
					statInfo.colour = stat.colour;
					statInfo.friendlyName = stat.LegendName;
					statInfo.jsVarName = "statData_" + i + "_" + GetJSStatName(stat.Name) + "<UNIQUE>";
					statInfo.graphID = "Stat_" + i + "_" + stat.Name + "<UNIQUE>";
					statInfo.jsTextElementId = "csvStatText__" + i + GetJSStatName(stat.Name) + "<UNIQUE>";
					statInfo.jsGroupElementId = "csvStatGroup__" + i + GetJSStatName(stat.Name) + "<UNIQUE>";
					statInfo.originalStatSamples = stat;
					statInfo.statSamples = new StatSamples(stat, false);
					interactiveStats.Add(statInfo);
				}
			}

			// Record the max value at each index
			// TODO: strip out data outside the range (needs an offset as well as a multiplier)
			int multiplier = 1;
			float numStatsPerPixel = (float)(range.MaxX - range.MinX) / rect.width;

			// Compute the unfiltered sample count
			int rawSampleCount = 0;
			foreach (InteractiveStatInfo statInfo in interactiveStats)
			{
				if (statInfo.originalStatSamples != null)
				{
					rawSampleCount = Math.Max(statInfo.originalStatSamples.samples.Count, rawSampleCount);
				}
			}
			int filteredSampleCount = rawSampleCount;

			// TODO: truncate the stats/events using minX/maxX to save memory, so filteredSampleCount is (int)(range.MaxX) - (int)(range.MinX) 
            // if not downsampling. MinX will require a bias as well as a multiplier in the javascript

			// Downsample the stats if needed
			if (numStatsPerPixel > 1)
			{
				multiplier = (int)numStatsPerPixel;

				// Compute max value for each frame
				List<float> maxValues = new List<float>(rawSampleCount);
				foreach (InteractiveStatInfo statInfo in interactiveStats)
				{
					if (statInfo.originalStatSamples != null)
					{
						for (int i = 0; i < statInfo.originalStatSamples.samples.Count; i++)
						{
							float value = statInfo.originalStatSamples.samples[i];
							if (i >= maxValues.Count)
							{
								maxValues.Add(value);
							}
							else
							{
								maxValues[i] = Math.Max(maxValues[i], value);
							}
						}
					}
				}

				filteredSampleCount = maxValues.Count / multiplier;// (int)(range.MaxX) - (int)(range.MinX);

				// Create the filtered stat array
				int offset = multiplier / 2;
				foreach (InteractiveStatInfo statInfo in interactiveStats)
				{
					if (statInfo.originalStatSamples == null)
					{
						continue;
					}
					for (int i = 0; i < filteredSampleCount; i++)
					{
						int srcStartIndex = Math.Max(i * multiplier - offset, 0);
						int srcEndIndex = Math.Min(i * multiplier + offset + 1, maxValues.Count);

						int bestIndex = -1;
						float highestValue = float.MinValue;
						// Find a good index based on nearby maxValues
						for (int j = srcStartIndex; j < srcEndIndex; j++)
						{
							if (maxValues[j] > highestValue)
							{
								highestValue = maxValues[j];
								bestIndex = j;
							}
						}
						if (bestIndex < statInfo.originalStatSamples.samples.Count)
						{
							statInfo.statSamples.samples.Add(statInfo.originalStatSamples.samples[bestIndex]);
						}
					}
				}
			}
			else
			{
				foreach (InteractiveStatInfo statInfo in interactiveStats)
				{
					statInfo.statSamples = statInfo.originalStatSamples;
				}
			}

			float oneOverMultiplier = 1.0f / (float)multiplier;

			// Generate the event list
			List<CsvEvent> allEvents = new List<CsvEvent>();
			for (int i = 0; i < csvStats.Count; i++)
			{
				foreach (CsvEvent ev in csvStats[i].Events)
				{
					allEvents.Add(ev);
				}
			}


			// Create a hidden panel for storing all the stat group elements
			svg.WriteLine("<g id='interactivePanelInnerHidden<UNIQUE>' visibility='collapse'>");
			Rect zeroRect = new Rect(0, 0, 0, 0);
			foreach (InteractiveStatInfo statInfo in interactiveStats)
			{
				svg.WriteLine("<g id='" + statInfo.jsGroupElementId + "' transform='translate(0,0)' visibility='inherit'>");
				int textXOffset = -5;
				if (statInfo.colour != null)
				{
					svg.WriteLine("<rect x='-16' y='0' width='10' height='10' fill=" + statInfo.colour.SVGString() + " filter='url(#dropShadowFilter)' stroke-width='1' stroke='" + theme.LineColour + "'/>");
					textXOffset = -20;
				}
				DrawText(svg, statInfo.friendlyName, textXOffset, 9, 11, zeroRect, new Colour(255, 255, 255, 1.0f), "end", "Helvetica", "", true);
				DrawText(svg, " ", 5, 9, 11, zeroRect, new Colour(255, 255, 255, 1.0f), "start", "Helvetica", statInfo.jsTextElementId, true);

				svg.WriteLine("</g>");
			}
			svg.WriteLine("</g>"); // interactivePanelInnerHidden

			svg.WriteLine("<g id='interactivePanel<UNIQUE>' visibility='hidden'>");
			DrawVerticalLine(svg, 0, new Colour(255, 255, 255, 0.4f), rect, range, 1.0f, true, true, "", true);
			svg.WriteLine("<g id='interactivePanelInnerWithRect<UNIQUE>'>");
			svg.WriteLine("<rect x='0' y='0' width='100' height='100' fill='rgba(0,0,0,0.3)' blend='1' id='interactivePanelRect<UNIQUE>' rx='5' ry='5'/>");
			svg.WriteLine("<g id='interactivePanelInner<UNIQUE>'>");
			svg.WriteLine("</g>"); // interactivePanelInner
			svg.WriteLine("</g>"); // interactivePanelInnerWithRect

			svg.WriteLine("<g id='interactiveEventPanelInnerWithRect<UNIQUE>'>");
			svg.WriteLine("<rect x='0' y='0' width='100' height='100' fill='rgba(0,0,0,0.3)' blend='1' id='interactiveEventPanelRect<UNIQUE>' rx='5' ry='5'/>");
			svg.WriteLine("<g id='interactiveEventPanelInner<UNIQUE>'>");

			float textOffsetY = rect.y - 20;
			DrawText(svg, "Events", 0, 9+ textOffsetY, 11, zeroRect, new Colour(255, 255, 255, 1.0f), "start", "Helvetica", "", true);
			svg.WriteLine("</g>"); // interactiveEventPanelInner
			svg.WriteLine("</g>"); // interactiveEventPanelInnerWithRect
			svg.WriteLine("</g>"); // interactivepanel


			svg.WriteLine("<script type='application/ecmascript'> <![CDATA[");

			// Write the event list
			List<string> elementStrings = new List<string>();
			for (int i = 0; i < allEvents.Count; i++)
			{
				CsvEvent ev = allEvents[i];
				elementStrings.Add("{ text: '" + ev.Name + "', frame: " + ev.Frame.ToString() + "}");
			}
			svg.WriteLine("var allEvents<UNIQUE> = [" + String.Join(",", elementStrings) + "]");

			// Write the event index list for each sample frame
			List<int>[] filteredEventIndexLists = new List<int>[filteredSampleCount];
			for (int i = 0; i < filteredEventIndexLists.Length; i++)
				filteredEventIndexLists[i] = new List<int>();

			int eventPaddingPixels=6;
			for (int i = 0; i < allEvents.Count; i++)
			{
				int frame = allEvents[i].Frame;
				int filteredFrameIndex = (int)Math.Round((float)frame * oneOverMultiplier);

				for (int j= filteredFrameIndex - eventPaddingPixels; j< filteredFrameIndex + eventPaddingPixels; j++)
				{
					if (j >= 0 && j < filteredEventIndexLists.Length)
					{
						filteredEventIndexLists[j].Add(i);
					}
				}
			}

			elementStrings = new List<string>();
			for (int i = 0; i < filteredEventIndexLists.Length; i++)
			{
				List<int> eventIndexList = filteredEventIndexLists[i];
				List<string> innerElementStrings = new List<string>();
				foreach (int index in eventIndexList)
				{
					innerElementStrings.Add(index.ToString());
				}
				elementStrings.Add("["+String.Join(",", innerElementStrings)+"]");
			}
			svg.WriteLine("var eventIndexLists<UNIQUE> = [" + String.Join(",", elementStrings) + "]");

			// Write the data array for each stat
			foreach (InteractiveStatInfo statInfo in interactiveStats)
			{
				if (statInfo.statSamples != null)
				{
					svg.Write("var " + statInfo.jsVarName + " = [");
					foreach (float value in statInfo.statSamples.samples)
					{
						float fracValue = frac(value);
						// If this is very close to a whole number, write as a whole number to save 3 bytes
						if (fracValue <= 0.005f || fracValue >= 0.995f)
						{
							svg.WriteFast((int)Math.Round(value, 0) + ",");
						}
						else
						{
							string str = value.ToString("0.00");
							if (str[0] == '0' && str[1] == '.')
							{
								// Trim off the leading 0 before the decimal point - it's unnecessary
								str = str.Substring(1);
							}
							if (str.Last() == '0')
							{
								// Trim off trailing 0s
								str = str.Substring(0,str.Length-1);
							}
							svg.WriteFast(str + ",");
						}
					}
					svg.WriteLine("]");
				}
			}

			svg.WriteLine("function OnLoaded<UNIQUE>(evt)");
			svg.WriteLine("{");
			svg.WriteLine("    var legendPanel = document.getElementById('LegendPanel<UNIQUE>');");
			svg.WriteLine("    var legendRect = document.getElementById('legendPanelRect<UNIQUE>');");
			svg.WriteLine("    var legendBbox = legendPanel.getBBox();");
			svg.WriteLine("    legendRect.setAttribute('width',legendBbox.width+80);");
			svg.WriteLine("    legendRect.setAttribute('height',legendBbox.height+8);");
			svg.WriteLine("    legendRect.setAttribute('x',legendRect.getAttribute('x')-legendBbox.width);");
			//svg.WriteLine("    legendRect.setAttribute('y',legendBbox.y-8);");
			svg.WriteLine("}");

			if (bSnapToPeaks)
			{
				// Record the max value at each index
				List<float> maxValues = new List<float>();
				foreach (InteractiveStatInfo statInfo in interactiveStats)
				{
					if (statInfo.statSamples != null)
					{
						for (int i = 0; i < statInfo.statSamples.samples.Count; i++)
						{
							float value = statInfo.statSamples.samples[i];
							if (i >= maxValues.Count)
							{
								maxValues.Add(value);
							}
							else
							{
								maxValues[i] = Math.Max(maxValues[i], value);
							}
						}
					}
				}

				svg.Write("var maxValues<UNIQUE> = [");
				foreach (float value in maxValues)
				{
					svg.WriteFast(value + ",");
				}
				svg.WriteLine("]");

				svg.WriteLine("function FindInterestingFrame<UNIQUE>(x,range)");
				svg.WriteLine("{");
				svg.WriteLine("    range *= " + oneOverMultiplier + ";");
				svg.WriteLine("    x = Math.round( x*" + oneOverMultiplier + " );");
				svg.WriteLine("    halfRange = Math.round(range/2.0);");
				svg.WriteLine("    startX = Math.round(Math.max(x-halfRange,0));");
				svg.WriteLine("    endX = Math.round(Math.min(x+halfRange,maxValues<UNIQUE>.length));");
				svg.WriteLine("    maxVal = 0.0;");
				svg.WriteLine("    bestIndex = x;");
				svg.WriteLine("    for ( i=startX;i<endX;i++) ");
				svg.WriteLine("    {");
				svg.WriteLine("        if (maxValues<UNIQUE>[i]>maxVal)");
				svg.WriteLine("        {");
				svg.WriteLine("            bestIndex=i;");
				svg.WriteLine("            maxVal = maxValues<UNIQUE>[i];");
				svg.WriteLine("        }");
				svg.WriteLine("     }");
				svg.WriteLine("     return Math.round( bestIndex *" + multiplier + ");");
				svg.WriteLine("}");
				svg.WriteLine("  ");
			}
			else
			{
				svg.WriteLine("function FindInterestingFrame<UNIQUE>(x,range) { return x; }");
			}




			svg.WriteLine("function GetGraphX<UNIQUE>(mouseX)");
			svg.WriteLine("{");
			svg.WriteLine("  return (mouseX - " + rect.x + ") * (" + range.MaxX + " - " + range.MinX + ") / " + rect.width + " + " + range.MinX + ";");
			svg.WriteLine("}");
			svg.WriteLine("  ");

			svg.WriteLine("function compareSamples(a, b)");
			svg.WriteLine("{");
			svg.WriteLine("      if (a.isFrame)");
			svg.WriteLine("          return -1;");
			svg.WriteLine("      if (b.isFrame)");
			svg.WriteLine("          return 1;");
			svg.WriteLine("      if (a.value > b.value)");
			svg.WriteLine("          return -1;");
			svg.WriteLine("      if (a.value < b.value)");
			svg.WriteLine("          return 1;");
			svg.WriteLine("      return 0;");
			svg.WriteLine("}");

			svg.WriteLine("function ToSvgX<UNIQUE>(graphX)");
			svg.WriteLine("{");
			svg.WriteLine("    scaleX = " + rect.width / (range.MaxX - range.MinX) + ";");
			svg.WriteLine("    return " + rect.x + " + (graphX - " + range.MinX + ") * scaleX;");
			svg.WriteLine("}");

			/*
            svg.WriteLine("var graphView = { offsetX:0.0, offsetY:0.0, scaleX:1.0, scaleY:1.0 };");
            svg.WriteLine("function OnGraphAreaWheeled<UNIQUE>(evt)");
            svg.WriteLine("{");
            svg.WriteLine("     var graphAreaElement = document.getElementById('graphAreaTransform');");
            svg.WriteLine("     graphView.scaleX+=0.1;");
            svg.WriteLine("     ");
            svg.WriteLine("     graphAreaElement.setAttribute('transform','scale('+graphView.scaleX+','+graphView.scaleY+')')");
            svg.WriteLine("}");
*/

			int onePixelFrameCount = (int)(ToCsvXScale(1.0f, rect, range) * 8.0f);
			float rightEdgeX = ToSvgX(range.MaxX, rect, range);

			svg.WriteLine("function OnGraphAreaClicked<UNIQUE>(evt)");
			svg.WriteLine("{");
			svg.WriteLine("  graphX = GetGraphX<UNIQUE>(evt.offsetX); ");
			svg.WriteLine("  var interactivePanel = document.getElementById('interactivePanel<UNIQUE>');");
			svg.WriteLine("  var legendPanel = document.getElementById('LegendPanel<UNIQUE>');");
			// Snap to an interesting frame (the max value under the pixel)

			svg.WriteLine("  var frameNum = FindInterestingFrame<UNIQUE>( Math.round(graphX+0.5), " + onePixelFrameCount + " );");
			svg.WriteLine("  if (frameNum >= " + range.MinX + " && frameNum < " + range.MaxX + ")");
			svg.WriteLine("  {");
			svg.WriteLine("    var xOffset = 0;");
			svg.WriteLine("    var lineX = ToSvgX<UNIQUE>(frameNum);");
			svg.WriteLine("    var textX = lineX + xOffset;");
			svg.WriteLine("    var textY = " + textOffsetY);

			svg.WriteLine("    var interactivePanelInner = document.getElementById('interactivePanelInner<UNIQUE>');");
			svg.WriteLine("    legendPanel.setAttribute('visibility','hidden')");
			svg.WriteLine("    interactivePanel.setAttribute('visibility','visible')");
			svg.WriteLine("    interactivePanel.setAttribute('transform','translate('+textX+',0)')");
			svg.WriteLine("    var dataIndex = Math.round( frameNum * " + oneOverMultiplier + " );");
			svg.WriteLine("    dataIndex = Math.min( dataIndex, "+ (filteredSampleCount-1) +" ); ");

			// Fill out the sample data array
			svg.WriteLine("    var samples = [");
			foreach (InteractiveStatInfo statInfo in interactiveStats)
			{
				string groupElementString = "document.getElementById('" + statInfo.jsGroupElementId + "')";
				string textElementString = "document.getElementById('" + statInfo.jsTextElementId + "')";
				if (statInfo.jsVarName == null)
				{
					svg.Write("            { value: frameNum+" + graphParams.frameOffset + ", name: '" + statInfo.friendlyName + "', colour: 'rgb(0,0,0)', isFrame:true");
				}
				else
				{
					string valueStr = statInfo.jsVarName + "[dataIndex]";
					svg.Write("            { value: " + valueStr + ", name: '" + statInfo.friendlyName + "', colour: " + statInfo.colour.SVGString() + ", isFrame:false");
				}
				svg.WriteLine(", groupElement: " + groupElementString + ", textElement: " + textElementString + " },");
			}
			svg.WriteLine("    ];");
			svg.WriteLine("    samples.sort(compareSamples);");

			// Draw the interactive legend
			// Move all elements from the visible rect into the hidden rect
			svg.WriteLine("    var panelInner = document.getElementById('interactivePanelInner<UNIQUE>');");
			svg.WriteLine("    var panelInnerHidden = document.getElementById('interactivePanelInnerHidden<UNIQUE>');");
			svg.WriteLine("    while (panelInner.childNodes.length>0)");
			svg.WriteLine("    {");
			svg.WriteLine("        panelInnerHidden.appendChild(panelInner.childNodes[0]);");
			svg.WriteLine("    }");

			svg.WriteLine("    var maxSampleCountToDisplay="+(rect.height/15)+";");
			svg.WriteLine("    for ( i=0;i<samples.length;i++ )");
			svg.WriteLine("    {");
			svg.WriteLine("        var groupElement = samples[i].groupElement;");
			svg.WriteLine("        if ( samples[i].value > 0 && i<=maxSampleCountToDisplay)");
			svg.WriteLine("        {");
			svg.WriteLine("            groupElement.setAttribute('transform','translate(0,'+textY+')');");
			svg.WriteLine("            var textElement = samples[i].textElement;");
			svg.WriteLine("            var textNode = document.createTextNode(samples[i].value);");
			svg.WriteLine("            textElement.replaceChild(textNode,textElement.childNodes[0]); ");
			svg.WriteLine("            if (groupElement.parentNode != panelInner)");
			svg.WriteLine("              panelInner.appendChild(groupElement);");
			svg.WriteLine("            textY += 15;");
			svg.WriteLine("        }");
			svg.WriteLine("        else");
			svg.WriteLine("        {");
			svg.WriteLine("            if (groupElement.parentNode != panelInnerHidden)");
			svg.WriteLine("                panelInnerHidden.appendChild(groupElement);");
			svg.WriteLine("        }");
			svg.WriteLine("    }");

			// Display events
			svg.WriteLine("    var eventPanelInner = document.getElementById('interactiveEventPanelInner<UNIQUE>');");
			svg.WriteLine("    while (eventPanelInner.childNodes.length>2)");
			svg.WriteLine("    {");
			svg.WriteLine("        eventPanelInner.removeChild(eventPanelInner.lastChild);");
			svg.WriteLine("    }");

			svg.WriteLine("    var eventTextY = " + textOffsetY+"+15;");
			svg.WriteLine("    var eventIndexList = eventIndexLists<UNIQUE>[dataIndex];");
			svg.WriteLine("    for ( i=0;i<eventIndexList.length;i++ )");
			svg.WriteLine("    {");
			svg.WriteLine("        var index = eventIndexList[i];");
			svg.WriteLine("        var event = allEvents<UNIQUE>[index];");
			svg.WriteLine("        if ( eventTextY<=" + rect.height.ToString() + ")");
			svg.WriteLine("        {");
			svg.WriteLine("            var frameOffset = event.frame - frameNum;");
			svg.WriteLine("            var frameOffsetStr = (frameOffset>0 ? '+' : '') +frameOffset.toString()");
			svg.WriteLine("            var textElement = document.createElementNS('http://www.w3.org/2000/svg','text');");
			svg.WriteLine("            textElement.setAttribute('transform','translate(0,'+eventTextY+')');");
			svg.WriteLine("            textElement.innerHTML = event.text + ' (' + frameOffsetStr +')';");
			svg.WriteLine("            textElement.setAttribute('fill','rgb(224, 224, 224)');");
			svg.WriteLine("            textElement.setAttribute('filter','url(#dropShadowFilter)');");
			svg.WriteLine("            textElement.setAttribute('stroke','CSVStats.Colour');");
			svg.WriteLine("            textElement.setAttribute('font-size','10');");
			svg.WriteLine("            textElement.setAttribute('font-family','Helvetica');");
			svg.WriteLine("            textElement.setAttribute('style','text-anchor: start');");
			svg.WriteLine("            textElement.setAttribute('x','0');");
			svg.WriteLine("            textElement.setAttribute('y','9');");
			svg.WriteLine("            textElement.setAttribute('visibility','inherit');");
			svg.WriteLine("            eventPanelInner.appendChild(textElement);");
			svg.WriteLine("            eventTextY += 12;");
			svg.WriteLine("        }");
			svg.WriteLine("    }");

			// Set the panel rect dimensions based on the content
			svg.WriteLine("    var panelRect = document.getElementById('interactivePanelRect<UNIQUE>');");
			svg.WriteLine("    var bbox = interactivePanelInner.getBBox();");
			svg.WriteLine("    panelRect.setAttribute('width',bbox.width+16);");
			svg.WriteLine("    panelRect.setAttribute('height',textY+8);");
			svg.WriteLine("    panelRect.setAttribute('x',bbox.x-8);");
			svg.WriteLine("    panelRect.setAttribute('y',bbox.y-8);");

			float maxX = rect.x + rect.width;
			svg.WriteLine("    var panelOffset = 0;");
			svg.WriteLine("    if ( bbox.x + textX < 0 )");
			svg.WriteLine("    {");
			svg.WriteLine("        panelOffset = -(bbox.x + textX);");
			svg.WriteLine("    }");
			svg.WriteLine("    else if ( bbox.x+bbox.width + textX > " + maxX + " ) ");
			svg.WriteLine("    {    ");
			svg.WriteLine("        panelOffset = " + maxX + "-(bbox.x+bbox.width + textX);");
			svg.WriteLine("    }   ");
			svg.WriteLine("    var interactivePanelInnerWithRect = document.getElementById('interactivePanelInnerWithRect<UNIQUE>');");
			svg.WriteLine("    interactivePanelInnerWithRect.setAttribute('transform','translate('+panelOffset+',0)');");

			// Move the event panel and set its visibility
			svg.WriteLine("    var eventBbox = eventPanelInner.getBBox();");
			svg.WriteLine("    if ( textX + eventBbox.width + bbox.width + bbox.x > " + maxX + " ) ");
			svg.WriteLine("			panelOffset-=eventBbox.width + bbox.width+40;");

			svg.WriteLine("    var eventPanel = document.getElementById('interactiveEventPanelInnerWithRect<UNIQUE>');");
			svg.WriteLine("    if (eventIndexList.length>0)");
			svg.WriteLine("    {");
			svg.WriteLine("      eventPanel.setAttribute('visibility','inherit');");
			svg.WriteLine("      eventPanel.setAttribute('transform','translate('+(bbox.width+bbox.x+20+panelOffset)+',0)');");
			svg.WriteLine("    }");
			svg.WriteLine("    else");
			svg.WriteLine("    {");
			svg.WriteLine("      eventPanel.setAttribute('visibility','hidden');");
			svg.WriteLine("    }");

			// Set the event panel rect size
			svg.WriteLine("    var eventPanelRect = document.getElementById('interactiveEventPanelRect<UNIQUE>');");
			svg.WriteLine("    eventPanelRect.setAttribute('width',eventBbox.width+16);");
			svg.WriteLine("    eventPanelRect.setAttribute('height',eventTextY+8);");
			svg.WriteLine("    eventPanelRect.setAttribute('x',eventBbox.x-8);");
			svg.WriteLine("    eventPanelRect.setAttribute('y',eventBbox.y-8);");
			svg.WriteLine("  }");

			// If we moused outside the box then hide the interactive panel
			svg.WriteLine("  else");
			svg.WriteLine("  {");
			svg.WriteLine("    interactivePanel.setAttribute('visibility','hidden')");
			svg.WriteLine("    legendPanel.setAttribute('visibility','visible')");
			svg.WriteLine("  }");
			svg.WriteLine("}");
			svg.WriteLine("]]> </script>");

			Rect dimensions = new Rect(0, 0, graphParams.width, graphParams.height);
			DrawGraphArea(svg, theme, new Colour(0, 0, 0, 0.0f), dimensions, false, true);
		}

		void DrawMetadata(SvgFile svg, CsvMetadata metadata, Rect rect, Colour colour)
		{
			float y = rect.height - 15;

			string Platform = metadata.GetValue("platform", "[Unknown platform]");
			string BuildConfiguration = metadata.GetValue("config", "[Unknown config]");
			string BuildVersion = metadata.GetValue("buildversion", "[Unknown version]");
			string SessionId = metadata.GetValue("sessionid", "");
			string PlaylistId = metadata.GetValue("playlistid", "");
			// If we have a device profile, write that out instead of the platform
			Platform = metadata.GetValue("deviceprofile", Platform);
			BuildVersion = metadata.GetValue("buildversion", BuildVersion);

			string Commandline = metadata.GetValue("commandline", "");
			svg.WriteLine("<text x='" + 10 + "' y='" + y + "' fill=" + colour.SVGString() + " font-size='9' font-family='Helvetica' >" + BuildConfiguration + " " + Platform + " " + BuildVersion + " " + SessionId + " " + PlaylistId + "</text>");
			y += 10;
			svg.WriteLine("<text x='" + 10 + "' y='" + y + "' fill=" + colour.SVGString() + " font-size='9' font-family='Courier New' >" + Commandline + "</text>");
		}

		bool IsEventShown(string eventString, GraphParams graphParams)
		{
			return GetEventPriority(eventString, graphParams) >= 0;
		}

		int GetEventPriority(string eventString, GraphParams graphParams)
		{
			if (graphParams.showEventNames == null)
			{
				return -1;
			}
			if (graphParams.showEventNames.Count == 0)
			{
				return -1;
			}
			if (eventString.Length == 0)
			{
				return -1;
			}

			eventString = eventString.ToLower();
			int priority = 0;
			foreach (string showEventName in graphParams.showEventNames)
			{
				string showEventNameLower = showEventName.ToLower();
				if (showEventNameLower.EndsWith("*"))
				{
					int index = showEventNameLower.LastIndexOf('*');
					string prefix = showEventNameLower.Substring(0, index);
					if (eventString.StartsWith(prefix))
					{
						return priority;
					}
				}
				else if (eventString == showEventNameLower)
				{
					return priority;
				}
				priority++;
			}
			return -1;
		}

		bool IsStatIgnored(string statName, GraphParams graphParams)
		{
			statName = statName.ToLower();

			if (graphParams.maxHierarchyDepth != -1)
			{
				int Depth = 0;
				foreach (char c in statName)
				{
					if (c == graphParams.hierarchySeparator)
					{
						Depth++;
						if (Depth > graphParams.maxHierarchyDepth)
						{
							return true;
						}
					}
				}
			}

			if (graphParams.ignoreStats.Count == 0)
			{
				return false;
			}

			foreach (string ignoreStat in graphParams.ignoreStats)
			{
				string ignoreStatLower = ignoreStat.ToLower();
				if (ignoreStatLower.EndsWith("*"))
				{
					int index = ignoreStatLower.LastIndexOf('*');
					string prefix = ignoreStatLower.Substring(0, index);
					if (statName.StartsWith(prefix))
					{
						return true;
					}
				}
				else if (statName == ignoreStatLower)
				{
					return true;
				}
			}

			return false;
		}

		
		class FrameRange
		{
			public bool isLimited()
			{
				return start > 0 || end < Int32.MaxValue;
			}
			
			public int start = 0;
			public int end = Int32.MaxValue;
		};

		FrameRange GetEventTruncationFrameRange(CsvStats csvStats, GraphParams graphParams)
		{
			// Apply startEvent and endEvent truncation if requested
			FrameRange frameRange = new FrameRange();
			if (graphParams.startEvent != null)
			{
				foreach (CsvEvent ev in csvStats.Events)
				{
					if (CsvStats.DoesSearchStringMatch(ev.Name, graphParams.startEvent))
					{
						frameRange.start = Math.Clamp(ev.Frame + graphParams.startEventOffset, 0, csvStats.SampleCount - 1);
						break;
					}
				}
			}
			if (graphParams.endEvent != null)
			{
				foreach (CsvEvent ev in csvStats.Events)
				{
					if (ev.Frame >= frameRange.start && CsvStats.DoesSearchStringMatch(ev.Name, graphParams.endEvent))
					{
						frameRange.end = Math.Clamp(ev.Frame + graphParams.endEventOffset, 0, csvStats.SampleCount - 1);
						break;
					}
				}
			}
			return frameRange;
		}

		CsvStats ProcessCsvStats(CsvStats csvStatsIn, GraphParams graphParams, out FrameRange frameRange)
		{
			CsvStats csvStats = new CsvStats(csvStatsIn, graphParams.statNames.ToArray());

			// Compute any derived stats
			foreach (string statStr in graphParams.statNames)
			{
				if (CsvStats.IsDerivedStatExpression(statStr))
				{
					StatSamples derivedStat = csvStatsIn.MakeDerivedStatFromExpression(statStr);
					if (derivedStat != null)
					{
						csvStats.AddStat(derivedStat);
					}
				}
			}

			if (graphParams.discardLastFrame)
			{
				foreach (StatSamples stat in csvStats.Stats.Values.ToArray())
				{
					if (stat.samples.Count > 0)
					{
						stat.samples.RemoveAt(stat.samples.Count - 1);
					}
				}
			}

			// Process the minFilterStat
			float minFilterStatValue = graphParams.minFilterStatValue;
			StatSamples minFilterStat = null;
			if (graphParams.minFilterStatName != null && minFilterStatValue > -float.MaxValue)
			{
				minFilterStat = csvStats.GetStat(graphParams.minFilterStatName);
				if (minFilterStat != null)
				{
					for (int i = 0; i < minFilterStat.samples.Count; ++i)
					{
						if (minFilterStat.samples[i] < minFilterStatValue)
						{
							minFilterStat.samples[i] = 0.0f;
						}
					}
					minFilterStat.ComputeAverageAndTotal();
				}
				else
				{
					// Need a proper stat name for the min filter to work
					minFilterStatValue = -float.MaxValue;
				}
			}

			// Filter out stats which are ignored etc
			List<StatSamples> FilteredStats = new List<StatSamples>();
			foreach (StatSamples stat in csvStats.Stats.Values.ToArray())
			{
				if (IsStatIgnored(stat.Name, graphParams))
				{
					continue;
				}

				if (minFilterStatValue > -float.MaxValue)
				{
					if (stat != minFilterStat)
					{
						// Reset other stats when the filter stat entry was below the threshold
						for (int i = 0; i < stat.samples.Count; i++)
						{
							if (minFilterStat.samples[i] == 0.0f)
							{
								stat.samples[i] = 0.0f;
							}
						}
					}
				}

				if (graphParams.filterOutZeros)
				{
					float lastNonZero = 0.0f;
					for (int i = 0; i < stat.samples.Count; i++)
					{
						if (stat.samples[i] == 0.0f)
						{
							stat.samples[i] = lastNonZero;
						}
						else
						{
							lastNonZero = stat.samples[i];
						}
					}
					stat.ComputeAverageAndTotal();
				}

				if (graphParams.statMultiplier != 1.0f)
				{
					for (int i = 0; i < stat.samples.Count; i++)
					{
						stat.samples[i] *= graphParams.statMultiplier;
					}
				}

				// Filter out stats where the average below averageThreshold
				if (graphParams.averageThreshold > -float.MaxValue)
				{
					if (stat.average < graphParams.averageThreshold)
					{
						continue;
					}
				}

				// Filter out stats below the threshold
				if (graphParams.threshold > -float.MaxValue)
				{
					bool aboveThreshold = false;
					foreach (float val in stat.samples)
					{
						if (val > graphParams.threshold)
						{
							aboveThreshold = true;
							break;
						}
					}

					if (!aboveThreshold)
					{
						continue;
					}
				}

				// LLM specific, spaces in LLM seem to be $32$.
				// This is a temp fix until LLM outputs without $32$.
				stat.Name = stat.Name.Replace("$32$", " ");
				// If we get here, the stat wasn't filtered
				FilteredStats.Add(stat);
			}

			// Have any stats actually been filtered? If so, replace the list
			if (FilteredStats.Count < csvStats.Stats.Count)
			{
				csvStats.Stats.Clear();
				foreach (StatSamples stat in FilteredStats)
				{
					csvStats.AddStat(stat);
				}
			}

			frameRange = GetEventTruncationFrameRange(csvStats, graphParams);

			// Filter out events
			List<CsvEvent> FilteredEvents = new List<CsvEvent>();
			foreach (CsvEvent ev in csvStats.Events)
			{
				if (IsEventShown(ev.Name, graphParams))
				{
					FilteredEvents.Add(ev);
				}
			}
			csvStats.Events = FilteredEvents;

			return csvStats;
		}

		void RecomputeStatAveragesForRange(CsvStats stats, Range range)
		{
			foreach (StatSamples stat in stats.Stats.Values.ToArray())
			{
				int rangeStart = Math.Max(0, (int)range.MinX);
				int rangeEnd = Math.Min(stat.samples.Count, (int)range.MaxX);

				stat.ComputeAverageAndTotal(rangeStart, rangeEnd);
			}
		}
	}


}