<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <RootNamespace>CsvStats</RootNamespace>
    <AssemblyTitle>CsvToSvgLib</AssemblyTitle>
    <Product>CsvToSvgLib</Product>
    <RuntimeIdentifiers>win-x64;osx-x64;win-arm64;osx-arm64;linux-x64;linux-arm64</RuntimeIdentifiers>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
</Project>