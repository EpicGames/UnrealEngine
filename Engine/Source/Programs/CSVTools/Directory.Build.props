<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <Import
    Project="$([MSBuild]::GetPathOfFileAbove('Directory.Build.props', '$(MSBuildThisFileDirectory)../'))"
    Condition="'' != $([MSBuild]::GetPathOfFileAbove('Directory.Build.props', '$(MSBuildThisFileDirectory)../'))" />

  <Import Project="../Shared/UnrealEngine.csproj.props" />

  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <RuntimeIdentifiers>win-x64;osx-x64;win-arm64;osx-arm64;linux-x64;linux-arm64</RuntimeIdentifiers>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <Copyright>
      Copyright %28C%29 Microsoft. All rights reserved.
      Copyright Epic Games, Inc. All Rights Reserved.
    </Copyright>
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="$(MSBuildThisFileDirectory)../Shared/MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
</Project>
