@echo off
setlocal enabledelayedexpansion

pushd "%~dp0\..\..\..

for /f "tokens=1" %%A in ('p4 where . ^| findstr /c:"//"') do (
    set p4Path=%%A
)
echo P4 engine Path: !p4Path!

p4 sync !p4Path!/Binaries/DotNET/CsvTools/...
p4 sync !p4Path!/Source/Programs/CSVTools/...
p4 sync !p4Path!/Source/Programs/Shared/...
popd