using CSVStats;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.CsvStatsTests
{
	public class CsvMetadataTests
	{
		[Fact]
		public void Constructor_Default_CreatesEmptyValues()
		{
			var metadata = new CsvMetadata();
			Assert.NotNull(metadata.Values);
			Assert.Empty(metadata.Values);
		}

		[Fact]
		public void ParseFromCsvLine_ParsesKeyValuePairs()
		{
			// CSV metadata format: [key],value,[key],value,...
			string[] line = new[] { "[platform]", "windows", "[buildversion]", "5.8.0" };
			var metadata = new CsvMetadata(line);

			Assert.Equal("windows", metadata.Values["platform"]);
			Assert.Equal("5.8.0", metadata.Values["buildversion"]);
		}

		[Fact]
		public void ParseFromCsvLine_KeysAreLowercase()
		{
			string[] line = new[] { "[Platform]", "Windows", "[BuildVersion]", "5.8.0" };
			var metadata = new CsvMetadata(line);

			// Keys should be lowercased, values preserved
			Assert.True(metadata.Values.ContainsKey("platform"));
			Assert.Equal("Windows", metadata.Values["platform"]);
		}

		[Fact]
		public void ParseFromCsvLine_CommandlineSpecialCasing_HandlesCommas()
		{
			// The commandline key gets special treatment - everything after it is joined with commas
			string[] line = new[] { "[platform]", "windows", "[commandline]", "\"-arg1", "val1", "-arg2", "val2\"" };
			var metadata = new CsvMetadata(line);

			Assert.True(metadata.Values.ContainsKey("commandline"));
			string cmdline = metadata.Values["commandline"];
			// All parts should be reassembled with commas, quotes stripped from first/last
			Assert.Contains("-arg1", cmdline);
			Assert.Contains("val1", cmdline);
			Assert.Contains("-arg2", cmdline);
			Assert.Contains("val2", cmdline);
		}

		[Fact]
		public void CopyConstructor_ProducesIndependentCopy()
		{
			var original = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			var copy = new CsvMetadata(original);

			// Should have the same values
			Assert.Equal(original.Values["platform"], copy.Values["platform"]);

			// Modifying copy should not affect original
			copy.Values["platform"] = "linux";
			Assert.Equal("windows", original.Values["platform"]);
			Assert.Equal("linux", copy.Values["platform"]);
		}

		[Fact]
		public void Clone_ProducesCopy()
		{
			var original = CsvTestHelper.CreateMetadata(("platform", "windows"));
			var clone = original.Clone();

			Assert.Equal("windows", clone.Values["platform"]);
		}

		[Fact]
		public void Matches_IdenticalMetadata_ReturnsTrue()
		{
			var a = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			var b = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			Assert.True(CsvMetadata.Matches(a, b));
		}

		[Fact]
		public void Matches_DifferentValues_ReturnsFalse()
		{
			var a = CsvTestHelper.CreateMetadata(("platform", "windows"));
			var b = CsvTestHelper.CreateMetadata(("platform", "linux"));
			Assert.False(CsvMetadata.Matches(a, b));
		}

		[Fact]
		public void Matches_DifferentKeys_ReturnsFalse()
		{
			var a = CsvTestHelper.CreateMetadata(("platform", "windows"));
			var b = CsvTestHelper.CreateMetadata(("config", "test"));
			Assert.False(CsvMetadata.Matches(a, b));
		}

		[Fact]
		public void Matches_DifferentCounts_ReturnsFalse()
		{
			var a = CsvTestHelper.CreateMetadata(("platform", "windows"));
			var b = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			Assert.False(CsvMetadata.Matches(a, b));
		}

		[Fact]
		public void Matches_BothNull_ReturnsTrue()
		{
			Assert.True(CsvMetadata.Matches(null, null));
		}

		[Fact]
		public void Matches_OneNull_ReturnsFalse()
		{
			var a = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.False(CsvMetadata.Matches(a, null));
			Assert.False(CsvMetadata.Matches(null, a));
		}

		[Fact]
		public void GetValue_ExistingKey_ReturnsValue()
		{
			var metadata = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.Equal("windows", metadata.GetValue("platform", "default"));
		}

		[Fact]
		public void GetValue_MissingKey_ReturnsDefault()
		{
			var metadata = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.Equal("default", metadata.GetValue("missing", "default"));
		}
	}
}
