using System.IO;
using CSVStats;
using Xunit;

namespace PerfReportTool.Tests.CsvStatsTests
{
	[Trait("Category", "FileIO")]
	public class CsvStatsReadTests : IClassFixture<TestDataFixture>
	{
		private readonly TestDataFixture _fixture;

		// File name constants for the in-project test data
		private const string PerfCsv1 = "Perf_++UE5+Main-CL-52441555-CitySample_Windows_B02C1ABC494618E056385CB0080C1931.csv.bin";
		private const string PerfCsv2 = "Perf_++UE5+Main-CL-52441555-CitySample_Windows_F11D38AE4A4086B50D8FB7A37E337B3E.csv.bin";
		private const string LlmCsv1 = "LLM_++UE5+Main-CL-52441555-CitySample_Windows_126E42324728AE04A37A70B29E582593.csv.bin";
		private const string LlmCsv2 = "LLM_++UE5+Main-CL-52441555-CitySample_Windows_2C2C8B3D49A805590E22DD9E0AF2FEF5.csv.bin";
		private const string NoMetadataCsv = "perf_csv_nometadata.bin";

		public CsvStatsReadTests(TestDataFixture fixture)
		{
			_fixture = fixture;
		}

		[Fact]
		public void ReadCSVFile_BinaryCsvBin_ReadsStatsAndMetadata()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(PerfCsv1);
			Assert.True(File.Exists(csvPath), $"Test file not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath);
			Assert.NotNull(stats);
			Assert.True(stats.Stats.Count > 0, "Expected at least one stat from binary CSV");
			Assert.NotNull(stats.metaData);
		}

		[Fact]
		public void ReadCSVFile_NoMetadataCsv_HandlesMissingMetadata()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(NoMetadataCsv);
			Assert.True(File.Exists(csvPath), $"No-metadata test file not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath);
			Assert.NotNull(stats);
			Assert.True(stats.Stats.Count > 0, "Expected stats even without metadata");
			Assert.Null(stats.metaData);
		}

		[Fact]
		public void ReadCSVFile_LlmCsv_ReadsMemoryStats()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(LlmCsv1);
			Assert.True(File.Exists(csvPath), $"LLM CSV not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath);
			Assert.NotNull(stats);
			Assert.True(stats.Stats.Count > 0, "Expected memory stats from LLM CSV");
			Assert.NotNull(stats.metaData);
			Assert.Equal("1", stats.metaData.GetValue("llm", "0"));
		}

		[Fact]
		public void ReadCSVFile_BinaryCsvBin_HasExpectedStatTypes()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(PerfCsv1);
			Assert.True(File.Exists(csvPath), $"Binary CSV not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath);
			Assert.NotNull(stats);

			// Perf CSVs should have many stats (typically hundreds)
			Assert.True(stats.Stats.Count > 10, $"Expected many stats but got {stats.Stats.Count}");

			// Verify samples are non-empty
			bool hasNonEmptyStat = false;
			foreach (var stat in stats.Stats.Values)
			{
				if (stat.GetNumSamples() > 0)
				{
					hasNonEmptyStat = true;
					break;
				}
			}
			Assert.True(hasNonEmptyStat, "Expected at least one stat with samples");
		}

		[Fact]
		public void ReadCSVFile_JustHeader_ReadsOnlyStatNames()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(PerfCsv1);
			Assert.True(File.Exists(csvPath), $"Test file not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath, bJustHeader: true);
			Assert.NotNull(stats);

			// Should have stat names but empty samples
			Assert.True(stats.Stats.Count > 0, "Expected stat definitions from header");
			foreach (var stat in stats.Stats.Values)
			{
				Assert.Equal(0, stat.GetNumSamples());
			}
		}

		[Fact]
		public void ReadCSVFile_WithStatFilter_OnlyReturnsMatchingStats()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(PerfCsv1);
			Assert.True(File.Exists(csvPath), $"Test file not found: {csvPath}");

			// Read all stats first to know what exists
			var allStats = CsvStats.ReadCSVFile(csvPath);
			Assert.True(allStats.Stats.Count > 1, "Need more than 1 stat for this test");

			// Read with a filter for just one stat
			string firstStatName = null;
			foreach (var key in allStats.Stats.Keys)
			{
				firstStatName = key;
				break;
			}

			var filteredStats = CsvStats.ReadCSVFile(csvPath, statNames: new[] { firstStatName });
			Assert.NotNull(filteredStats);
			Assert.True(filteredStats.Stats.Count <= allStats.Stats.Count);
		}

		[Fact]
		public void ReadCSVFile_MetadataContainsExpectedKeys()
		{
			_fixture.RequireTestData();

			string csvPath = _fixture.GetFile(PerfCsv1);
			Assert.True(File.Exists(csvPath), $"Test file not found: {csvPath}");

			var stats = CsvStats.ReadCSVFile(csvPath);
			Assert.NotNull(stats.metaData);

			// UE5 perf CSV metadata should contain standard keys
			Assert.True(stats.metaData.Values.Count > 0, "Expected metadata key-value pairs");
			Assert.Equal("Windows", stats.metaData.GetValue("platform", null));
			Assert.NotNull(stats.metaData.GetValue("buildversion", null));
		}

		[Fact]
		public void ReadCSVFile_TwoPerfCsvs_HaveSimilarStatCounts()
		{
			_fixture.RequireTestData();

			string csvPath1 = _fixture.GetFile(PerfCsv1);
			string csvPath2 = _fixture.GetFile(PerfCsv2);

			var stats1 = CsvStats.ReadCSVFile(csvPath1, bJustHeader: true);
			var stats2 = CsvStats.ReadCSVFile(csvPath2, bJustHeader: true);

			// Both perf CSVs from the same build should have roughly the same stat count
			// (minor differences are possible due to dynamic stats)
			double ratio = (double)stats1.Stats.Count / stats2.Stats.Count;
			Assert.True(ratio > 0.95 && ratio < 1.05,
				$"Stat counts differ significantly: {stats1.Stats.Count} vs {stats2.Stats.Count}");
		}

		[Fact]
		public void ReadCSVFile_LlmCsv_HasMoreStatsThanPerfCsv()
		{
			_fixture.RequireTestData();

			var perfStats = CsvStats.ReadCSVFile(_fixture.GetFile(PerfCsv1), bJustHeader: true);
			var llmStats = CsvStats.ReadCSVFile(_fixture.GetFile(LlmCsv1), bJustHeader: true);

			// LLM CSVs include additional LLM/* memory stats
			Assert.True(llmStats.Stats.Count > perfStats.Stats.Count,
				$"Expected LLM ({llmStats.Stats.Count} stats) to have more stats than Perf ({perfStats.Stats.Count} stats)");
		}
	}
}
