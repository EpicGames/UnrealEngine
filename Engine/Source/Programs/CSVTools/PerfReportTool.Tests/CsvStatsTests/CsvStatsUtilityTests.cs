using CSVStats;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.CsvStatsTests
{
	public class DoesSearchStringMatchTests
	{
		[Fact]
		public void ExactMatch_ReturnsTrue()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("frametime", "frametime"));
		}

		[Fact]
		public void CaseInsensitive_ReturnsTrue()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("FrameTime", "frametime"));
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("frametime", "FrameTime"));
		}

		[Fact]
		public void WildcardStar_MatchesPrefix()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("GameThread/Total", "gamethread*"));
		}

		[Fact]
		public void WildcardStar_MatchesSuffix()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("RHI/DrawPrimitive", "*/drawprimitive"));
		}

		[Fact]
		public void WildcardMiddle_MatchesPattern()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("GameThread/Total", "*thread*"));
		}

		[Fact]
		public void MultipleWildcards_MatchesAll()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("GameThread/Total/Inner", "*thread*total*"));
		}

		[Fact]
		public void NoMatch_ReturnsFalse()
		{
			Assert.False(CSVStats.CsvStats.DoesSearchStringMatch("frametime", "renderthread"));
		}

		[Fact]
		public void WildcardNoMatch_ReturnsFalse()
		{
			Assert.False(CSVStats.CsvStats.DoesSearchStringMatch("frametime", "render*"));
		}

		[Fact]
		public void EmptySearchString_OnlyMatchesEmpty()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("", ""));
			Assert.False(CSVStats.CsvStats.DoesSearchStringMatch("something", ""));
		}

		[Fact]
		public void WildcardOnly_MatchesAnything()
		{
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("anything", "*"));
			Assert.True(CSVStats.CsvStats.DoesSearchStringMatch("", "*"));
		}
	}

	public class StatSamplesTests
	{
		[Fact]
		public void ComputeAverage_ValidSamples_ReturnsCorrectAverage()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 20, 30, 40, 50 });
			float avg = stat.ComputeAverage();
			Assert.Equal(30.0f, avg, 0.001f);
		}

		[Fact]
		public void ComputeAverage_WithRange_ReturnsCorrectAverage()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 20, 30, 40, 50 });
			float avg = stat.ComputeAverage(1, 4); // 20, 30, 40
			Assert.Equal(30.0f, avg, 0.001f);
		}

		[Fact]
		public void ComputeMaxValue_ReturnsMax()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 50, 20, 40, 30 });
			Assert.Equal(50.0f, stat.ComputeMaxValue());
		}

		[Fact]
		public void ComputeMinValue_ReturnsMin()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 50, 20, 40, 30 });
			Assert.Equal(10.0f, stat.ComputeMinValue());
		}

		[Fact]
		public void ComputeMaxValue_WithRange_ReturnsMaxInRange()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 50, 20, 40, 30 });
			Assert.Equal(50.0f, stat.ComputeMaxValue(0, 3)); // 10, 50, 20
		}

		[Fact]
		public void GetRatioOfFramesInBudget_CorrectRatio()
		{
			var stat = CsvTestHelper.CreateStatSamples("frametime",
				new float[] { 10, 15, 20, 25, 30, 35, 40, 45, 50, 55 });

			float ratio = stat.GetRatioOfFramesInBudget(33.3f); // 10, 15, 20, 25, 30 = 5/10
			Assert.Equal(0.5f, ratio, 0.001f);
		}

		[Fact]
		public void GetRatioOfFramesOverBudget_IsComplementOfInBudget()
		{
			var stat = CsvTestHelper.CreateStatSamples("frametime",
				new float[] { 10, 15, 20, 25, 30, 35, 40, 45, 50, 55 });

			float inBudget = stat.GetRatioOfFramesInBudget(33.3f);
			float overBudget = stat.GetRatioOfFramesOverBudget(33.3f);
			Assert.Equal(1.0f, inBudget + overBudget, 0.001f);
		}

		[Fact]
		public void GetCountOfFramesOverBudget_IgnoringFirstAndLast()
		{
			var stat = CsvTestHelper.CreateStatSamples("frametime",
				new float[] { 100, 10, 20, 30, 100 });

			// Explicitly pass defaults: first and last frames are ignored
			int count = stat.GetCountOfFramesOverBudget(25, IgnoreFirstFrame: true, IgnoreLastFrame: true);
			Assert.Equal(1, count); // Only 30 is over 25 (100s at index 0 and 4 are ignored)
		}

		[Fact]
		public void GetCountOfFramesOverBudget_NoIgnore_CountsAll()
		{
			var stat = CsvTestHelper.CreateStatSamples("frametime",
				new float[] { 100, 10, 20, 30, 100 });

			int count = stat.GetCountOfFramesOverBudget(25, IgnoreFirstFrame: false, IgnoreLastFrame: false);
			Assert.Equal(3, count); // 100 (index 0), 30 (index 3), 100 (index 4)
		}

		[Fact]
		public void ComputeAverageAndTotal_SetsFields()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 10, 20, 30 });
			stat.ComputeAverageAndTotal();
			Assert.Equal(20.0f, stat.average, 0.001f);
			Assert.Equal(60.0, stat.total, 0.001);
		}

		[Fact]
		public void GetNumSamples_ReturnsCount()
		{
			var stat = CsvTestHelper.CreateStatSamples("test", new float[] { 1, 2, 3, 4, 5 });
			Assert.Equal(5, stat.GetNumSamples());
		}

		[Fact]
		public void CompareTo_HigherAverage_IsLess()
		{
			var high = CsvTestHelper.CreateStatSamples("high", new float[] { 100 });
			high.ComputeAverageAndTotal();
			var low = CsvTestHelper.CreateStatSamples("low", new float[] { 10 });
			low.ComputeAverageAndTotal();

			// CompareTo sorts descending (higher average = earlier)
			Assert.True(high.CompareTo(low) < 0);
			Assert.True(low.CompareTo(high) > 0);
		}
	}
}
