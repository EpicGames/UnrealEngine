using CSVStats;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.CsvStatsTests
{
	public class MetadataQueryBuilderTests
	{
		[Fact]
		public void BuildQueryExpressionTree_SimpleEquality_Works()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows");
			Assert.NotNull(expr);

			var metadata = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.True(expr.Evaluate(metadata));
		}

		[Fact]
		public void BuildQueryExpressionTree_SimpleEquality_NoMatch()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows");
			var metadata = CsvTestHelper.CreateMetadata(("platform", "linux"));
			Assert.False(expr.Evaluate(metadata));
		}

		[Fact]
		public void BuildQueryExpressionTree_NotEqual_Works()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform!=windows");
			var metadata = CsvTestHelper.CreateMetadata(("platform", "linux"));
			Assert.True(expr.Evaluate(metadata));

			var metadata2 = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.False(expr.Evaluate(metadata2));
		}

		[Fact]
		public void BuildQueryExpressionTree_And_CombinesCorrectly()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows and config=test");

			var both = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			Assert.True(expr.Evaluate(both));

			var onlyPlatform = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "shipping"));
			Assert.False(expr.Evaluate(onlyPlatform));
		}

		[Fact]
		public void BuildQueryExpressionTree_Or_CombinesCorrectly()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows or platform=linux");

			var win = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.True(expr.Evaluate(win));

			var linux = CsvTestHelper.CreateMetadata(("platform", "linux"));
			Assert.True(expr.Evaluate(linux));

			var mac = CsvTestHelper.CreateMetadata(("platform", "mac"));
			Assert.False(expr.Evaluate(mac));
		}

		[Fact]
		public void BuildQueryExpressionTree_NotOperator_NegatesExpression()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("not(platform=windows)");

			var win = CsvTestHelper.CreateMetadata(("platform", "windows"));
			Assert.False(expr.Evaluate(win));

			var linux = CsvTestHelper.CreateMetadata(("platform", "linux"));
			Assert.True(expr.Evaluate(linux));
		}

		[Fact]
		public void BuildQueryExpressionTree_NestedParentheses_ParsesCorrectly()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree(
				"config=test and (platform=windows or platform=linux)");

			var winTest = CsvTestHelper.CreateMetadata(("config", "test"), ("platform", "windows"));
			Assert.True(expr.Evaluate(winTest));

			var linuxTest = CsvTestHelper.CreateMetadata(("config", "test"), ("platform", "linux"));
			Assert.True(expr.Evaluate(linuxTest));

			var macTest = CsvTestHelper.CreateMetadata(("config", "test"), ("platform", "mac"));
			Assert.False(expr.Evaluate(macTest));

			var winShipping = CsvTestHelper.CreateMetadata(("config", "shipping"), ("platform", "windows"));
			Assert.False(expr.Evaluate(winShipping));
		}

		[Fact]
		public void BuildQueryExpressionTree_ComparisonOperators_Work()
		{
			var ltExpr = MetadataQueryBuilder.BuildQueryExpressionTree("fps<60");
			var gtExpr = MetadataQueryBuilder.BuildQueryExpressionTree("fps>30");
			var leExpr = MetadataQueryBuilder.BuildQueryExpressionTree("fps<=60");
			var geExpr = MetadataQueryBuilder.BuildQueryExpressionTree("fps>=30");

			var meta45 = CsvTestHelper.CreateMetadata(("fps", "45"));
			Assert.True(ltExpr.Evaluate(meta45));
			Assert.True(gtExpr.Evaluate(meta45));
			Assert.True(leExpr.Evaluate(meta45));
			Assert.True(geExpr.Evaluate(meta45));

			var meta60 = CsvTestHelper.CreateMetadata(("fps", "60"));
			Assert.False(ltExpr.Evaluate(meta60));
			Assert.True(gtExpr.Evaluate(meta60));
			Assert.True(leExpr.Evaluate(meta60));
			Assert.True(geExpr.Evaluate(meta60));
		}

		[Fact]
		public void BuildQueryExpressionTree_WildcardValue_MatchesPattern()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("deviceprofile=windows*");

			var win = CsvTestHelper.CreateMetadata(("deviceprofile", "windows"));
			Assert.True(expr.Evaluate(win));

			var winHigh = CsvTestHelper.CreateMetadata(("deviceprofile", "windows_high"));
			Assert.True(expr.Evaluate(winHigh));

			var linux = CsvTestHelper.CreateMetadata(("deviceprofile", "linux"));
			Assert.False(expr.Evaluate(linux));
		}

		[Fact]
		public void BuildQueryExpressionTree_ContainsOperator_Works()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("buildversion~=main");

			var main = CsvTestHelper.CreateMetadata(("buildversion", "++ue5+main-cl-12345"));
			Assert.True(expr.Evaluate(main));

			var release = CsvTestHelper.CreateMetadata(("buildversion", "++ue5+release-5.4"));
			Assert.False(expr.Evaluate(release));
		}

		[Fact]
		public void BuildQueryExpressionTree_CommaNotation_ConvertsToAnd()
		{
			// Old-style comma-separated filters should be treated as AND
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows,config=test");

			var match = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			Assert.True(expr.Evaluate(match));

			var noMatch = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "shipping"));
			Assert.False(expr.Evaluate(noMatch));
		}

		[Fact]
		public void Evaluate_ComplexFilter_MatchesExpectedResults()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree(
				"buildversion=++UE5+Main-CL-52441555 and (platform=windows or platform=linux) and not(deviceprofile=windows_low* or deviceprofile=linux_low*)");

			// Should match: Windows with correct build, non-low profile
			var winHigh = CsvTestHelper.CreateMetadata(
				("buildversion", "++ue5+main-cl-52441555"),
				("platform", "windows"),
				("deviceprofile", "windows"));
			Assert.True(expr.Evaluate(winHigh));

			// Should NOT match: Windows low profile (excluded by not clause)
			var winLow = CsvTestHelper.CreateMetadata(
				("buildversion", "++ue5+main-cl-52441555"),
				("platform", "windows"),
				("deviceprofile", "windows_low"));
			Assert.False(expr.Evaluate(winLow));

			// Should NOT match: wrong build version
			var wrongBuild = CsvTestHelper.CreateMetadata(
				("buildversion", "++ue5+release-5.4-cl-99999"),
				("platform", "windows"),
				("deviceprofile", "windows"));
			Assert.False(expr.Evaluate(wrongBuild));

			// Should NOT match: Mac platform (not in the or list)
			var mac = CsvTestHelper.CreateMetadata(
				("buildversion", "++ue5+main-cl-52441555"),
				("platform", "mac"),
				("deviceprofile", "mac"));
			Assert.False(expr.Evaluate(mac));
		}

		[Fact]
		public void Evaluate_MissingMetadataKey_ReturnsFalse()
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree("platform=windows");
			var emptyMeta = CsvTestHelper.CreateMetadata();
			Assert.False(expr.Evaluate(emptyMeta));
		}

		[Theory]
		[InlineData("platform=windows")]
		[InlineData("a=1 and b=2")]
		[InlineData("a=1 or b=2")]
		[InlineData("not(a=1)")]
		[InlineData("(a=1 or b=2) and c=3")]
		public void BuildQueryExpressionTree_ValidExpressions_DoNotThrow(string filter)
		{
			var expr = MetadataQueryBuilder.BuildQueryExpressionTree(filter);
			Assert.NotNull(expr);
		}
	}
}
