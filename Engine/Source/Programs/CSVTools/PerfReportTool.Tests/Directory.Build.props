<?xml version="1.0" encoding="utf-8"?>
<!-- Override parent Directory.Build.props to prevent UE build targets from breaking test discovery -->
<Project>
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
  </PropertyGroup>
</Project>
