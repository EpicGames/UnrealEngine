// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Nodes;
using Xunit;

namespace PerfReportTool.Tests.Integration
{
	[Trait("Category", "Integration")]
	[Collection("EndToEnd")]
	public class CliArgumentTests : IClassFixture<TestDataFixture>, IDisposable
	{
		private readonly TestDataFixture _fixture;
		private readonly string _outputDir;
		private readonly string _toolPath;

		public CliArgumentTests(TestDataFixture fixture)
		{
			_fixture = fixture;
			_outputDir = Path.Combine(Path.GetTempPath(), "PerfReportToolCli_" + Path.GetRandomFileName());
			Directory.CreateDirectory(_outputDir);

			string assemblyDir = Path.GetDirectoryName(typeof(PerfReportTool.Version).Assembly.Location);
			_toolPath = Path.Combine(assemblyDir, "PerfreportTool.dll");
		}

		public void Dispose()
		{
			if (Directory.Exists(_outputDir))
			{
				try { Directory.Delete(_outputDir, true); } catch { }
			}
		}

		private (int exitCode, string output) RunTool(string arguments, int timeoutMs = 300000)
		{
			var psi = new ProcessStartInfo
			{
				FileName = "dotnet",
				Arguments = $"\"{_toolPath}\" {arguments}",
				RedirectStandardOutput = true,
				RedirectStandardError = true,
				UseShellExecute = false,
				CreateNoWindow = true
			};

			using var process = Process.Start(psi);
			string stdout = process.StandardOutput.ReadToEnd();
			string stderr = process.StandardError.ReadToEnd();
			if (!process.WaitForExit(timeoutMs))
			{
				try { process.Kill(); } catch { }
				return (-1, stdout + stderr + "\n[ERROR] Process timed out");
			}

			return (process.ExitCode, stdout + stderr);
		}

		/// <summary>Common args for bulk mode with email table output.</summary>
		private string BulkArgs(string outputDir)
		{
			return $"-csvdir \"{_fixture.GetPerfCsvDir()}\" -emailtable -recurse -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports -searchpattern Perf_*.csv.bin";
		}

		private string GeneratePrcCache(string name)
		{
			string cacheDir = Path.Combine(_outputDir, name + "_cache");
			Directory.CreateDirectory(cacheDir);
			string outputDir = Path.Combine(_outputDir, name + "_html");

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCacheInvalidate " +
				$"-summaryTableCache \"{cacheDir}\"");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(cacheDir, "*.prc").Length > 0,
				$"PRC cache generation failed — no .prc files in {cacheDir}. Output: {output.Substring(0, Math.Min(500, output.Length))}");

			return cacheDir;
		}

		/// <summary>Read the first HTML summary table file from a directory.</summary>
		private string ReadFirstHtml(string dir, string pattern = "*.html")
		{
			Assert.True(Directory.Exists(dir), $"Output directory does not exist: {dir}");
			string[] files = Directory.GetFiles(dir, pattern, SearchOption.TopDirectoryOnly);
			Assert.True(files.Length > 0,
				$"No '{pattern}' files found in {dir}. Tool may have failed silently — check tool output for errors.");
			return File.ReadAllText(files[0]);
		}

		#region Data Filtering

		[Fact]
		public void MinX_MaxX_FrameTruncation_AffectsOutput()
		{
			_fixture.RequireTestData();
			string outputFull = Path.Combine(_outputDir, "MinMaxX_Full");
			string outputCropped = Path.Combine(_outputDir, "MinMaxX_Cropped");

			RunTool(BulkArgs(outputFull));
			RunTool($"{BulkArgs(outputCropped)} -minx 10 -maxx 50");

			string htmlFull = ReadFirstHtml(outputFull);
			string htmlCropped = ReadFirstHtml(outputCropped);

			Assert.NotNull(htmlFull);
			Assert.NotNull(htmlCropped);
			// Cropped output should differ from full output (different stats computed over fewer frames)
			Assert.NotEqual(htmlFull, htmlCropped);
		}

		[Fact]
		public void NoStripEvents_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NoStrip");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -nostripevents");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0, "Should produce HTML output");
		}

		#endregion

		#region Summary Table Format

		[Fact]
		public void CollateTable_ProducesCollatedHtmlFile()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Collate");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -collatetable");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir));
			string[] collatedFiles = Directory.GetFiles(outputDir, "*Collated*", SearchOption.TopDirectoryOnly);
			Assert.True(collatedFiles.Length > 0, "Collate table should produce a *Collated* file");
		}

		[Fact]
		public void CsvTable_ProducesCsvNotHtml()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "CsvTable");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -csvtable");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir));
			string[] csvFiles = Directory.GetFiles(outputDir, "*.csv", SearchOption.TopDirectoryOnly);
			Assert.True(csvFiles.Length > 0, "CsvTable should produce CSV output files");
			// CSV file should contain comma-separated data with a header row
			string csvContent = File.ReadAllText(csvFiles[0]);
			Assert.Contains(",", csvContent);
		}

		[Fact]
		public void TransposeTable_OutputContainsTransposeMarker()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Transpose");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -transposetable");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// Transposed tables embed SummaryTableIsTransposed=true in JavaScript
			Assert.Contains("SummaryTableIsTransposed", html);
		}

		[Fact]
		public void ReverseTable_ReversesRowOrder()
		{
			_fixture.RequireTestData();
			string outputNormal = Path.Combine(_outputDir, "ReverseNormal");
			string outputReversed = Path.Combine(_outputDir, "ReverseReversed");

			RunTool(BulkArgs(outputNormal));
			RunTool($"{BulkArgs(outputReversed)} -reversetable");

			string htmlNormal = ReadFirstHtml(outputNormal);
			string htmlReversed = ReadFirstHtml(outputReversed);

			Assert.NotNull(htmlNormal);
			Assert.NotNull(htmlReversed);
			// Reversed table should produce different HTML (different row ordering)
			Assert.NotEqual(htmlNormal, htmlReversed);
		}

		[Fact]
		public void ScrollableTable_EmbedsScrollableJavaScript()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Scrollable");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -scrollabletable");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			Assert.Contains("regenerateStickyColumnCss", html);
		}

		[Fact]
		public void ColorizeTable_Off_ProducesNoBackgroundColors()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "ColorOff");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -colorizetable off");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// With colorize off, cells should not have bgcolor attributes from thresholds
			// (there may still be some from other sources, but far fewer)
			Assert.DoesNotContain("bgcolor='#ff", html.ToLower());
		}

		[Fact]
		public void ColorizeTable_Auto_ProducesBackgroundColors()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "ColorAuto");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -colorizetable auto");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// Auto colorize should produce bgcolor attributes on cells
			Assert.Contains("bgcolor", html.ToLower());
		}

		[Fact]
		public void SpreadsheetFriendly_PrefixesStringValues()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Spreadsheet");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -spreadsheetfriendly");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// Spreadsheet-friendly mode prefixes non-numeric text entries with a single quote
			// Look for the quote prefix pattern in table cells: >'text
			Assert.Contains(">'", html);
		}

		[Fact]
		public void NoSummaryMinMax_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NoMinMax");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -nosummaryminmax");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		[Fact]
		public void SummaryTableFilename_UsesCustomName()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "CustomFilename");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -summarytablefilename CustomTable.html");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir));
			string[] customFiles = Directory.GetFiles(outputDir, "CustomTable*");
			Assert.True(customFiles.Length > 0, "Output should use the custom filename 'CustomTable'");
			// The default name "SummaryTable" should NOT appear
			string[] defaultFiles = Directory.GetFiles(outputDir, "SummaryTable*");
			Assert.True(defaultFiles.Length == 0, "Default 'SummaryTable' filename should not be used when custom name is specified");
		}

		#endregion

		#region Diff Rows

		[Fact]
		public void AddDiffRows_InsertsAdditionalRows()
		{
			_fixture.RequireTestData();
			string outputNoDiff = Path.Combine(_outputDir, "DiffRowsNone");
			string outputWithDiff = Path.Combine(_outputDir, "DiffRowsWith");

			RunTool(BulkArgs(outputNoDiff));
			RunTool($"{BulkArgs(outputWithDiff)} -adddiffrows");

			string htmlNoDiff = ReadFirstHtml(outputNoDiff);
			string htmlWithDiff = ReadFirstHtml(outputWithDiff);

			Assert.NotNull(htmlNoDiff);
			Assert.NotNull(htmlWithDiff);
			// Diff rows version should have more <tr> elements
			int rowCountNoDiff = htmlNoDiff.Split("<tr").Length;
			int rowCountWithDiff = htmlWithDiff.Split("<tr").Length;
			Assert.True(rowCountWithDiff >= rowCountNoDiff,
				$"Diff rows should add rows: {rowCountNoDiff} without vs {rowCountWithDiff} with");
		}

		[Fact]
		public void AddDiffRows_WithSortByDiff_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "DiffRowsSort");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -adddiffrows -sortcolumnsbydiff");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		[Fact]
		public void ShowOnlyDiffRows_OutputsFewerRows()
		{
			_fixture.RequireTestData();
			string outputWithDiff = Path.Combine(_outputDir, "DiffRowsFull");
			string outputOnlyDiff = Path.Combine(_outputDir, "OnlyDiffRows");

			RunTool($"{BulkArgs(outputWithDiff)} -adddiffrows");
			RunTool($"{BulkArgs(outputOnlyDiff)} -adddiffrows -showonlydiffrows");

			string htmlFull = ReadFirstHtml(outputWithDiff);
			string htmlOnly = ReadFirstHtml(outputOnlyDiff);

			Assert.NotNull(htmlFull);
			Assert.NotNull(htmlOnly);
			// ShowOnlyDiffRows skips data rows entirely, so output should have fewer <tr> elements
			int rowCountFull = htmlFull.Split("<tr").Length;
			int rowCountOnly = htmlOnly.Split("<tr").Length;
			Assert.True(rowCountOnly < rowCountFull,
				$"ShowOnlyDiffRows should produce fewer rows: {rowCountFull} full vs {rowCountOnly} only-diff");
		}

		#endregion

		#region Stat Reading Modes

		[Fact]
		public void ReadStatMinMax_AddsMinMaxColumns()
		{
			_fixture.RequireTestData();
			string outputNormal = Path.Combine(_outputDir, "ReadStatNormal");
			string outputMinMax = Path.Combine(_outputDir, "ReadStatMinMax");

			RunTool(BulkArgs(outputNormal));
			RunTool($"{BulkArgs(outputMinMax)} -readstatminmax");

			string htmlNormal = ReadFirstHtml(outputNormal);
			string htmlMinMax = ReadFirstHtml(outputMinMax);

			Assert.NotNull(htmlNormal);
			Assert.NotNull(htmlMinMax);
			// MinMax mode should produce a larger table (more columns)
			Assert.True(htmlMinMax.Length > htmlNormal.Length,
				"ReadStatMinMax should produce more output with additional min/max columns");
		}

		[Fact]
		public void ReadStatAvg_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "ReadStatAvg");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -readstatavg");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region No Reports / NoDetailedReports

		[Fact(Skip = "BUG: -noreports does not gate WriteSummaryTableReport, causing WriteSummaryTableReport to run on a table with rows but no columns (GenerateReport, which populates row data, is skipped). Crashes in WildcardMatchStringList binary search on empty list.")]
		public void NoReports_SkipsAllReports_StillGeneratesCache()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NoReports");
			string cacheDir = Path.Combine(_outputDir, "NoReportsCache");
			Directory.CreateDirectory(cacheDir);

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-noreports " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCacheInvalidate " +
				$"-summaryTableCache \"{cacheDir}\"");

			Assert.DoesNotContain("[ERROR]", output);
		}

		#endregion

		#region Variable Dumping

		[Fact]
		public void DumpVariables_PrintsVariablesToStdout()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "DumpVars");
			string csvDir = _fixture.GetPerfCsvDir();
			string[] csvFiles = Directory.GetFiles(csvDir, "Perf_*.csv.bin");
			Assert.True(csvFiles.Length > 0, $"No Perf_*.csv.bin files found in {csvDir}");

			var (exitCode, output) = RunTool(
				$"-csv \"{csvFiles[0]}\" -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports -dumpvariables");

			Assert.DoesNotContain("[ERROR]", output);
			// -dumpvariables should print variable names and values to stdout
			// Variables typically include budget values from ReportTypes.xml
			Assert.True(output.Length > 100, "DumpVariables should produce substantial stdout output");
		}

		#endregion

		#region ListSummaryTables

		[Fact]
		public void ListSummaryTables_ListsTableNames()
		{
			var (exitCode, output) = RunTool("-listsummarytables");
			Assert.DoesNotContain("[ERROR]", output);
			// Should print "Listing summary tables:" followed by table names
			Assert.Contains("summary table", output.ToLower());
		}

		#endregion

		#region Collation Options

		[Fact]
		public void CollateTableOnly_ProducesOnlyCollatedOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "CollateOnly");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -collatetableonly");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir));
			// With -collatetableonly, should NOT produce a non-collated summary table
			string[] htmlFiles = Directory.GetFiles(outputDir, "*.html", SearchOption.TopDirectoryOnly);
			Assert.True(htmlFiles.Length > 0, "Should produce at least one HTML file");
		}

		[Fact]
		public void NoWeightedAvg_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NoWeight");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -collatetable -noweightedavg");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region Column Sort Modes

		[Theory]
		[InlineData("sortbymax")]
		[InlineData("sortbyavg")]
		[InlineData("wildcardSortByMax")]
		[InlineData("wildcardSortByAvg")]
		public void ColumnSortMode_ProducesOutput(string mode)
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, $"ColSort_{mode}");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -columnsortmode {mode}");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0,
				$"Column sort mode '{mode}' should produce HTML output");
		}

		#endregion

		#region Override Metadata

		[Fact]
		public void OverrideMetadata_AcceptedWithoutError()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "OverrideMeta");

			// -overrideMetadata applies to the -metadataProxy path, not bulk CSV mode.
			// Verify the argument is accepted and processed without error in bulk mode.
			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} -overridemetadata platform=TestOverride,targetframerate=60");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region Cache Options

		[Fact]
		public void SummaryTableCacheReadOnly_DoesNotWriteNewFiles()
		{
			_fixture.RequireTestData();
			string cacheDir = GeneratePrcCache("CacheRO");
			int prcCountBefore = Directory.GetFiles(cacheDir, "*.prc").Length;

			string outputDir = Path.Combine(_outputDir, "CacheRO_Out");
			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-summaryTableCacheUseOnlyCsvID " +
				$"-summaryTableCacheReadOnly " +
				$"-summaryTableCache \"{cacheDir}\"");

			Assert.DoesNotContain("[ERROR]", output);
			int prcCountAfter = Directory.GetFiles(cacheDir, "*.prc").Length;
			Assert.Equal(prcCountBefore, prcCountAfter);
		}

		[Fact]
		public void SummaryTableCachePurgeInvalid_ProducesOutput()
		{
			_fixture.RequireTestData();
			string cacheDir = GeneratePrcCache("CachePurge");
			string outputDir = Path.Combine(_outputDir, "CachePurge_Out");

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-summaryTableCacheUseOnlyCsvID " +
				$"-summaryTableCachePurgeInvalid " +
				$"-summaryTableCache \"{cacheDir}\"");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region Performance/Threading Options

		[Fact]
		public void CsvToSvgSequential_ProducesIdenticalOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Sequential");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -csvtosvgsequential");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		[Fact]
		public void GraphThreads_CustomValue_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "GraphThreads");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -graphthreads 1");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		[Fact]
		public void PrecacheSettings_CustomValues_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Precache");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -precachecount 2 -precachethreads 2");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region Stat Threshold Column Filter

		[Fact]
		public void SummaryTableStatThreshold_ReducesColumnCount()
		{
			_fixture.RequireTestData();
			string outputNormal = Path.Combine(_outputDir, "StatThresholdNormal");
			string outputFiltered = Path.Combine(_outputDir, "StatThresholdFiltered");

			RunTool(BulkArgs(outputNormal));
			// High threshold should filter out many low-value stat columns
			RunTool($"{BulkArgs(outputFiltered)} -summarytablestatthreshold 999999");

			string htmlNormal = ReadFirstHtml(outputNormal);
			string htmlFiltered = ReadFirstHtml(outputFiltered);

			Assert.NotNull(htmlNormal);
			Assert.NotNull(htmlFiltered);
			// Filtered output should have fewer <th> headers (fewer columns)
			int colCountNormal = htmlNormal.Split("<th").Length;
			int colCountFiltered = htmlFiltered.Split("<th").Length;
			Assert.True(colCountFiltered < colCountNormal,
				$"High stat threshold should filter out columns: {colCountNormal} normal vs {colCountFiltered} filtered");
		}

		#endregion

		#region Hide Metadata Columns

		[Fact]
		public void HideMetadataColumns_RemovesMetadataFromTable()
		{
			_fixture.RequireTestData();
			string outputNormal = Path.Combine(_outputDir, "HideMetaNormal");
			string outputHidden = Path.Combine(_outputDir, "HideMetaHidden");

			RunTool(BulkArgs(outputNormal));
			RunTool($"{BulkArgs(outputHidden)} -hidemetadatacolumns");

			string htmlNormal = ReadFirstHtml(outputNormal);
			string htmlHidden = ReadFirstHtml(outputHidden);

			Assert.NotNull(htmlNormal);
			Assert.NotNull(htmlHidden);
			// Hidden metadata output should be smaller (fewer columns)
			int colCountNormal = htmlNormal.Split("<th").Length;
			int colCountHidden = htmlHidden.Split("<th").Length;
			Assert.True(colCountHidden < colCountNormal,
				$"Hiding metadata should reduce columns: {colCountNormal} normal vs {colCountHidden} hidden");
		}

		#endregion

		#region Response File

		[Fact]
		public void ResponseFile_ProcessesCorrectly()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "ResponseFile");
			string responseFile = Path.Combine(_outputDir, "response.txt");
			string csvDir = _fixture.GetPerfCsvDir();

			File.WriteAllLines(responseFile, new[]
			{
				$"-csvdir \"{csvDir}\"",
				"-emailtable",
				"-recurse",
				$"-o \"{outputDir}\"",
				"-nowatermarks",
				"-nodetailedreports",
				"-searchpattern Perf_*.csv.bin"
			});

			var (exitCode, output) = RunTool($"-response \"{responseFile}\"");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir));
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0,
				"Response file should produce identical output to direct arguments");
		}

		#endregion

		#region SummaryTable JS Data

		[Fact]
		public void SummaryTableJsData_EmbedsJavaScriptData()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "JsData");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -summarytablejsdata");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// JS data mode embeds row/column data in a <script> section
			Assert.Contains("<script>", html);
			Assert.Contains("elementType", html);
		}

		#endregion

		#region Max Summary Table String Length

		[Fact]
		public void MaxSummaryTableStringLength_TruncatesLongStrings()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "MaxStrLen");

			// Use a very short max length to force truncation
			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -maxsummarytablestringlength 5");

			Assert.DoesNotContain("[ERROR]", output);
			string html = ReadFirstHtml(outputDir);
			Assert.NotNull(html);
			// Truncated strings end with "..."
			Assert.Contains("...", html);
		}

		#endregion

		#region PerfLog

		[Fact]
		public void PerfLog_ProducesTimingOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "PerfLogTest");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -perflog");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.Contains("[PerfLog]", output);
			// PerfLog lines include timing in ms format
			Assert.Contains("ms", output);
			Assert.Contains("TOTAL:", output);
		}

		#endregion

		#region ListFiles

		[Fact]
		public void ListFiles_PrintsMatchingFilenames()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "ListFiles");

			var (exitCode, output) = RunTool(
				$"-csvdir \"{_fixture.GetPerfCsvDir()}\" -recurse -o \"{outputDir}\" " +
				$"-searchpattern Perf_*.csv.bin -listfiles");

			Assert.DoesNotContain("[ERROR]", output);
			// Should print filenames that match the search pattern
			Assert.Contains("Perf_", output);
			Assert.Contains(".csv.bin", output);
		}

		#endregion

		#region Regression Analysis

		[Fact]
		public void OnlyShowRegressedColumns_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "Regression");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -onlyshowregressedcolumns");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		[Fact]
		public void RegressionWithCustomThresholds_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "RegressionCustom");

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-onlyshowregressedcolumns -regressionstddevthreshold 3 -regressionoutlierstddevthreshold 5");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region Require Metadata

		[Fact]
		public void RequireMetadata_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "RequireMeta");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -requiremetadata");

			Assert.DoesNotContain("[ERROR]", output);
		}

		#endregion

		#region Condensed Table

		[Fact]
		public void TransposeCollatedTable_ContainsTransposeMarker()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "TransposeCollated");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -collatetable -transposecollatedtable");

			Assert.DoesNotContain("[ERROR]", output);
			// Check the collated file for transpose marker
			string[] collatedFiles = Directory.GetFiles(outputDir, "*Collated*");
			Assert.True(collatedFiles.Length > 0);
			string html = File.ReadAllText(collatedFiles[0]);
			Assert.Contains("SummaryTableIsTransposed", html);
		}

		#endregion

		#region NoCsvCacheFiles

		[Fact]
		public void NoCsvCacheFiles_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NoCsvCache");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -nocsvcachefiles");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region MinFrameCount

		[Fact]
		public void MinFrameCount_ExcludesCsvsAndLogs()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "MinFrameCount");

			// Use a very high frame count to filter out all CSVs
			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -minframecount 999999999");

			Assert.DoesNotContain("[ERROR]", output);
			// Should log that CSVs were excluded due to frame count
			Assert.Contains("frame count", output.ToLower());
		}

		#endregion

		#region Sort Trailing Digits As Numeric

		[Fact]
		public void SortTrailingDigitsAsNumeric_ProducesOutput()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "NumSort");

			var (exitCode, output) = RunTool($"{BulkArgs(outputDir)} -sorttrailingdigitsasnumeric");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0);
		}

		#endregion

		#region JSON output format with Bulk Mode

		[Fact]
		public void SummaryTableOutputFormatsJson_WithBulkMode_ProducesJsonAndHtml()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "JsonBulk");

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-summaryTableOutputFormats html,json");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0,
				"Bulk mode should produce HTML output alongside JSON");
			Assert.True(Directory.GetFiles(outputDir, "*.json").Length > 0,
				"JSON output should be written when summaryTableOutputFormats includes json");
		}

		[Fact]
		public void SummaryTableOutputFormatsJson_Collated_WithBulkMode_ProducesJsonAndHtml()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "CollatedJsonBulk");

			var (exitCode, output) = RunTool(
				$"{BulkArgs(outputDir)} " +
				$"-summaryTableOutputFormats html,json -collateTable");

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(outputDir, "*.html").Length > 0,
				"Bulk mode should produce HTML output alongside collated JSON");
			string[] jsonFiles = Directory.GetFiles(outputDir, "*.json");
			Assert.True(jsonFiles.Length > 0,
				"Collated JSON should be written when summaryTableOutputFormats includes json with -collateTable");
		}

		#endregion

		#region summaryTableOutputFormats combinations

		/// <summary>
		/// Helper: run bulk mode (without -emailtable) with the given -summaryTableOutputFormats value
		/// and return the set of output file extensions produced (lowercase, e.g. ".html").
		/// </summary>
		private HashSet<string> RunOutputFormats(string tag, string formats, string extraArgs = "")
		{
			string outputDir = Path.Combine(_outputDir, tag);
			var (exitCode, output) = RunTool(
				$"-csvdir \"{_fixture.GetPerfCsvDir()}\" -recurse -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports -searchpattern Perf_*.csv.bin " +
				$"-summaryTableOutputFormats {formats} {extraArgs}");

			Assert.DoesNotContain("[ERROR]", output);

			string[] files = Directory.GetFiles(outputDir);
			var extensions = new HashSet<string>(
				files.Select(f => Path.GetExtension(f).ToLowerInvariant()),
				StringComparer.OrdinalIgnoreCase);
			return extensions;
		}

		[Fact]
		public void OutputFormats_DefaultIsHtmlOnly()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "FmtDefault");
			var (_, output) = RunTool(
				$"-csvdir \"{_fixture.GetPerfCsvDir()}\" -recurse -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports -searchpattern Perf_*.csv.bin");
			Assert.DoesNotContain("[ERROR]", output);

			string[] files = Directory.GetFiles(outputDir);
			var extensions = files.Select(f => Path.GetExtension(f).ToLowerInvariant()).ToHashSet(StringComparer.OrdinalIgnoreCase);
			Assert.Contains(".html", extensions);
			Assert.DoesNotContain(".csv", extensions);
			Assert.DoesNotContain(".json", extensions);
		}

		[Fact]
		public void OutputFormats_HtmlOnly_ProducesOnlyHtml()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtHtml", "html");
			Assert.Contains(".html", ext);
			Assert.DoesNotContain(".csv", ext);
			Assert.DoesNotContain(".json", ext);
		}

		[Fact]
		public void OutputFormats_CsvOnly_ProducesOnlyCsv()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtCsv", "csv");
			Assert.Contains(".csv", ext);
			Assert.DoesNotContain(".html", ext);
			Assert.DoesNotContain(".json", ext);
		}

		[Fact]
		public void OutputFormats_JsonOnly_ProducesOnlyJson()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtJson", "json");
			Assert.Contains(".json", ext);
			Assert.DoesNotContain(".html", ext);
			Assert.DoesNotContain(".csv", ext);
		}

		[Fact]
		public void OutputFormats_HtmlAndCsv_ProducesBoth()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtHtmlCsv", "html,csv");
			Assert.Contains(".html", ext);
			Assert.Contains(".csv", ext);
			Assert.DoesNotContain(".json", ext);
		}

		[Fact]
		public void OutputFormats_HtmlAndJson_ProducesBoth()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtHtmlJson", "html,json");
			Assert.Contains(".html", ext);
			Assert.Contains(".json", ext);
			Assert.DoesNotContain(".csv", ext);
		}

		[Fact]
		public void OutputFormats_CsvAndJson_ProducesBoth()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtCsvJson", "csv,json");
			Assert.Contains(".csv", ext);
			Assert.Contains(".json", ext);
			Assert.DoesNotContain(".html", ext);
		}

		[Fact]
		public void OutputFormats_AllThree_ProducesHtmlCsvAndJson()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtAll", "html,csv,json");
			Assert.Contains(".html", ext);
			Assert.Contains(".csv", ext);
			Assert.Contains(".json", ext);
		}

		[Fact]
		public void OutputFormats_Collated_ProducesCorrectFiles()
		{
			_fixture.RequireTestData();
			var ext = RunOutputFormats("FmtCollated", "html,csv,json", "-collateTable");
			Assert.Contains(".html", ext);
			Assert.Contains(".csv", ext);
			Assert.Contains(".json", ext);
		}

		[Fact]
		public void OutputFormats_DeprecatedCsvTable_ProducesCsvOnly()
		{
			_fixture.RequireTestData();
			string outputDir = Path.Combine(_outputDir, "FmtCsvTableCompat");
			var (exitCode, output) = RunTool(
				$"-csvdir \"{_fixture.GetPerfCsvDir()}\" -recurse -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports -searchpattern Perf_*.csv.bin " +
				$"-csvTable");

			Assert.DoesNotContain("[ERROR]", output);

			string[] files = Directory.GetFiles(outputDir);
			var extensions = new HashSet<string>(
				files.Select(f => Path.GetExtension(f).ToLowerInvariant()),
				StringComparer.OrdinalIgnoreCase);

			Assert.Contains(".csv", extensions);
			Assert.DoesNotContain(".html", extensions);
			Assert.DoesNotContain(".json", extensions);
		}

		#endregion

		#region JSON output format — compatible arguments

		/// <summary>
		/// Helper: run with -summaryTableOutputFormats json via PRC cache.
		/// For collated output, pass bCollated=true which adds -collateTableOnly.
		/// Returns parsed JSON root and raw output.
		/// </summary>
		private (Dictionary<string, JsonElement> root, string output) RunJsonFromCache(
			string cacheName, bool bCollated, string extraArgs = "")
		{
			string cacheDir = GeneratePrcCache(cacheName);
			string jsonOutputDir = Path.Combine(_outputDir, cacheName + "_out");
			Directory.CreateDirectory(jsonOutputDir);

			string collateArg = bCollated ? "-collateTableOnly" : "";
			var (exitCode, output) = RunTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableOutputFormats json " +
				$"{collateArg} " +
				$"-nodetailedreports -nowatermarks " +
				$"-precacheThreads 1 " +
				$"-o \"{jsonOutputDir}\" " +
				extraArgs);

			Assert.DoesNotContain("[ERROR]", output);

			string jsonPath = Path.Combine(jsonOutputDir, "SummaryTable.json");
			Assert.True(File.Exists(jsonPath), $"JSON not generated for {cacheName} at {jsonPath}");

			string content = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(content);
			Assert.NotNull(root);
			return (root, output);
		}

		private (Dictionary<string, JsonElement> root, string output) RunFilteredJson(
			string cacheName, string extraArgs = "")
			=> RunJsonFromCache(cacheName, bCollated: false, extraArgs);

		private (Dictionary<string, JsonElement> root, string output) RunCollatedJson(
			string cacheName, string extraArgs = "")
			=> RunJsonFromCache(cacheName, bCollated: true, extraArgs);

		/// <summary>Collect all column keys across all rows in a flat JSON dict.</summary>
		private static HashSet<string> CollectAllKeys(Dictionary<string, JsonElement> root)
		{
			var keys = new HashSet<string>();
			foreach (var entry in root)
			{
				var rowDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entry.Value.GetRawText());
				foreach (string key in rowDict.Keys)
				{
					keys.Add(key);
				}
			}
			return keys;
		}

		// -customTable: override the stat filter list entirely
		[Fact]
		public void JsonFiltered_CustomTable_OnlyIncludesSpecifiedStats()
		{
			_fixture.RequireTestData();

			var defaultKeys = CollectAllKeys(RunFilteredJson("JFCustomBase").root);
			var customKeys = CollectAllKeys(RunFilteredJson("JFCustom", "-customTable MVP,HitchTimePercent").root);

			Assert.True(customKeys.Count < defaultKeys.Count,
				$"Custom filter should produce fewer columns ({customKeys.Count}) than default ({defaultKeys.Count})");
			Assert.True(customKeys.Contains("MVP"), "Custom filter should include MVP");
			Assert.True(customKeys.Contains("HitchTimePercent"), "Custom filter should include HitchTimePercent");
		}

		// -customTableSort: override the row sort columns
		[Fact]
		public void JsonCollated_CustomTableSort_ChangesCollationKeys()
		{
			_fixture.RequireTestData();

			var defaultRoot = RunCollatedJson("JCSort_Default").root;
			var customRoot = RunCollatedJson("JCSort_Custom", "-customTableSort Platform").root;

			Assert.True(customRoot.Count > 0);
			// With a broader sort key the default will produce more groups;
			// with just Platform there should be fewer or equal groups
			Assert.True(customRoot.Count <= defaultRoot.Count,
				$"Sorting by fewer columns should produce fewer groups ({customRoot.Count}) " +
				$"than the default ({defaultRoot.Count})");
		}

		// -summaryTable <name>: select a named summary table definition from the XML
		[Fact]
		public void JsonFiltered_SummaryTableName_UsesNamedDefinition()
		{
			_fixture.RequireTestData();

			// Use the "memory" summary table (filters to MemoryFreeMB*, PhysicalUsedMB*, Total Time*)
			var keys = CollectAllKeys(RunFilteredJson("JFMemory", "-summaryTable memory").root);

			// "memory" table should NOT include stats from "default" like MVP, Hitches, etc.
			Assert.False(keys.Any(k => k.StartsWith("MVP")),
				"Memory summary table should not include MVP stats");
			Assert.False(keys.Any(k => k.StartsWith("Hitches")),
				"Memory summary table should not include Hitches stats");
		}

		// -summaryTableXmlAppend: append additional stats to the filter list
		[Fact]
		public void JsonFiltered_SummaryTableXmlAppend_AddsExtraStats()
		{
			_fixture.RequireTestData();

			var baseKeys = CollectAllKeys(RunFilteredJson("JFAppend_Base", "-summaryTable condensed").root);
			var appendKeys = CollectAllKeys(RunFilteredJson("JFAppend_Extra",
				"-summaryTable condensed -summaryTableXmlAppend framecount*").root);

			Assert.True(appendKeys.Count >= baseKeys.Count,
				$"Appending stats should add columns: base={baseKeys.Count}, appended={appendKeys.Count}");
			Assert.True(appendKeys.Any(k => k.ToLower().Contains("framecount")),
				"Appended stat 'framecount*' should appear in the output");
		}

		// -weightByColumn: weight collated averages by a specific column
		[Fact]
		public void JsonCollated_WeightByColumn_ProducesValidOutput()
		{
			_fixture.RequireTestData();

			var defaultRoot = RunCollatedJson("JCWeight_Default").root;
			var weightedRoot = RunCollatedJson("JCWeight_By", "-weightByColumn framecount").root;

			Assert.True(defaultRoot.Count > 0);
			Assert.True(weightedRoot.Count > 0);
			// Both should produce valid output with the same row keys
			Assert.Equal(defaultRoot.Count, weightedRoot.Count);
		}

		// -noWeightedAvg: disable weighted averages
		[Fact]
		public void JsonCollated_NoWeightedAvg_ProducesValidOutput()
		{
			_fixture.RequireTestData();
			Assert.True(RunCollatedJson("JCNoWeight", "-noWeightedAvg").root.Count > 0);
		}

		// -reverseTable: reverse the row order
		[Fact]
		public void JsonFiltered_ReverseTable_ReversesRowOrder()
		{
			_fixture.RequireTestData();

			// Generate both variants (validates no errors via RunFilteredJson assertions)
			RunFilteredJson("JFReverse_Normal");
			RunFilteredJson("JFReverse_Rev", "-reverseTable");

			// Re-read with JsonNode.Parse which preserves document key order,
			// unlike Dictionary<string,T> whose enumeration order is unspecified.
			string normalPath = Path.Combine(_outputDir, "JFReverse_Normal_out", "SummaryTable.json");
			string reversedPath = Path.Combine(_outputDir, "JFReverse_Rev_out", "SummaryTable.json");
			var normalKeys = JsonNode.Parse(File.ReadAllText(normalPath))!.AsObject().Select(kv => kv.Key).ToList();
			var reversedKeys = JsonNode.Parse(File.ReadAllText(reversedPath))!.AsObject().Select(kv => kv.Key).ToList();

			Assert.Equal(normalKeys.Count, reversedKeys.Count);
			if (normalKeys.Count >= 2)
			{
				// First row in normal should be last in reversed (or vice versa)
				Assert.Equal(normalKeys.First(), reversedKeys.Last());
				Assert.Equal(normalKeys.Last(), reversedKeys.First());
			}
		}

		// -summaryTableStatThreshold: filter out low-value stat columns
		[Fact]
		public void JsonFiltered_StatThreshold_FiltersLowValueColumns()
		{
			_fixture.RequireTestData();

			var baseKeys = CollectAllKeys(RunFilteredJson("JFThresh_Base").root);
			var filteredKeys = CollectAllKeys(RunFilteredJson("JFThresh_High",
				"-summaryTableStatThreshold 999999").root);

			Assert.True(filteredKeys.Count < baseKeys.Count,
				$"High stat threshold should remove columns: base={baseKeys.Count}, filtered={filteredKeys.Count}");
		}

		// -hideMetadataColumns: remove metadata columns from the output
		[Fact]
		public void JsonFiltered_HideMetadataColumns_RemovesMetadata()
		{
			_fixture.RequireTestData();

			var withKeys = CollectAllKeys(RunFilteredJson("JFHideMeta_With").root);
			var withoutKeys = CollectAllKeys(RunFilteredJson("JFHideMeta_Without",
				"-hideMetadataColumns").root);

			Assert.True(withoutKeys.Count < withKeys.Count,
				$"Hiding metadata should remove columns: with={withKeys.Count}, without={withoutKeys.Count}");
		}

		// -collatedStringVisibility: control string column visibility in collated output
		[Theory]
		[InlineData("show")]
		[InlineData("hide")]
		[InlineData("auto")]
		public void JsonCollated_CollatedStringVisibility_ProducesValidOutput(string visibility)
		{
			_fixture.RequireTestData();
			Assert.True(RunCollatedJson($"JCStrVis_{visibility}",
				$"-collatedStringVisibility {visibility}").root.Count > 0);
		}

		// -collatedDateVisibility: control date column visibility
		[Theory]
		[InlineData("hide")]
		[InlineData("newest")]
		[InlineData("oldest")]
		public void JsonCollated_CollatedDateVisibility_ProducesValidOutput(string visibility)
		{
			_fixture.RequireTestData();
			Assert.True(RunCollatedJson($"JCDateVis_{visibility}",
				$"-collatedDateVisibility {visibility}").root.Count > 0);
		}

		// -columnSortMode: various column sorting modes
		[Theory]
		[InlineData("sortByMax")]
		[InlineData("sortByAvg")]
		public void JsonFiltered_ColumnSortMode_ProducesValidOutput(string mode)
		{
			_fixture.RequireTestData();
			Assert.True(RunFilteredJson($"JFColSort_{mode}",
				$"-columnSortMode {mode}").root.Count > 0);
		}

		// -sortTrailingDigitsAsNumeric
		[Fact]
		public void JsonFiltered_SortTrailingDigitsAsNumeric_ProducesValidOutput()
		{
			_fixture.RequireTestData();
			Assert.True(RunFilteredJson("JFNumDigitSort",
				"-sortTrailingDigitsAsNumeric").root.Count > 0);
		}

		// -customTable + -customTableSort + collated JSON + -noSummaryMinMax
		// Note: the sort column (platform) must be in -customTable for collation to work,
		// since CollateSortedTable requires the collate-by columns to be present in the filtered table.
		[Fact]
		public void JsonCollated_CustomTable_WithNoMinMax_ProducesFilteredCollatedOutput()
		{
			_fixture.RequireTestData();
			var keys = CollectAllKeys(RunCollatedJson("JCCustomCombo",
				"-customTable MVP,HitchTimePercent,Hitches/Min,platform -customTableSort platform -nosummaryminmax").root);

			Assert.False(keys.Any(k => k.StartsWith("Min ")),
				"noSummaryMinMax should suppress Min columns");
			Assert.False(keys.Any(k => k.StartsWith("Max ")),
				"noSummaryMinMax should suppress Max columns");
			Assert.True(keys.Any(k => k.Contains("MVP")),
				"Custom filter should include MVP stat");
		}

		// Full combo: -summaryTable + -summaryTableXmlAppend + -weightByColumn + -collatedStringVisibility
		[Fact]
		public void JsonCollated_FullCombo_AllArgsWorkTogether()
		{
			_fixture.RequireTestData();
			var keys = CollectAllKeys(RunCollatedJson("JCFullCombo",
				"-summaryTable condensed -summaryTableXmlAppend framecount* " +
				"-weightByColumn framecount -collatedStringVisibility hide").root);

			Assert.True(keys.Contains("Count"), "Collated output should have Count column");
			Assert.True(keys.Any(k => k.StartsWith("Avg ")), "Collated output should have Avg columns");
		}

		#endregion
	}
}
