using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.Json;
using Xunit;

namespace PerfReportTool.Tests.Integration
{
	[Trait("Category", "Integration")]
	[Collection("EndToEnd")]
	public class EndToEndTests : IClassFixture<TestDataFixture>, IDisposable
	{
		private readonly TestDataFixture _fixture;
		private readonly string _outputDir;
		private readonly string _toolPath;

		public EndToEndTests(TestDataFixture fixture)
		{
			_fixture = fixture;

			// Create a temp output directory for this test run
			_outputDir = Path.Combine(Path.GetTempPath(), "PerfReportToolTests_" + Path.GetRandomFileName());
			Directory.CreateDirectory(_outputDir);

			// Find the PerfReportTool executable
			string assemblyDir = Path.GetDirectoryName(typeof(PerfReportTool.Version).Assembly.Location);
			_toolPath = Path.Combine(assemblyDir, "PerfreportTool.dll");
		}

		public void Dispose()
		{
			if (Directory.Exists(_outputDir))
			{
				try { Directory.Delete(_outputDir, true); } catch { }
			}
		}

		private (int exitCode, string output) RunPerfReportTool(string arguments, int timeoutMs = 120000)
		{
			var psi = new ProcessStartInfo
			{
				FileName = "dotnet",
				Arguments = $"\"{_toolPath}\" {arguments}",
				RedirectStandardOutput = true,
				RedirectStandardError = true,
				UseShellExecute = false,
				CreateNoWindow = true
			};

			using var process = Process.Start(psi);
			string stdout = process.StandardOutput.ReadToEnd();
			string stderr = process.StandardError.ReadToEnd();
			if (!process.WaitForExit(timeoutMs))
			{
				try { process.Kill(); } catch { }
				return (-1, stdout + stderr + "\n[ERROR] Process timed out");
			}

			return (process.ExitCode, stdout + stderr);
		}

		/// <summary>
		/// Helper: generate PRC cache from the Perf test CSVs and return the cache dir path.
		/// </summary>
		private string GeneratePerfPrcCache(string cacheName)
		{
			string cacheDir = Path.Combine(_outputDir, cacheName);
			Directory.CreateDirectory(cacheDir);
			string outputDir = Path.Combine(_outputDir, cacheName + "_html");

			var (exitCode, output) = RunPerfReportTool(
				$"-csvdir \"{_fixture.GetPerfCsvDir()}\" -emailtable -recurse -o \"{outputDir}\" " +
				$"-nowatermarks -nodetailedreports " +
				$"-searchpattern Perf_*.csv.bin " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCacheInvalidate " +
				$"-summaryTableCache \"{cacheDir}\"",
				timeoutMs: 300000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.GetFiles(cacheDir, "*.prc").Length > 0,
				$"PRC cache generation failed — no .prc files in {cacheDir}. Output: {output.Substring(0, Math.Min(500, output.Length))}");

			return cacheDir;
		}

		[Fact]
		public void VersionOutput_PrintsVersion()
		{
			// Running with no args should print the version/help
			var (exitCode, output) = RunPerfReportTool("");
			Assert.Contains("PerfReportTool v", output);
		}

		[Fact]
		public void BulkMode_CsvDir_ProducesSummaryTable()
		{
			_fixture.RequireTestData();

			string outputDir = Path.Combine(_outputDir, "BulkCsvDir");
			string cacheDir = Path.Combine(_outputDir, "Cache1");
			Directory.CreateDirectory(cacheDir);

			string csvDir = _fixture.GetPerfCsvDir();
			var (exitCode, output) = RunPerfReportTool(
				$"-csvdir \"{csvDir}\" -emailtable -recurse -o \"{outputDir}\" " +
				$"-perflog -nowatermarks -nodetailedreports " +
				$"-searchpattern *.csv.bin " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCacheInvalidate " +
				$"-summaryTableCache \"{cacheDir}\"",
				timeoutMs: 300000);

			Assert.True(Directory.Exists(outputDir), "Output directory was not created");

			string[] htmlFiles = Directory.GetFiles(outputDir, "*.html", SearchOption.TopDirectoryOnly);
			Assert.True(htmlFiles.Length > 0, $"No HTML files generated. Output: {output.Substring(0, Math.Min(500, output.Length))}");

			string[] prcFiles = Directory.GetFiles(cacheDir, "*.prc");
			Assert.True(prcFiles.Length > 0, "No PRC cache files generated");

			Assert.DoesNotContain("[ERROR]", output);
		}

		[Fact]
		public void BulkMode_WithCache_ReusesCachedData()
		{
			_fixture.RequireTestData();

			string cacheDir = Path.Combine(_outputDir, "Cache2");
			Directory.CreateDirectory(cacheDir);
			string csvDir = _fixture.GetPerfCsvDir();

			string outputDir1 = Path.Combine(_outputDir, "WithCache_Pass1");
			RunPerfReportTool(
				$"-csvdir \"{csvDir}\" -emailtable -recurse -o \"{outputDir1}\" " +
				$"-nowatermarks -nodetailedreports " +
				$"-searchpattern *.csv.bin " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCacheInvalidate " +
				$"-summaryTableCache \"{cacheDir}\"",
				timeoutMs: 300000);

			int prcCount = Directory.GetFiles(cacheDir, "*.prc").Length;
			Assert.True(prcCount > 0, "First pass should have created PRC files");

			string outputDir2 = Path.Combine(_outputDir, "WithCache_Pass2");
			var (exitCode2, output2) = RunPerfReportTool(
				$"-csvdir \"{csvDir}\" -emailtable -recurse -o \"{outputDir2}\" " +
				$"-nowatermarks -nodetailedreports " +
				$"-searchpattern *.csv.bin " +
				$"-summaryTableCacheUseOnlyCsvID " +
				$"-summaryTableCache \"{cacheDir}\"",
				timeoutMs: 300000);

			Assert.True(Directory.Exists(outputDir2), "Second pass output directory not created");
			Assert.DoesNotContain("[ERROR]", output2);
		}

		[Fact]
		public void BulkMode_MetadataFilter_FiltersCorrectly()
		{
			_fixture.RequireTestData();

			string outputDir = Path.Combine(_outputDir, "MetadataFilter");
			string cacheDir = Path.Combine(_outputDir, "Cache3");
			Directory.CreateDirectory(cacheDir);

			string csvDir = _fixture.GetPerfCsvDir();
			string filter = "gauntletsubtest=Perf";

			var (exitCode, output) = RunPerfReportTool(
				$"-csvdir \"{csvDir}\" -emailtable -recurse -o \"{outputDir}\" " +
				$"-perflog -nodetailedreports -nowatermarks " +
				$"-metadatafilter \"{filter}\" " +
				$"-searchpattern *.csv.bin " +
				$"-summaryTableCacheUseOnlyCsvID -summaryTableCache \"{cacheDir}\"",
				timeoutMs: 300000);

			Assert.True(Directory.Exists(outputDir), "Output directory not created for metadata filter test");
			Assert.DoesNotContain("[ERROR]", output);
		}

		[Fact]
		public void BulkMode_LlmReport_ProducesOutput()
		{
			_fixture.RequireTestData();

			string outputDir = Path.Combine(_outputDir, "LLM");
			string csvDir = _fixture.GetPerfCsvDir();

			var (exitCode, output) = RunPerfReportTool(
				$"-csvdir \"{csvDir}\" -o \"{outputDir}\" -nowatermarks " +
				$"-reportxml LLMReportTypes.xml -reportType LLM " +
				$"-searchpattern LLM_*.csv.bin",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(Directory.Exists(outputDir), "LLM output directory not created");
		}

		#region JSON serialization tests

		[Fact]
		public void SummaryTableToJson_WriteAllElementData_ContainsAllTypes()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonAllTypes");

			string jsonPath = Path.Combine(_outputDir, "AllElementData.json");
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonWriteAllElementData " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(File.Exists(jsonPath), "JSON output not generated");

			// Parse and validate structure
			string jsonContent = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(jsonContent);
			Assert.NotNull(root);
			Assert.True(root.Count > 0, "JSON should contain at least one CSV entry");

			// Each CSV entry should have keys for each SummaryTableElement.Type (except COUNT and ExternalMetadata)
			string[] expectedTypeKeys = { "CsvStatAverage", "CsvMetadata", "SummaryTableMetric", "ToolMetadata", "CsvStatMin", "CsvStatMax" };

			foreach (var csvEntry in root)
			{
				var entryDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(csvEntry.Value.GetRawText());
				foreach (string typeKey in expectedTypeKeys)
				{
					Assert.True(entryDict.ContainsKey(typeKey),
						$"CSV entry '{csvEntry.Key}' missing expected type key '{typeKey}'");
				}

				// CsvMetadata should have values like platform, buildversion
				var metadata = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entryDict["CsvMetadata"].GetRawText());
				Assert.True(metadata.Count > 0, "CsvMetadata should have entries");

				// With WriteAllElementData, each element should be an object with a "value" key
				foreach (var metaEntry in metadata)
				{
					Assert.Equal(JsonValueKind.Object, metaEntry.Value.ValueKind);
					var elementDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(metaEntry.Value.GetRawText());
					Assert.True(elementDict.ContainsKey("value"), $"Element '{metaEntry.Key}' missing 'value' key");
				}

				// CsvStatAverage should have stat entries
				var statAverages = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entryDict["CsvStatAverage"].GetRawText());
				Assert.True(statAverages.Count > 0, "CsvStatAverage should have stat entries");
			}
		}

		[Fact]
		public void SummaryTableToJson_MetadataOnly_OnlyWritesCsvMetadata()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonMetaOnly");

			string jsonPath = Path.Combine(_outputDir, "MetadataOnly.json");
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonMetadataOnly " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(File.Exists(jsonPath), "JSON output not generated");

			string jsonContent = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(jsonContent);
			Assert.True(root.Count > 0, "JSON should contain at least one CSV entry");

			foreach (var csvEntry in root)
			{
				var entryDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(csvEntry.Value.GetRawText());

				// Should only contain CsvMetadata
				Assert.True(entryDict.ContainsKey("CsvMetadata"), "Missing CsvMetadata key");
				Assert.Single(entryDict); // Only CsvMetadata, nothing else

				// Should not contain stat types
				Assert.False(entryDict.ContainsKey("CsvStatAverage"), "MetadataOnly should not include CsvStatAverage");
				Assert.False(entryDict.ContainsKey("ToolMetadata"), "MetadataOnly should not include ToolMetadata");
			}
		}

		[Fact]
		public void SummaryTableToJson_SeparateFiles_ProducesMultipleFiles()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonSeparate");

			string jsonDir = Path.Combine(_outputDir, "SeparateJsonFiles");
			Directory.CreateDirectory(jsonDir);
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonSeparateFiles " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonDir}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);

			// Should produce one .json file per CSV (we have 2 Perf CSVs)
			string[] jsonFiles = Directory.GetFiles(jsonDir, "*.json");
			Assert.True(jsonFiles.Length >= 2, $"Expected at least 2 separate JSON files, got {jsonFiles.Length}");

			// Each file should be valid JSON with the per-CSV structure
			foreach (string jsonFile in jsonFiles)
			{
				string content = File.ReadAllText(jsonFile);
				var entryDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(content);
				Assert.NotNull(entryDict);
				Assert.True(entryDict.ContainsKey("CsvMetadata"), $"File {Path.GetFileName(jsonFile)} missing CsvMetadata");
			}
		}

		[Fact]
		public void SummaryTableToJson_DefaultMode_WritesValuesDirectly()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonDefault");

			// Without WriteAllElementData, values should be written directly (not as objects with "value" key)
			string jsonPath = Path.Combine(_outputDir, "Default.json");
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(File.Exists(jsonPath), "JSON output not generated");

			string jsonContent = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(jsonContent);
			Assert.NotNull(root);
			Assert.True(root.Count > 0, "JSON should contain at least one CSV entry");

			foreach (var csvEntry in root)
			{
				var entryDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(csvEntry.Value.GetRawText());
				var metadata = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entryDict["CsvMetadata"].GetRawText());

				// Without WriteAllElementData, metadata values should be direct strings/numbers, not objects
				foreach (var metaEntry in metadata)
				{
					Assert.NotEqual(JsonValueKind.Object, metaEntry.Value.ValueKind);
				}
			}
		}

		[Fact]
		public void SummaryTableToJson_NoIndent_ProducesCompactJson()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonNoIndent");

			string jsonPath = Path.Combine(_outputDir, "NoIndent.json");
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonNoIndent " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(File.Exists(jsonPath), "JSON output not generated");

			string jsonContent = File.ReadAllText(jsonPath);
			// Compact JSON should have no newlines in the content (single line)
			int lineCount = jsonContent.Split('\n').Length;
			Assert.True(lineCount <= 2, $"NoIndent JSON should be compact but has {lineCount} lines");
		}

		[Fact]
		public void SummaryTableToJson_FileStream_ProducesValidJson()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonStream");

			string jsonPath = Path.Combine(_outputDir, "Streamed.json");
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonFileStream " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);
			Assert.True(File.Exists(jsonPath), "JSON output not generated");

			// Verify it's parseable JSON
			string jsonContent = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(jsonContent);
			Assert.NotNull(root);
			Assert.True(root.Count > 0, "Streamed JSON should contain CSV entries");
		}

		[Fact]
		public void JsonToPrcs_ConvertsJsonBackToPrcFiles()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonToPrc_Src");

			// Export to JSON
			string jsonPath = Path.Combine(_outputDir, "ForPrcConvert.json");
			RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableToJsonWriteAllElementData " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{jsonPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			Assert.True(File.Exists(jsonPath), "Source JSON not generated");

			// Convert JSON back to PRCs
			string prcOutputDir = Path.Combine(_outputDir, "ConvertedPrcs");
			Directory.CreateDirectory(prcOutputDir);
			var (exitCode, output) = RunPerfReportTool(
				$"-jsonToPrcs \"{jsonPath}\" " +
				$"-summaryTableCache \"{prcOutputDir}\"",
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);

			string[] prcFiles = Directory.GetFiles(prcOutputDir, "*.prc");
			Assert.True(prcFiles.Length >= 2, $"Expected at least 2 PRC files from JSON conversion, got {prcFiles.Length}");
		}

		#endregion

		#region Filtered/Collated JSON tests (via -summaryTableOutputFormats=json)

		/// <summary>
		/// Helper: run with -summaryTableOutputFormats=json via PRC cache.
		/// For collated output, pass bCollated=true which adds -collateTableOnly.
		/// Returns parsed JSON root and raw output.
		/// </summary>
		private (Dictionary<string, JsonElement> root, string output) RunOutputFormatJsonFromCache(
			string cacheName, bool bCollated, string extraArgs = "")
		{
			string cacheDir = GeneratePerfPrcCache(cacheName);
			string jsonOutputDir = Path.Combine(_outputDir, cacheName + "_out");
			Directory.CreateDirectory(jsonOutputDir);

			string collateArg = bCollated ? "-collateTableOnly" : "";
			var (exitCode, output) = RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableOutputFormats json " +
				$"{collateArg} " +
				$"-nodetailedreports -nowatermarks " +
				$"-precacheThreads 1 " +
				$"-o \"{jsonOutputDir}\" " +
				extraArgs,
				timeoutMs: 120000);

			Assert.DoesNotContain("[ERROR]", output);

			string jsonPath = Path.Combine(jsonOutputDir, "SummaryTable.json");
			Assert.True(File.Exists(jsonPath), $"JSON not generated for {cacheName} at {jsonPath}");

			string content = File.ReadAllText(jsonPath);
			var root = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(content);
			Assert.NotNull(root);
			return (root, output);
		}

		[Fact]
		public void SummaryTableOutputFormatsJson_ProducesFlatJson()
		{
			_fixture.RequireTestData();

			var (root, _) = RunOutputFormatJsonFromCache("JsonFiltered", bCollated: false);

			Assert.True(root.Count > 0, "JSON should contain at least one entry");

			// Flat format: no type-sectioned nesting like CsvStatAverage/CsvMetadata
			foreach (var entry in root)
			{
				var rowDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entry.Value.GetRawText());
				Assert.False(rowDict.ContainsKey("CsvStatAverage"),
					"Output format JSON should use flat format, not type-sectioned format");
				Assert.False(rowDict.ContainsKey("CsvMetadata"),
					"Output format JSON should use flat format, not type-sectioned format");
			}
		}

		[Fact]
		public void SummaryTableOutputFormatsJson_OnlyIncludesFilteredStats()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonFilteredStats");

			// Unfiltered (old -summaryTableToJson path which dumps all row data)
			string unfilteredPath = Path.Combine(_outputDir, "Unfiltered.json");
			RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summarytabletojsonfastmode " +
				$"-summaryTableToJson \"{unfilteredPath}\" " +
				$"-precacheThreads 1",
				timeoutMs: 120000);

			// Filtered (via -summaryTableOutputFormats=json)
			string filteredOutputDir = Path.Combine(_outputDir, "FilteredStats_out");
			Directory.CreateDirectory(filteredOutputDir);
			RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableOutputFormats json " +
				$"-nodetailedreports -nowatermarks " +
				$"-precacheThreads 1 " +
				$"-o \"{filteredOutputDir}\"",
				timeoutMs: 120000);

			Assert.True(File.Exists(unfilteredPath), "Unfiltered JSON not generated");
			string filteredPath = Path.Combine(filteredOutputDir, "SummaryTable.json");
			Assert.True(File.Exists(filteredPath), "Filtered JSON not generated");

			string unfilteredContent = File.ReadAllText(unfilteredPath);
			string filteredContent = File.ReadAllText(filteredPath);

			// The filtered file should be smaller since it only includes stats from the summary table filter
			Assert.True(filteredContent.Length < unfilteredContent.Length,
				"Filtered JSON should contain less data than unfiltered JSON");
		}

		[Fact]
		public void SummaryTableOutputFormatsJson_Collated_ProducesCollatedFlatJson()
		{
			_fixture.RequireTestData();

			var (root, _) = RunOutputFormatJsonFromCache("JsonCollated", bCollated: true);

			Assert.True(root.Count > 0, "JSON should contain at least one collated entry");

			// Collated rows should have aggregate column names and a Count field
			bool foundAvgColumn = false;
			bool foundCountColumn = false;
			foreach (var entry in root)
			{
				var rowDict = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(entry.Value.GetRawText());
				foreach (string key in rowDict.Keys)
				{
					if (key.StartsWith("Avg ")) foundAvgColumn = true;
					if (key == "Count") foundCountColumn = true;
				}
				// Flat format, not type-sectioned
				Assert.False(rowDict.ContainsKey("CsvStatAverage"),
					"Collated JSON should use flat format, not type-sectioned format");
			}
			Assert.True(foundAvgColumn, "Collated JSON should contain Avg-prefixed columns");
			Assert.True(foundCountColumn, "Collated JSON should contain a Count column");
		}

		[Fact]
		public void SummaryTableOutputFormatsJson_Collated_FewerRowsThanNonCollated()
		{
			_fixture.RequireTestData();

			string cacheDir = GeneratePerfPrcCache("JsonCollatedRows");

			// Non-collated
			string filteredOutputDir = Path.Combine(_outputDir, "FilteredRows_out");
			Directory.CreateDirectory(filteredOutputDir);
			RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableOutputFormats json " +
				$"-nodetailedreports -nowatermarks " +
				$"-precacheThreads 1 -o \"{filteredOutputDir}\"",
				timeoutMs: 120000);

			// Collated
			string collatedOutputDir = Path.Combine(_outputDir, "CollatedRows_out");
			Directory.CreateDirectory(collatedOutputDir);
			RunPerfReportTool(
				$"-summarytablecachein \"{cacheDir}\" " +
				$"-summaryTableOutputFormats json -collateTableOnly " +
				$"-nodetailedreports -nowatermarks " +
				$"-precacheThreads 1 -o \"{collatedOutputDir}\"",
				timeoutMs: 120000);

			string filteredPath = Path.Combine(filteredOutputDir, "SummaryTable.json");
			string collatedPath = Path.Combine(collatedOutputDir, "SummaryTable.json");
			Assert.True(File.Exists(filteredPath));
			Assert.True(File.Exists(collatedPath));

			var filteredRoot = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(File.ReadAllText(filteredPath));
			var collatedRoot = JsonSerializer.Deserialize<Dictionary<string, JsonElement>>(File.ReadAllText(collatedPath));

			Assert.True(collatedRoot.Count <= filteredRoot.Count,
				$"Collated rows ({collatedRoot.Count}) should be <= filtered rows ({filteredRoot.Count})");
		}

		#endregion
	}
}
