<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="17.14.1" />
    <PackageReference Include="xunit" Version="2.9.3" />
    <PackageReference Include="xunit.runner.visualstudio" Version="2.8.2" />
  </ItemGroup>
  <ItemGroup>
    <Content Include="TestData\**" CopyToOutputDirectory="PreserveNewest" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\PerfReportTool\PerfreportTool.csproj" />
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
  </ItemGroup>
</Project>
