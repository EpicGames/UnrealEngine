using System.Collections.Generic;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class ColourTests
	{
		[Fact]
		public void Colour_FromHexString_ParsesCorrectly()
		{
			var colour = new Colour("#ff0000");
			Assert.Equal(1.0f, colour.r, 0.01f);
			Assert.Equal(0.0f, colour.g, 0.01f);
			Assert.Equal(0.0f, colour.b, 0.01f);
		}

		[Fact]
		public void Colour_FromHexString_NoHash_ParsesCorrectly()
		{
			var colour = new Colour("00ff00");
			Assert.Equal(0.0f, colour.r, 0.01f);
			Assert.Equal(1.0f, colour.g, 0.01f);
			Assert.Equal(0.0f, colour.b, 0.01f);
		}

		[Fact]
		public void Colour_ToString_ProducesHexString()
		{
			var colour = new Colour(1.0f, 0.0f, 0.0f);
			string result = colour.ToString();
			Assert.Equal("#ff0000", result);
		}

		[Fact]
		public void Colour_ToString_RoundTrips()
		{
			var original = new Colour("#3a7fc2");
			string str = original.ToString();
			var restored = new Colour(str);
			Assert.Equal(original.r, restored.r, 0.01f);
			Assert.Equal(original.g, restored.g, 0.01f);
			Assert.Equal(original.b, restored.b, 0.01f);
		}

		[Fact]
		public void Colour_Lerp_MidpointIsCorrect()
		{
			var black = new Colour(0.0f, 0.0f, 0.0f);
			var white = new Colour(1.0f, 1.0f, 1.0f);
			var mid = Colour.Lerp(black, white, 0.5f);
			Assert.Equal(0.5f, mid.r, 0.01f);
			Assert.Equal(0.5f, mid.g, 0.01f);
			Assert.Equal(0.5f, mid.b, 0.01f);
		}

		[Fact]
		public void Colour_Lerp_ZeroReturnFirst()
		{
			var a = new Colour(1.0f, 0.0f, 0.0f);
			var b = new Colour(0.0f, 1.0f, 0.0f);
			var result = Colour.Lerp(a, b, 0.0f);
			Assert.Equal(a.r, result.r, 0.001f);
			Assert.Equal(a.g, result.g, 0.001f);
			Assert.Equal(a.b, result.b, 0.001f);
		}

		[Fact]
		public void Colour_Lerp_OneReturnSecond()
		{
			var a = new Colour(1.0f, 0.0f, 0.0f);
			var b = new Colour(0.0f, 1.0f, 0.0f);
			var result = Colour.Lerp(a, b, 1.0f);
			Assert.Equal(b.r, result.r, 0.001f);
			Assert.Equal(b.g, result.g, 0.001f);
			Assert.Equal(b.b, result.b, 0.001f);
		}
	}

	public class ColourThresholdListTests
	{
		[Fact]
		public void GetColourForValue_BelowLowest_ReturnsGreen()
		{
			// Constructor params: (red=40, orange=30, yellow=20, green=10)
			// Internally stored in ascending order: green(10), yellow(20), orange(30), red(40)
			var list = new ColourThresholdList(40, 30, 20, 10);
			string colour = list.GetColourForValue(5.0);

			// Below the green threshold should return the green colour
			string expectedGreen = Colour.Green.ToHTMLString();
			Assert.Equal(expectedGreen, colour);
		}

		[Fact]
		public void GetColourForValue_AboveHighest_ReturnsRed()
		{
			var list = new ColourThresholdList(40, 30, 20, 10);
			string colour = list.GetColourForValue(50.0);

			string expectedRed = Colour.Red.ToHTMLString();
			Assert.Equal(expectedRed, colour);
		}

		[Fact]
		public void GetColourForValue_ExactThreshold_ReturnsCorrectColour()
		{
			// With lerpColours=false, exact threshold should return that threshold's colour
			var list = new ColourThresholdList(40, 30, 20, 10, lerpColours: false);
			string colourAtGreen = list.GetColourForValue(10.0);
			Assert.Equal(Colour.Green.ToHTMLString(), colourAtGreen);
		}

		[Fact]
		public void GetColourForValue_MidValue_LerpsColours()
		{
			var list = new ColourThresholdList(40, 30, 20, 10);
			string colourAt15 = list.GetColourForValue(15.0); // Between green(10) and yellow(20)

			// Should not be pure green or pure yellow when lerping
			Assert.NotEqual(Colour.Green.ToHTMLString(), colourAt15);
			Assert.NotEqual(Colour.Yellow.ToHTMLString(), colourAt15);
			Assert.NotEqual("'#ffffff'", colourAt15);
		}

		[Fact]
		public void GetColourForValue_NoLerp_SnapsToThreshold()
		{
			var list = new ColourThresholdList(40, 30, 20, 10, lerpColours: false);

			// Without lerping, values between thresholds snap to the threshold they fall within
			string colourAt15 = list.GetColourForValue(15.0); // Between green(10) and yellow(20)
			string colourAt25 = list.GetColourForValue(25.0); // Between yellow(20) and orange(30)

			// These two should be different colours (different thresholds)
			Assert.NotEqual(colourAt15, colourAt25);
			// Neither should be the lerped result or white
			Assert.NotEqual("'#ffffff'", colourAt15);
			Assert.NotEqual("'#ffffff'", colourAt25);
		}

		[Fact]
		public void GetColourForValue_EmptyList_ReturnsWhite()
		{
			var list = new ColourThresholdList();
			string colour = list.GetColourForValue(10.0);
			Assert.Equal("'#ffffff'", colour);
		}

		[Fact]
		public void GetColourForValue_StringValue_ConvertsAndReturns()
		{
			var list = new ColourThresholdList(40, 30, 20, 10);
			string colourFromString = list.GetColourForValue("25.0");
			string colourFromDouble = list.GetColourForValue(25.0);

			Assert.Equal(colourFromDouble, colourFromString);
		}

		[Fact]
		public void GetColourForValue_NonNumericString_ReturnsWhite()
		{
			var list = new ColourThresholdList(40, 30, 20, 10);
			string colour = list.GetColourForValue("not_a_number");
			Assert.Equal("'#ffffff'", colour);
		}

		[Fact]
		public void GetSafeColourForValue_NullList_ReturnsWhite()
		{
			Assert.Equal("'#ffffff'", ColourThresholdList.GetSafeColourForValue(null, 10.0));
			Assert.Equal("'#ffffff'", ColourThresholdList.GetSafeColourForValue(null, "10.0"));
		}

		[Fact]
		public void ColourThresholdList_Count_MatchesAddedThresholds()
		{
			var list = new ColourThresholdList();
			Assert.Equal(0, list.Count);

			list.Add(new ThresholdInfo(10.0, Colour.Green));
			list.Add(new ThresholdInfo(20.0, Colour.Red));
			Assert.Equal(2, list.Count);
		}

		[Fact]
		public void ColourThresholdList_ToJsonDict_ContainsThresholdValues()
		{
			var list = new ColourThresholdList(40, 30, 20, 10);
			var jsonDict = list.ToJsonDict();

			Assert.True(jsonDict.ContainsKey("thresholds"));
			Assert.True(jsonDict.ContainsKey("lerpColours"));

			var thresholds = (List<double>)jsonDict["thresholds"];
			Assert.Equal(4, thresholds.Count);
			// Constructor adds internally in ascending order: green(10), yellow(20), orange(30), red(40)
			Assert.Equal(10.0, thresholds[0]);
			Assert.Equal(20.0, thresholds[1]);
			Assert.Equal(30.0, thresholds[2]);
			Assert.Equal(40.0, thresholds[3]);
		}

		[Fact]
		public void GetThresholdColour_CustomColours_ReturnsCorrectColour()
		{
			var custom = new Colour("#112233");
			var thresholds = new List<ThresholdInfo>
			{
				new ThresholdInfo(10.0, custom),
				new ThresholdInfo(20.0, Colour.Red)
			};
			var list = new ColourThresholdList(thresholds, lerpColours: false);

			string colourBelow = list.GetColourForValue(5.0);
			Assert.Equal(custom.ToHTMLString(), colourBelow);
		}
	}
}
