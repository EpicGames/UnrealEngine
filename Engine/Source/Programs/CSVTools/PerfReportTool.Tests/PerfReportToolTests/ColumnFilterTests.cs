// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using PerfReportTool;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class StatThresholdColumnFilterTests
	{
		private static SummaryTableColumn CreateNumericColumn(string name, SummaryTableElement.Type type, double[] values)
		{
			var col = new SummaryTableColumn(name, true, name, false, type);
			for (int i = 0; i < values.Length; i++)
			{
				col.SetValue(i, values[i]);
			}
			return col;
		}

		private static SummaryTableColumn CreateStringColumn(string name, SummaryTableElement.Type type, string[] values)
		{
			var col = new SummaryTableColumn(name, false, name, false, type);
			for (int i = 0; i < values.Length; i++)
			{
				col.SetStringValue(i, values[i]);
			}
			return col;
		}

		[Fact]
		public void ShouldFilter_AllValuesAboveThreshold_ReturnsFalse()
		{
			var filter = new StatThresholdColumnFilter(10.0f);
			var col = CreateNumericColumn("stat", SummaryTableElement.Type.CsvStatAverage, new[] { 20.0, 30.0, 40.0 });
			var table = new SummaryTable();
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_SomeValuesBelowThreshold_ReturnsTrue()
		{
			var filter = new StatThresholdColumnFilter(10.0f);
			var col = CreateNumericColumn("stat", SummaryTableElement.Type.CsvStatAverage, new[] { 5.0, 30.0, 40.0 });
			var table = new SummaryTable();
			Assert.True(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_AllValuesBelowThreshold_ReturnsTrue()
		{
			var filter = new StatThresholdColumnFilter(10.0f);
			var col = CreateNumericColumn("stat", SummaryTableElement.Type.CsvStatAverage, new[] { 5.0, 3.0, 1.0 });
			var table = new SummaryTable();
			Assert.True(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_ThresholdZero_NeverFilters()
		{
			var filter = new StatThresholdColumnFilter(0.0f);
			var col = CreateNumericColumn("stat", SummaryTableElement.Type.CsvStatAverage, new[] { -5.0, 0.0 });
			var table = new SummaryTable();
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_NegativeThreshold_NeverFilters()
		{
			var filter = new StatThresholdColumnFilter(-1.0f);
			var col = CreateNumericColumn("stat", SummaryTableElement.Type.CsvStatAverage, new[] { -5.0 });
			var table = new SummaryTable();
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_NonNumericColumn_ReturnsFalse()
		{
			var filter = new StatThresholdColumnFilter(10.0f);
			var col = CreateStringColumn("meta", SummaryTableElement.Type.CsvMetadata, new[] { "hello" });
			var table = new SummaryTable();
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_MetadataType_ReturnsFalse()
		{
			// Even numeric metadata columns should not be filtered by threshold
			var filter = new StatThresholdColumnFilter(10.0f);
			var col = CreateNumericColumn("meta", SummaryTableElement.Type.CsvMetadata, new[] { 1.0 });
			var table = new SummaryTable();
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_AllFilterableTypes_FiltersWhenBelowThreshold()
		{
			var typesToTest = new[]
			{
				SummaryTableElement.Type.CsvStatAverage,
				SummaryTableElement.Type.CsvStatMin,
				SummaryTableElement.Type.CsvStatMax,
				SummaryTableElement.Type.SummaryTableMetric,
				SummaryTableElement.Type.ExternalMetadata
			};

			foreach (var type in typesToTest)
			{
				var filter = new StatThresholdColumnFilter(100.0f);
				var col = CreateNumericColumn("stat", type, new[] { 5.0 });
				var table = new SummaryTable();
				Assert.True(filter.ShouldFilter(col, table), $"Type {type} should be filtered when below threshold");
			}
		}
	}

	public class MetadataColumnFilterTests
	{
		private static SummaryTableColumn CreateColumn(string name, SummaryTableElement.Type type)
		{
			return new SummaryTableColumn(name, false, name, false, type);
		}

		[Fact]
		public void ShouldFilter_CsvMetadataColumn_ReturnsTrue()
		{
			var filter = new MetadataColumnFilter(new List<string>());
			var col = CreateColumn("platform", SummaryTableElement.Type.CsvMetadata);
			Assert.True(filter.ShouldFilter(col, new SummaryTable()));
		}

		[Fact]
		public void ShouldFilter_ToolMetadataColumn_ReturnsTrue()
		{
			var filter = new MetadataColumnFilter(new List<string>());
			var col = CreateColumn("toolversion", SummaryTableElement.Type.ToolMetadata);
			Assert.True(filter.ShouldFilter(col, new SummaryTable()));
		}

		[Fact]
		public void ShouldFilter_CsvStatColumn_ReturnsFalse()
		{
			var filter = new MetadataColumnFilter(new List<string>());
			var col = CreateColumn("frametime", SummaryTableElement.Type.CsvStatAverage);
			Assert.False(filter.ShouldFilter(col, new SummaryTable()));
		}

		[Fact]
		public void ShouldFilter_MetadataInIgnoreList_ReturnsFalse()
		{
			var filter = new MetadataColumnFilter(new List<string> { "platform" });
			var col = CreateColumn("platform", SummaryTableElement.Type.CsvMetadata);
			Assert.False(filter.ShouldFilter(col, new SummaryTable()));
		}

		[Fact]
		public void ShouldFilter_MetadataNotInIgnoreList_ReturnsTrue()
		{
			var filter = new MetadataColumnFilter(new List<string> { "platform" });
			var col = CreateColumn("buildversion", SummaryTableElement.Type.CsvMetadata);
			Assert.True(filter.ShouldFilter(col, new SummaryTable()));
		}

		[Fact]
		public void ShouldFilter_SummaryMetric_ReturnsFalse()
		{
			var filter = new MetadataColumnFilter(new List<string>());
			var col = CreateColumn("MVP", SummaryTableElement.Type.SummaryTableMetric);
			Assert.False(filter.ShouldFilter(col, new SummaryTable()));
		}
	}

	public class RegressionColumnFilterTests
	{
		private static SummaryTableColumn CreateNumericColumn(string name, double[] values, SummaryTableElement.Type type = SummaryTableElement.Type.CsvStatAverage)
		{
			var col = new SummaryTableColumn(name, true, name, false, type);
			for (int i = 0; i < values.Length; i++)
			{
				col.SetValue(i, values[i]);
			}
			return col;
		}

		private static SummaryTable CreateTableWithColumn(SummaryTableColumn column)
		{
			var table = new SummaryTable();
			var rowData = new SummaryTableRowData();
			// We need to add rows to the table to set its Count property.
			// Add multiple rows by adding row data entries that match our column values.
			for (int i = 0; i < column.GetCount(); i++)
			{
				var row = new SummaryTableRowData();
				row.Add(SummaryTableElement.Type.CsvStatAverage, column.name, column.GetValue(i));
				table.AddRowData(row, true, false, false, false);
			}
			return table;
		}

		[Fact]
		public void ShouldFilter_NonNumericColumn_ReturnsFalse()
		{
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			var col = new SummaryTableColumn("meta", false, "meta", false, SummaryTableElement.Type.CsvMetadata);
			col.SetStringValue(0, "test");
			var table = CreateTableWithColumn(CreateNumericColumn("dummy", new[] { 1.0 }));
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_EmptyTable_ReturnsTrue()
		{
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			var col = CreateNumericColumn("stat", new double[0]);
			var table = new SummaryTable();
			Assert.True(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_ClearRegression_ReturnsFalse()
		{
			// First row group (1 row) has a significantly higher value than historical data
			// Historical: ~10.0, First row: 50.0 => clear regression for HighIsBad
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			var col = CreateNumericColumn("stat", new[] { 50.0, 10.0, 10.5, 9.5, 10.0, 11.0, 9.0 });
			var table = CreateTableWithColumn(col);
			Assert.False(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_NoRegression_ReturnsTrue()
		{
			// All values are similar, no regression
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			var col = CreateNumericColumn("stat", new[] { 10.0, 10.1, 9.9, 10.0, 10.2, 9.8 });
			var table = CreateTableWithColumn(col);
			Assert.True(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_TooManyMissingValues_ReturnsTrue()
		{
			// >60% missing values should filter out
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			// Create column with mostly MaxValue (missing) entries
			var col = CreateNumericColumn("stat", new[] { 50.0, double.MaxValue, double.MaxValue, double.MaxValue, double.MaxValue, double.MaxValue, double.MaxValue, double.MaxValue, 10.0, double.MaxValue });
			var table = CreateTableWithColumn(col);
			Assert.True(filter.ShouldFilter(col, table));
		}

		[Fact]
		public void ShouldFilter_MetadataTypeColumn_ReturnsFalse()
		{
			// Metadata columns should pass through (not filtered by regression)
			var filter = new RegressionColumnFilter(null, 2.0f, 4.0f);
			var col = CreateNumericColumn("meta", new[] { 50.0, 10.0 }, SummaryTableElement.Type.CsvMetadata);
			var table = CreateTableWithColumn(CreateNumericColumn("dummy", new[] { 50.0, 10.0 }));
			Assert.False(filter.ShouldFilter(col, table));
		}
	}
}
