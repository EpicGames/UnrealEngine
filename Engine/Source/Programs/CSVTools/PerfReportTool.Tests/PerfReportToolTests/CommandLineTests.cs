// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using CSVStats;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class CommandLineParsingTests
	{
		#region IsParamName

		[Fact]
		public void IsParamName_DashPrefix_ReturnsTrue()
		{
			Assert.True(CommandLine.IsParamName("-flag"));
		}

		[Fact]
		public void IsParamName_NoDash_ReturnsFalse()
		{
			Assert.False(CommandLine.IsParamName("flag"));
		}

		[Fact]
		public void IsParamName_EmptyString_ReturnsFalse()
		{
			Assert.False(CommandLine.IsParamName(""));
		}

		[Fact]
		public void IsParamName_NegativeNumber_ReturnsFalse()
		{
			Assert.False(CommandLine.IsParamName("-123"));
		}

		[Fact]
		public void IsParamName_NegativeFloat_ReturnsFalse()
		{
			Assert.False(CommandLine.IsParamName("-3.14"));
		}

		[Fact]
		public void IsParamName_DashOnly_ReturnsTrue()
		{
			// "-" is not a valid number, so it is treated as a param
			Assert.True(CommandLine.IsParamName("-"));
		}

		#endregion

		#region Basic argument retrieval

		[Fact]
		public void GetArg_CaseInsensitive_ReturnsValue()
		{
			var cl = new CommandLine(new[] { "-MyArg", "hello" });
			Assert.Equal("hello", cl.GetArg("myarg"));
			Assert.Equal("hello", cl.GetArg("MYARG"));
			Assert.Equal("hello", cl.GetArg("MyArg"));
		}

		[Fact]
		public void GetArg_MissingArg_ReturnsEmptyString()
		{
			var cl = new CommandLine(new[] { "-foo", "bar" });
			Assert.Equal("", cl.GetArg("missing"));
		}

		[Fact]
		public void GetArg_WithDefault_ReturnDefaultWhenMissing()
		{
			var cl = new CommandLine(new[] { "-foo", "bar" });
			Assert.Equal("fallback", cl.GetArg("missing", "fallback"));
		}

		[Fact]
		public void GetArg_FlagWithoutValue_ReturnsOne()
		{
			// A flag with no value after it should default to "1"
			var cl = new CommandLine(new[] { "-verbose" });
			Assert.Equal("1", cl.GetArg("verbose"));
		}

		[Fact]
		public void GetArg_MultipleValues_JoinedBySemicolon()
		{
			// When multiple non-param values follow a param, they are joined with ";"
			var cl = new CommandLine(new[] { "-files", "a.csv", "b.csv", "c.csv" });
			string val = cl.GetArg("files");
			Assert.Equal("a.csv;b.csv;c.csv", val);
		}

		#endregion

		#region Typed argument retrieval

		[Fact]
		public void GetIntArg_ValidInt_ReturnsParsedValue()
		{
			var cl = new CommandLine(new[] { "-count", "42" });
			Assert.Equal(42, cl.GetIntArg("count", 0));
		}

		[Fact]
		public void GetIntArg_Missing_ReturnsDefault()
		{
			var cl = new CommandLine(new[] { "-other", "x" });
			Assert.Equal(99, cl.GetIntArg("count", 99));
		}

		[Fact]
		public void GetFloatArg_ValidFloat_ReturnsParsedValue()
		{
			var cl = new CommandLine(new[] { "-threshold", "3.14" });
			Assert.Equal(3.14f, cl.GetFloatArg("threshold", 0f), 0.001f);
		}

		[Fact]
		public void GetFloatArg_Missing_ReturnsDefault()
		{
			var cl = new CommandLine(new[] { "-other", "x" });
			Assert.Equal(1.5f, cl.GetFloatArg("threshold", 1.5f), 0.001f);
		}

		#endregion

		#region Boolean arguments

		[Fact]
		public void GetBoolArg_FlagPresent_ReturnsTrue()
		{
			var cl = new CommandLine(new[] { "-verbose" });
			Assert.True(cl.GetBoolArg("verbose", false));
		}

		[Fact]
		public void GetBoolArg_ExplicitZero_ReturnsFalse()
		{
			var cl = new CommandLine(new[] { "-verbose", "0" });
			Assert.False(cl.GetBoolArg("verbose", true));
		}

		[Fact]
		public void GetBoolArg_ExplicitOne_ReturnsTrue()
		{
			var cl = new CommandLine(new[] { "-verbose", "1" });
			Assert.True(cl.GetBoolArg("verbose", false));
		}

		[Fact]
		public void GetBoolArg_Missing_ReturnsDefault()
		{
			var cl = new CommandLine(new[] { "-other" });
			Assert.True(cl.GetBoolArg("verbose", true));
			Assert.False(cl.GetBoolArg("verbose", false));
		}

		[Fact]
		public void GetOptionalBoolArg_Present_ReturnsValue()
		{
			var cl = new CommandLine(new[] { "-flag", "1" });
			Assert.True(cl.GetOptionalBoolArg("flag").HasValue);
			Assert.True(cl.GetOptionalBoolArg("flag").Value);
		}

		[Fact]
		public void GetOptionalBoolArg_PresentZero_ReturnsFalse()
		{
			var cl = new CommandLine(new[] { "-flag", "0" });
			Assert.True(cl.GetOptionalBoolArg("flag").HasValue);
			Assert.False(cl.GetOptionalBoolArg("flag").Value);
		}

		[Fact]
		public void GetOptionalBoolArg_Missing_ReturnsNull()
		{
			var cl = new CommandLine(new[] { "-other" });
			Assert.Null(cl.GetOptionalBoolArg("flag"));
		}

		#endregion

		#region Negative value encoding

		[Fact]
		public void NegativeValue_MinusEncoded_DecodesCorrectly()
		{
			// The tool encodes negative values as "&minus;" to avoid them being parsed as flags
			var cl = new CommandLine(new[] { "-offset", "&minus;100" });
			Assert.Equal("-100", cl.GetArg("offset"));
			Assert.Equal(-100, cl.GetIntArg("offset", 0));
		}

		#endregion

		#region Duplicate argument handling

		[Fact]
		public void DuplicateArg_LastValueWins()
		{
			var cl = new CommandLine(new[] { "-name", "first", "-name", "second" });
			Assert.Equal("second", cl.GetArg("name"));
		}

		#endregion

		#region Response file

		[Fact]
		[Trait("Category", "FileIO")]
		public void ResponseFile_ReadsArgsFromFile()
		{
			string tempFile = Path.GetTempFileName();
			try
			{
				File.WriteAllLines(tempFile, new[]
				{
					"-csvdir /some/path",
					"-nodetailedreports",
					"-o /output"
				});

				var cl = new CommandLine(new[] { "-response", tempFile });
				Assert.Equal("/some/path", cl.GetArg("csvdir"));
				Assert.Equal("1", cl.GetArg("nodetailedreports"));
				Assert.Equal("/output", cl.GetArg("o"));
			}
			finally
			{
				File.Delete(tempFile);
			}
		}

		#endregion

		#region GetCommandLine

		[Fact]
		public void GetCommandLine_ReconstructsString()
		{
			var cl = new CommandLine(new[] { "-csv", "test.csv", "-o", "output" });
			string cmdLine = cl.GetCommandLine();
			Assert.Contains("-csv", cmdLine);
			Assert.Contains("test.csv", cmdLine);
			Assert.Contains("-o", cmdLine);
			Assert.Contains("output", cmdLine);
		}

		[Fact]
		public void GetCommandLine_QuotesArgsWithSpaces()
		{
			var cl = new CommandLine(new[] { "-csv", "path with spaces/test.csv" });
			string cmdLine = cl.GetCommandLine();
			Assert.Contains("\"path with spaces/test.csv\"", cmdLine);
		}

		#endregion
	}

	[Flags]
	public enum TestFlags
	{
		None  = 0,
		Alpha = 1,
		Beta  = 2,
		Gamma = 4,
		Delta = 8,
	}

	// Minimal subclass to expose protected CommandLineTool methods for testing
	public class TestableCommandLineTool : CommandLineTool
	{
		public TestableCommandLineTool(string[] args)
		{
			ReadCommandLine(args);
		}

		public T TestGetEnumArg<T>(string key, T defaultValue) where T : System.Enum
			=> GetEnumArg(key, defaultValue);
	}

	public class GetEnumArgTests
	{
		[Fact]
		public void SingleValue_ReturnsThatFlag()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Alpha" });
			Assert.Equal(TestFlags.Alpha, tool.TestGetEnumArg("flags", TestFlags.None));
		}

		[Fact]
		public void MultipleValues_ReturnsOrCombined()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Alpha,Gamma" });
			Assert.Equal(TestFlags.Alpha | TestFlags.Gamma, tool.TestGetEnumArg("flags", TestFlags.None));
		}

		[Fact]
		public void AllValues_ReturnsFullCombination()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Alpha,Beta,Gamma,Delta" });
			Assert.Equal(TestFlags.Alpha | TestFlags.Beta | TestFlags.Gamma | TestFlags.Delta, tool.TestGetEnumArg("flags", TestFlags.None));
		}

		[Fact]
		public void CaseInsensitive_ParsesCorrectly()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "alpha,BETA,gAmMa" });
			Assert.Equal(TestFlags.Alpha | TestFlags.Beta | TestFlags.Gamma, tool.TestGetEnumArg("flags", TestFlags.None));
		}

		[Fact]
		public void WhitespaceAroundValues_IsTrimmed()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Alpha , Beta , Gamma" });
			Assert.Equal(TestFlags.Alpha | TestFlags.Beta | TestFlags.Gamma, tool.TestGetEnumArg("flags", TestFlags.None));
		}

		[Fact]
		public void MissingArg_ReturnsDefault()
		{
			var tool = new TestableCommandLineTool(new[] { "-other", "value" });
			Assert.Equal(TestFlags.Beta, tool.TestGetEnumArg("flags", TestFlags.Beta));
		}

		[Fact]
		public void UnrecognizedValue_ReturnsDefault()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Bogus" });
			Assert.Equal(TestFlags.Beta, tool.TestGetEnumArg("flags", TestFlags.Beta));
		}

		[Fact]
		public void DuplicateValues_SameAsOnce()
		{
			var tool = new TestableCommandLineTool(new[] { "-flags", "Alpha,Alpha,Alpha" });
			Assert.Equal(TestFlags.Alpha, tool.TestGetEnumArg("flags", TestFlags.None));
		}
	}

	public class PerfLogTests
	{
		[Fact]
		public void PerfLog_LogTiming_ReturnsElapsedTime()
		{
			var log = new PerfLog(false);
			double elapsed = log.LogTiming("test");
			Assert.True(elapsed >= 0.0);
		}

		[Fact]
		public void PerfLog_LogTotalTiming_ReturnsPositive()
		{
			var log = new PerfLog(false);
			double total = log.LogTotalTiming();
			Assert.True(total >= 0.0);
		}
	}
}
