// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using CSVStats;
using PerfReportTool;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class ExternalMetadataCacheTests
	{
		[Fact]
		[Trait("Category", "FileIO")]
		public void InjectExternalMetadata_AddsMetadata()
		{
			string tempDir = Path.Combine(Path.GetTempPath(), "ExtMeta_" + Path.GetRandomFileName());
			Directory.CreateDirectory(tempDir);
			try
			{
				string metaFile = Path.Combine(tempDir, "test.txt");
				File.WriteAllLines(metaFile, new[]
				{
					"custom_key=custom_value",
					"another_key=another_value"
				});

				var cache = new ExternalMetadataCache(new List<string> { metaFile });
				var rowData = new SummaryTableRowData();
				rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test123");

				cache.InjectExternalMetadata(rowData);

				Assert.True(rowData.Contains("custom_key"), "External metadata key should be injected");
			}
			finally
			{
				Directory.Delete(tempDir, true);
			}
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void InjectExternalMetadata_DoesNotOverwriteExisting()
		{
			string tempDir = Path.Combine(Path.GetTempPath(), "ExtMeta_" + Path.GetRandomFileName());
			Directory.CreateDirectory(tempDir);
			try
			{
				string metaFile = Path.Combine(tempDir, "test.txt");
				File.WriteAllLines(metaFile, new[]
				{
					"platform=external_value"
				});

				var cache = new ExternalMetadataCache(new List<string> { metaFile });
				var rowData = new SummaryTableRowData();
				rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "original_value");
				rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test123");

				cache.InjectExternalMetadata(rowData);

				// The existing value should not be overwritten
				Assert.Equal("original_value", rowData.Get("platform").value);
			}
			finally
			{
				Directory.Delete(tempDir, true);
			}
		}
	}
}
