using PerfReportTool;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class HashHelperTests
	{
		[Fact]
		public void StringToHashStr_SameInput_ProducesSameOutput()
		{
			string input = "test string for hashing";
			string hash1 = HashHelper.StringToHashStr(input);
			string hash2 = HashHelper.StringToHashStr(input);
			Assert.Equal(hash1, hash2);
		}

		[Fact]
		public void StringToHashStr_DifferentInput_ProducesDifferentOutput()
		{
			string hash1 = HashHelper.StringToHashStr("input A");
			string hash2 = HashHelper.StringToHashStr("input B");
			Assert.NotEqual(hash1, hash2);
		}

		[Fact]
		public void StringToHashStr_ReturnsHexString()
		{
			string hash = HashHelper.StringToHashStr("test");
			// SHA256 produces 32 bytes = 64 hex characters
			Assert.Equal(64, hash.Length);
			Assert.Matches("^[0-9A-Fa-f]+$", hash);
		}

		[Fact]
		public void StringToHashStr_MaxCharsOut_TruncatesCorrectly()
		{
			string hashFull = HashHelper.StringToHashStr("test");
			string hashTruncated = HashHelper.StringToHashStr("test", 16);
			Assert.Equal(16, hashTruncated.Length);
			Assert.Equal(hashFull.Substring(0, 16), hashTruncated);
		}

		[Fact]
		public void StringToHashStr_EmptyString_DoesNotThrow()
		{
			string hash = HashHelper.StringToHashStr("");
			Assert.NotNull(hash);
			Assert.NotEmpty(hash);
		}
	}
}
