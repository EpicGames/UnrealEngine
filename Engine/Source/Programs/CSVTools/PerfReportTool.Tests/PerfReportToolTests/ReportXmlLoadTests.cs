using System.IO;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	[Trait("Category", "FileIO")]
	public class ReportXmlLoadTests
	{
		[Fact]
		public void ReportTypesXml_Exists_InOutputDir()
		{
			string path = ReportXmlTestHelper.GetReportTypesXmlPath();
			Assert.True(File.Exists(path), $"ReportTypes.xml not found at {path}");
		}

		[Fact]
		public void ReportGraphsXml_Exists_InOutputDir()
		{
			string path = ReportXmlTestHelper.GetReportGraphsXmlPath();
			Assert.True(File.Exists(path), $"ReportGraphs.xml not found at {path}");
		}

		[Fact]
		public void LlmReportTypesXml_Exists_InOutputDir()
		{
			string path = ReportXmlTestHelper.GetLlmReportTypesXmlPath();
			Assert.True(File.Exists(path), $"LLMReportTypes.xml not found at {path}");
		}

		[Fact]
		public void LlmReportGraphsXml_Exists_InOutputDir()
		{
			string path = ReportXmlTestHelper.GetLlmReportGraphsXmlPath();
			Assert.True(File.Exists(path), $"LLMReportGraphs.xml not found at {path}");
		}

		[Fact]
		public void ReportTypesXml_ContainsReportTypeDefinitions()
		{
			string path = ReportXmlTestHelper.GetReportTypesXmlPath();
			Assert.True(File.Exists(path), $"ReportTypes.xml not found at {path}");

			var doc = XDocument.Load(path);
			var root = doc.Root;
			Assert.NotNull(root);

			// Should contain reporttype elements defining how CSVs are processed
			var ns = root.GetDefaultNamespace();
			var reportTypes = root.Descendants().Where(e => e.Name.LocalName == "reporttype").ToList();
			Assert.True(reportTypes.Count > 0, "ReportTypes.xml should contain at least one <reporttype> element");

			// Each reporttype should have a name
			foreach (var rt in reportTypes)
			{
				var nameAttr = rt.Attribute("name");
				Assert.NotNull(nameAttr);
				Assert.False(string.IsNullOrEmpty(nameAttr.Value), "reporttype should have a non-empty name attribute");
			}
		}

		[Fact]
		public void ReportGraphsXml_ContainsGraphDefinitions()
		{
			string path = ReportXmlTestHelper.GetReportGraphsXmlPath();
			Assert.True(File.Exists(path), $"ReportGraphs.xml not found at {path}");

			var doc = XDocument.Load(path);
			var root = doc.Root;
			Assert.NotNull(root);

			// Should contain graph group elements
			var graphGroups = root.Descendants().Where(e =>
				e.Name.LocalName == "graphGroup" || e.Name.LocalName == "graph").ToList();
			Assert.True(graphGroups.Count > 0, "ReportGraphs.xml should contain graph definitions");
		}
	}
}
