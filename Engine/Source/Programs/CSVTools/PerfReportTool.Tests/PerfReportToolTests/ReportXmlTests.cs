// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Xml.Linq;
using PerfReportTool;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class XmlVariableMappingsExtendedTests
	{
		[Fact]
		public void SetVariable_StoresAndRetrieves()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("testVar", "hello");
			string result = vars.ResolveVariables("prefix_${testVar}_suffix");
			Assert.Equal("prefix_hello_suffix", result);
		}

		[Fact]
		public void ResolveVariables_MultipleVariables_AllResolved()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("a", "1");
			vars.SetVariable("b", "2");
			string result = vars.ResolveVariables("${a}+${b}");
			Assert.Equal("1+2", result);
		}

		[Fact]
		public void ResolveVariables_MissingVariable_ReplacedWithEmpty()
		{
			var vars = new XmlVariableMappings();
			string result = vars.ResolveVariables("prefix_${missing}_suffix");
			Assert.Equal("prefix__suffix", result);
		}

		[Fact]
		public void ResolveVariables_NoVariables_ReturnsOriginal()
		{
			var vars = new XmlVariableMappings();
			string result = vars.ResolveVariables("no variables here");
			Assert.Equal("no variables here", result);
		}

		[Fact]
		public void ResolveVariables_ArrayIndexing_ReturnsElement()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("list", "alpha,beta,gamma");
			string result = vars.ResolveVariables("${list[1]}");
			Assert.Equal("beta", result);
		}

		[Fact]
		public void ResolveVariables_ArrayIndexZero_ReturnsFirst()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("items", "first,second,third");
			string result = vars.ResolveVariables("${items[0]}");
			Assert.Equal("first", result);
		}

		[Fact]
		public void SetMetadataVariables_SetsMetaPrefix()
		{
			var vars = new XmlVariableMappings();
			var metadata = new CSVStats.CsvMetadata();
			metadata.Values["platform"] = "windows";
			metadata.Values["buildversion"] = "5.4";

			vars.SetMetadataVariables(metadata);

			string result = vars.ResolveVariables("${meta.platform}");
			Assert.Equal("windows", result);
		}

		[Fact]
		public void ApplyVariableSet_ParsesVarElements()
		{
			// New format: <var name="name">value</var> (value as element text)
			string xml = @"<variableSet>
				<var name='targetFps'>30</var>
				<var name='hitchMs'>150</var>
			</variableSet>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			vars.ApplyVariableSet(element, null);

			Assert.Equal("30", vars.ResolveVariables("${targetFps}"));
			Assert.Equal("150", vars.ResolveVariables("${hitchMs}"));
		}

		[Fact]
		public void ApplyVariableSet_WithMultiplier_ScalesNumericValues()
		{
			string xml = @"<variableSet multiplier='2.0'>
				<var name='budget'>33.33</var>
			</variableSet>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			vars.ApplyVariableSet(element, null, 1.0);

			string result = vars.ResolveVariables("${budget}");
			double parsed = double.Parse(result, System.Globalization.CultureInfo.InvariantCulture);
			Assert.Equal(66.66, parsed, 0.01);
		}
	}

	[Collection("SummaryFactory")]
	public class ReportTypeInfoTests
	{
		// FPSChartSummary has the (XElement, XmlVariableMappings, string) constructor that SummaryFactory.Create requires
		private const string FpsChartSummaryXml = "<summary type='fpschart' fps='30' hitchThreshold='150'><stats/></summary>";

		[Fact]
		public void GetSummaryTableCacheID_ReturnsNonEmptyString()
		{
			string xml = $"<reporttype name='test' title='Test Report'>{FpsChartSummaryXml}</reporttype>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			SummaryFactory.Init();
			var reportType = new ReportTypeInfo(element, new Dictionary<string, XElement>(), "", vars, null);

			string cacheId = reportType.GetSummaryTableCacheID();
			Assert.False(string.IsNullOrEmpty(cacheId));
		}

		[Fact]
		public void Constructor_ParsesNameAndTitle()
		{
			string xml = $"<reporttype name='flythrough' title='Flythrough Report'>{FpsChartSummaryXml}</reporttype>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			SummaryFactory.Init();
			var reportType = new ReportTypeInfo(element, new Dictionary<string, XElement>(), "", vars, null);

			Assert.Equal("flythrough", reportType.name);
			Assert.Equal("Flythrough Report", reportType.title);
		}

		[Fact]
		public void Constructor_ParsesStripEvents()
		{
			string xml = $"<reporttype name='test' title='Test' stripEvents='true'>{FpsChartSummaryXml}</reporttype>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			SummaryFactory.Init();
			var reportType = new ReportTypeInfo(element, new Dictionary<string, XElement>(), "", vars, null);

			Assert.True(reportType.bStripEvents);
		}

		[Fact]
		public void Constructor_ParsesMetadataToShow()
		{
			string xml = $@"<reporttype name='test' title='Test'>
				<metadataToShow>platform,buildversion,deviceprofile</metadataToShow>
				{FpsChartSummaryXml}
			</reporttype>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			SummaryFactory.Init();
			var reportType = new ReportTypeInfo(element, new Dictionary<string, XElement>(), "", vars, null);

			Assert.NotNull(reportType.metadataToShowList);
			Assert.Equal(3, reportType.metadataToShowList.Length);
			Assert.Contains("platform", reportType.metadataToShowList);
		}

		[Fact]
		public void Constructor_CreatesSummaries()
		{
			string xml = $"<reporttype name='test' title='Test'>{FpsChartSummaryXml}</reporttype>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();

			SummaryFactory.Init();
			var reportType = new ReportTypeInfo(element, new Dictionary<string, XElement>(), "", vars, null);

			Assert.True(reportType.summaries.Count > 0, "Should have at least one summary");
		}

		[Fact]
		public void GetSummaryTableCacheID_DifferentReportTypes_DifferentIDs()
		{
			SummaryFactory.Init();
			var vars = new XmlVariableMappings();

			string xml1 = $"<reporttype name='type1' title='Type 1'>{FpsChartSummaryXml}</reporttype>";
			string xml2 = $"<reporttype name='type2' title='Type 2'>{FpsChartSummaryXml}</reporttype>";

			var rt1 = new ReportTypeInfo(XElement.Parse(xml1), new Dictionary<string, XElement>(), "", vars, null);
			var rt2 = new ReportTypeInfo(XElement.Parse(xml2), new Dictionary<string, XElement>(), "", vars, null);

			Assert.NotEqual(rt1.GetSummaryTableCacheID(), rt2.GetSummaryTableCacheID());
		}
	}

	public class SummaryTableInfoTests
	{
		[Fact]
		public void Constructor_ParsesFilterAndRowSort()
		{
			var info = new SummaryTableInfo("frametime,drawcalls,platform", "buildversion,platform");
			Assert.Contains("frametime", info.columnFilterList);
			Assert.Contains("drawcalls", info.columnFilterList);
			Assert.Contains("platform", info.columnFilterList);
			Assert.Contains("buildversion", info.rowSortList);
			Assert.Contains("platform", info.rowSortList);
		}

		[Fact]
		public void DefaultValues_AreCorrect()
		{
			var info = new SummaryTableInfo();
			Assert.Equal(TableColorizeMode.Budget, info.tableColorizeMode);
			Assert.Equal(DateCollationVisibility.Newest, info.dateCollationVisibility);
			Assert.Equal(StringCollationVisibility.Auto, info.stringCollationVisibility);
			Assert.Equal(TableColumnSortMode.Default, info.columnSortMode);
			Assert.False(info.bReverseSortRows);
			Assert.False(info.bScrollableFormatting);
		}
	}
}
