using System.Collections.Generic;
using System.Reflection;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	[Collection("SummaryFactory")]
	public class SummaryFactoryTests
	{
		public SummaryFactoryTests()
		{
			SummaryFactory.Init();
		}

		private static Dictionary<string, System.Type> GetRegisteredTypes()
		{
			// Access the private static summaryNameLookup field via reflection
			var field = typeof(SummaryFactory).GetField("summaryNameLookup", BindingFlags.NonPublic | BindingFlags.Static);
			Assert.NotNull(field);
			return (Dictionary<string, System.Type>)field.GetValue(null);
		}

		[Theory]
		[InlineData("fpschart")]
		[InlineData("hitches")]
		[InlineData("histogram")]
		[InlineData("event")]
		[InlineData("peak")]
		[InlineData("bucketsummary")]
		[InlineData("boundedstatvalues")]
		[InlineData("checkpoint")]
		[InlineData("mapoverlay")]
		[InlineData("statBudgets")]
		[InlineData("timingregion")]
		[InlineData("standard")]
		[InlineData("extralinks")]
		public void Init_RegistersKnownSummaryType(string summaryName)
		{
			var registry = GetRegisteredTypes();
			Assert.True(registry.ContainsKey(summaryName), $"Summary type '{summaryName}' was not registered in SummaryFactory");
			Assert.True(registry[summaryName].IsSubclassOf(typeof(Summary)),
				$"Registered type for '{summaryName}' should be a Summary subclass");
		}

		[Fact]
		public void Init_RegistersAtLeast13Types()
		{
			var registry = GetRegisteredTypes();
			Assert.True(registry.Count >= 13, $"Expected at least 13 summary types but found {registry.Count}");
		}

		[Fact]
		public void Create_UnknownType_ThrowsException()
		{
			var ex = Assert.Throws<System.Exception>(() =>
				SummaryFactory.Create("NonExistentSummaryType", null, null, ""));
			Assert.Contains("not found", ex.Message);
		}
	}
}
