using System.IO;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class SummaryTableElementTests
	{
		[Fact]
		public void NumericConstructor_SetsProperties()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage,
				"TestStat",
				42.5,
				null,
				"tooltip text");

			Assert.Equal(SummaryTableElement.Type.CsvStatAverage, element.type);
			Assert.Equal("TestStat", element.name);
			Assert.True(element.isNumeric);
			Assert.Equal(42.5, element.numericValue);
			Assert.Equal("42.5", element.value);
			Assert.Equal("tooltip text", element.tooltip);
		}

		[Fact]
		public void StringConstructor_SetsProperties()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvMetadata,
				"platform",
				"windows",
				null,
				"");

			Assert.Equal(SummaryTableElement.Type.CsvMetadata, element.type);
			Assert.Equal("platform", element.name);
			Assert.False(element.isNumeric);
			Assert.Equal("windows", element.value);
		}

		[Fact]
		public void Constructor_EmptyName_ThrowsException()
		{
			Assert.Throws<System.Exception>(() =>
				new SummaryTableElement(SummaryTableElement.Type.CsvStatAverage, "", 0.0, null, ""));
		}

		[Fact]
		public void CacheRoundTrip_NumericElement_PreservesData()
		{
			var thresholds = new ColourThresholdList(40, 30, 20, 10);
			var original = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage,
				"frametime",
				16.67,
				thresholds,
				"Frame time in ms",
				(uint)SummaryTableElement.Flags.Hidden);

			using var stream = new MemoryStream();
			using (var writer = new BinaryWriter(stream, System.Text.Encoding.UTF8, leaveOpen: true))
			{
				original.WriteToCache(writer);
			}

			stream.Position = 0;
			using var reader = new BinaryReader(stream);
			var restored = SummaryTableElement.ReadFromCache(reader);

			Assert.Equal(original.type, restored.type);
			Assert.Equal(original.name, restored.name);
			Assert.Equal(original.value, restored.value);
			Assert.Equal(original.tooltip, restored.tooltip);
			Assert.Equal(original.numericValue, restored.numericValue);
			Assert.Equal(original.isNumeric, restored.isNumeric);
			Assert.Equal(original.flags, restored.flags);
			Assert.NotNull(restored.colorThresholdList);
			Assert.Equal(original.colorThresholdList.Count, restored.colorThresholdList.Count);
			for (int i = 0; i < original.colorThresholdList.Count; i++)
			{
				Assert.Equal(original.colorThresholdList.Thresholds[i].value, restored.colorThresholdList.Thresholds[i].value);
			}
		}

		[Fact]
		public void CacheRoundTrip_StringElement_PreservesData()
		{
			var original = new SummaryTableElement(
				SummaryTableElement.Type.CsvMetadata,
				"buildversion",
				"++UE5+Main-CL-12345",
				null,
				"");

			using var stream = new MemoryStream();
			using (var writer = new BinaryWriter(stream, System.Text.Encoding.UTF8, leaveOpen: true))
			{
				original.WriteToCache(writer);
			}

			stream.Position = 0;
			using var reader = new BinaryReader(stream);
			var restored = SummaryTableElement.ReadFromCache(reader);

			Assert.Equal(original.type, restored.type);
			Assert.Equal(original.name, restored.name);
			Assert.Equal(original.value, restored.value);
			Assert.False(restored.isNumeric);
			Assert.Null(restored.colorThresholdList);
		}

		[Fact]
		public void SetFlag_GetFlag_WorkCorrectly()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "test", 1.0, null, "");

			Assert.False(element.GetFlag(SummaryTableElement.Flags.Hidden));

			element.SetFlag(SummaryTableElement.Flags.Hidden, true);
			Assert.True(element.GetFlag(SummaryTableElement.Flags.Hidden));

			element.SetFlag(SummaryTableElement.Flags.Hidden, false);
			Assert.False(element.GetFlag(SummaryTableElement.Flags.Hidden));
		}

		[Fact]
		public void ToJsonDict_NumericValue_SerializesCorrectly()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67, null, "");

			var dict = element.ToJsonDict(true);

			Assert.Equal("CsvStatAverage", dict["type"]);
			Assert.IsType<decimal>(dict["value"]);
			Assert.Equal(16.67m, (decimal)dict["value"], 2);
		}

		[Fact]
		public void ToJsonDict_StringValue_SerializesCorrectly()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvMetadata, "platform", "windows", null, "");

			var dict = element.ToJsonDict(true);

			Assert.Equal("CsvMetadata", dict["type"]);
			Assert.Equal("windows", dict["value"]);
		}

		[Fact]
		public void ToJsonDict_WithTooltip_IncludesIt()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "stat", 1.0, null, "my tooltip");

			var dict = element.ToJsonDict(false);
			Assert.True(dict.ContainsKey("tooltip"));
			Assert.Equal("my tooltip", dict["tooltip"]);
		}

		[Fact]
		public void ToJsonDict_EmptyTooltip_ExcludesIt()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "stat", 1.0, null, "");

			var dict = element.ToJsonDict(false);
			Assert.False(dict.ContainsKey("tooltip"));
		}

		[Fact]
		public void ToJsonDict_WithFlags_IncludesFlagStrings()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "stat", 1.0, null, "",
				(uint)SummaryTableElement.Flags.Hidden);

			var dict = element.ToJsonDict(false);
			Assert.True(dict.ContainsKey("flags"));
		}

		[Fact]
		public void Clone_ProducesIndependentCopy()
		{
			var original = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "stat", 42.0, null, "tip");
			var clone = original.Clone();

			Assert.Equal(original.name, clone.name);
			Assert.Equal(original.numericValue, clone.numericValue);
			Assert.NotSame(original, clone);
		}

		[Fact]
		public void DynamicValue_Numeric_ReturnsDouble()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvStatAverage, "stat", 42.5, null, "");

			dynamic val = element.DynamicValue;
			Assert.IsType<double>(val);
			Assert.Equal(42.5, (double)val);
		}

		[Fact]
		public void DynamicValue_String_ReturnsString()
		{
			var element = new SummaryTableElement(
				SummaryTableElement.Type.CsvMetadata, "platform", "windows", null, "");

			dynamic val = element.DynamicValue;
			Assert.IsType<string>(val);
			Assert.Equal("windows", (string)val);
		}
	}
}
