using System.Collections.Generic;
using System.IO;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class SummaryTableRowDataTests
	{
		[Fact]
		public void Add_NumericEntry_CanBeRetrieved()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);

			Assert.True(rowData.Contains("frametime"));
			var element = rowData.Get("frametime");
			Assert.NotNull(element);
			Assert.Equal(16.67, element.numericValue);
			Assert.True(element.isNumeric);
		}

		[Fact]
		public void Add_StringEntry_CanBeRetrieved()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");

			Assert.True(rowData.Contains("platform"));
			var element = rowData.Get("platform");
			Assert.NotNull(element);
			Assert.Equal("windows", element.value);
		}

		[Fact]
		public void Add_KeyIsCaseInsensitive()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "FrameTime", 16.67);

			// Keys are lowercased on add
			Assert.True(rowData.Contains("frametime"));
			Assert.NotNull(rowData.Get("frametime"));
		}

		[Fact]
		public void Add_DuplicateKey_DoesNotThrow()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "stat", 1.0);
			// Should not throw, just warn
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "stat", 2.0);
		}

		[Fact]
		public void Add_StringNumericValue_ParsedAsNumeric()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "stat", "42.5");

			var element = rowData.Get("stat");
			Assert.True(element.isNumeric);
			Assert.Equal(42.5, element.numericValue);
		}

		[Fact]
		public void Add_CsvId_TreatedAsString()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "12345");

			var element = rowData.Get("csvid");
			// csvid should always be string even if it looks numeric
			Assert.False(element.isNumeric);
			Assert.Equal("12345", element.value);
		}

		[Fact]
		public void GetFrameCount_ReturnsCorrectCount()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", 1000.0);

			Assert.Equal(1000, rowData.GetFrameCount());
		}

		[Fact]
		public void GetFrameCount_NoEntry_ReturnsZero()
		{
			var rowData = new SummaryTableRowData();
			Assert.Equal(0, rowData.GetFrameCount());
		}

		[Fact]
		public void Get_MissingKey_ReturnsNull()
		{
			var rowData = new SummaryTableRowData();
			Assert.Null(rowData.Get("nonexistent"));
		}

		[Fact]
		public void RemoveSafe_ExistingKey_Removes()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "stat", 1.0);
			Assert.True(rowData.Contains("stat"));

			rowData.RemoveSafe("stat");
			Assert.False(rowData.Contains("stat"));
		}

		[Fact]
		public void RemoveSafe_MissingKey_DoesNotThrow()
		{
			var rowData = new SummaryTableRowData();
			rowData.RemoveSafe("nonexistent");
		}

		[Fact]
		public void ReadCsvMetadata_ExtractsOnlyMetadata()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "buildversion", "++UE5+Main-CL-12345");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);
			rowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", 1000.0);

			var metadata = rowData.ReadCsvMetadata();
			Assert.NotNull(metadata);
			Assert.Equal(2, metadata.Values.Count);
			Assert.Equal("windows", metadata.Values["platform"]);
			Assert.Equal("++UE5+Main-CL-12345", metadata.Values["buildversion"]);
		}

		[Fact]
		public void ReadCsvMetadata_NoMetadata_ReturnsNull()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);

			var metadata = rowData.ReadCsvMetadata();
			Assert.Null(metadata);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void CacheRoundTrip_PreservesAllData()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test-csv-id");
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);
			rowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", 500.0);

			string tempDir = Path.Combine(Path.GetTempPath(), "PerfReportToolTests_" + Path.GetRandomFileName());
			Directory.CreateDirectory(tempDir);
			try
			{
				bool writeSuccess = rowData.WriteToCache(tempDir, "test-csv-id");
				Assert.True(writeSuccess);

				var restored = SummaryTableRowData.TryReadFromCache(tempDir, "test-csv-id");
				Assert.NotNull(restored);
				Assert.Equal(rowData.dict.Count, restored.dict.Count);

				foreach (var key in rowData.dict.Keys)
				{
					Assert.True(restored.Contains(key));
					Assert.Equal(rowData.Get(key).name, restored.Get(key).name);
					Assert.Equal(rowData.Get(key).type, restored.Get(key).type);
					Assert.Equal(rowData.Get(key).numericValue, restored.Get(key).numericValue);
					Assert.Equal(rowData.Get(key).value, restored.Get(key).value);
				}
			}
			finally
			{
				Directory.Delete(tempDir, true);
			}
		}

		[Fact]
		public void TryReadFromCache_MissingFile_ReturnsNull()
		{
			var result = SummaryTableRowData.TryReadFromCache(Path.GetTempPath(), "nonexistent-id");
			Assert.Null(result);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void CacheRoundTrip_ExcludesExternalMetadata()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test-id");
			rowData.Add(SummaryTableElement.Type.ExternalMetadata, "external", "value");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "stat", 1.0);

			string tempDir = Path.Combine(Path.GetTempPath(), "PerfReportToolTests_" + Path.GetRandomFileName());
			Directory.CreateDirectory(tempDir);
			try
			{
				rowData.WriteToCache(tempDir, "test-id");
				var restored = SummaryTableRowData.TryReadFromCache(tempDir, "test-id");

				Assert.NotNull(restored);
				// External metadata should not be in the cache
				Assert.False(restored.Contains("external"));
				Assert.True(restored.Contains("csvid"));
				Assert.True(restored.Contains("stat"));
			}
			finally
			{
				Directory.Delete(tempDir, true);
			}
		}

		[Fact]
		public void ToJsonDict_MetadataOnly_FiltersCorrectly()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test-id");
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);

			var jsonDict = rowData.ToJsonDict(bCsvMetadataOnly: true, bWriteAllElementData: false);

			// Should only have CsvMetadata key
			Assert.Single(jsonDict);
			Assert.True(jsonDict.ContainsKey("CsvMetadata"));
			var metaDict = (Dictionary<string, dynamic>)jsonDict["CsvMetadata"];
			Assert.Equal(2, metaDict.Count);
		}

		[Fact]
		public void ToJsonDict_AllTypes_IncludesAllCategories()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "test-id");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);
			rowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", 500.0);

			var jsonDict = rowData.ToJsonDict(bCsvMetadataOnly: false, bWriteAllElementData: false);

			// Should have entries for each non-COUNT, non-External type
			Assert.True(jsonDict.ContainsKey("CsvMetadata"));
			Assert.True(jsonDict.ContainsKey("CsvStatAverage"));
			Assert.True(jsonDict.ContainsKey("ToolMetadata"));
			Assert.False(jsonDict.ContainsKey("ExternalMetadata"));
		}

		[Fact]
		public void JsonRoundTrip_PreservesData()
		{
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "json-test");
			rowData.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			rowData.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.67);
			rowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", 800.0);

			var jsonDict = rowData.ToJsonDict(bCsvMetadataOnly: false, bWriteAllElementData: true);
			var restored = new SummaryTableRowData(jsonDict);

			Assert.True(restored.Contains("csvid"));
			Assert.True(restored.Contains("platform"));
			Assert.True(restored.Contains("frametime"));
			Assert.True(restored.Contains("framecount"));

			Assert.Equal("json-test", restored.Get("csvid").value);
			Assert.Equal("windows", restored.Get("platform").value);
			Assert.Equal(16.67, restored.Get("frametime").numericValue, 2);
		}
	}
}
