// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using PerfReportTool;
using PerfSummaries;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class SummaryTableColumnTests
	{
		#region Construction and basic operations

		[Fact]
		public void Constructor_SetsNameWithAggregatePrefix()
		{
			var col = new SummaryTableColumn("frametime", true, "FrameTime", false, SummaryTableElement.Type.CsvStatAverage,
				inAggregateType: ColumnAggregateType.Avg);
			Assert.Equal("Avg frametime", col.name);
		}

		[Fact]
		public void Constructor_NoAggregate_NoPrefix()
		{
			var col = new SummaryTableColumn("frametime", true, "FrameTime", false, SummaryTableElement.Type.CsvStatAverage);
			Assert.Equal("frametime", col.name);
		}

		[Fact]
		public void GetAggregateTypePrefix_ReturnsCorrectPrefixes()
		{
			Assert.Equal("Avg ", SummaryTableColumn.getAggregateTypePrefix(ColumnAggregateType.Avg));
			Assert.Equal("Min ", SummaryTableColumn.getAggregateTypePrefix(ColumnAggregateType.Min));
			Assert.Equal("Max ", SummaryTableColumn.getAggregateTypePrefix(ColumnAggregateType.Max));
			Assert.Equal("", SummaryTableColumn.getAggregateTypePrefix(ColumnAggregateType.None));
		}

		#endregion

		#region Numeric value operations

		[Fact]
		public void SetAndGetValue_RoundTrips()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 42.5);
			col.SetValue(1, 99.0);
			Assert.Equal(42.5, col.GetValue(0));
			Assert.Equal(99.0, col.GetValue(1));
		}

		[Fact]
		public void GetValue_BeyondSize_ReturnsMaxValue()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			Assert.Equal(double.MaxValue, col.GetValue(5));
		}

		[Fact]
		public void GetMaxValue_IgnoresMaxValueSentinels()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			col.SetValue(1, 30.0);
			col.SetValue(2, 20.0);
			Assert.Equal(30.0, col.GetMaxValue());
		}

		[Fact]
		public void GetMinValue_IgnoresMaxValueSentinels()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			col.SetValue(1, 30.0);
			col.SetValue(2, 20.0);
			Assert.Equal(10.0, col.GetMinValue());
		}

		[Fact]
		public void GetAvgValue_IgnoresMaxValueSentinels()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			col.SetValue(1, 20.0);
			col.SetValue(2, 30.0);
			Assert.Equal(20.0, col.GetAvgValue(), 0.001);
		}

		[Fact]
		public void GetMaxValue_NoValues_ReturnsZero()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			Assert.Equal(0.0, col.GetMaxValue());
		}

		[Fact]
		public void GetMinValue_NoValues_ReturnsZero()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			Assert.Equal(0.0, col.GetMinValue());
		}

		[Fact]
		public void AreAllValuesOverThreshold_AllOver_ReturnsTrue()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 20.0);
			col.SetValue(1, 30.0);
			Assert.True(col.AreAllValuesOverThreshold(10.0));
		}

		[Fact]
		public void AreAllValuesOverThreshold_SomeBelowButOneAbove_ReturnsFalse()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 5.0);
			col.SetValue(1, 30.0);
			Assert.False(col.AreAllValuesOverThreshold(10.0));
		}

		[Fact]
		public void AreAllValuesOverThreshold_AllBelow_ReturnsFalse()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 5.0);
			col.SetValue(1, 8.0);
			Assert.False(col.AreAllValuesOverThreshold(10.0));
		}

		#endregion

		#region String value operations

		[Fact]
		public void SetAndGetStringValue_RoundTrips()
		{
			var col = new SummaryTableColumn("meta", false, null, false, SummaryTableElement.Type.CsvMetadata);
			col.SetStringValue(0, "hello");
			col.SetStringValue(1, "world");
			Assert.Equal("hello", col.GetStringValue(0));
			Assert.Equal("world", col.GetStringValue(1));
		}

		[Fact]
		public void GetCount_ReturnsMaxOfNumericAndString()
		{
			var col = new SummaryTableColumn("stat", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			col.SetValue(1, 20.0);
			col.SetValue(2, 30.0);
			Assert.Equal(3, col.GetCount());
		}

		#endregion

		#region GetStatCategory

		[Fact]
		public void GetStatCategory_CsvStatWithSlash_ReturnsCategory()
		{
			var col = new SummaryTableColumn("render/drawcalls", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			Assert.Equal("render", col.GetStatCategory());
		}

		[Fact]
		public void GetStatCategory_CsvStatWithoutSlash_ReturnsEmpty()
		{
			var col = new SummaryTableColumn("frametime", true, null, false, SummaryTableElement.Type.CsvStatAverage);
			Assert.Equal("", col.GetStatCategory());
		}

		[Fact]
		public void GetStatCategory_NonCsvStat_ReturnsEmpty()
		{
			var col = new SummaryTableColumn("render/drawcalls", true, null, false, SummaryTableElement.Type.SummaryTableMetric);
			Assert.Equal("", col.GetStatCategory());
		}

		#endregion

		#region Clone

		[Fact]
		public void Clone_ProducesIndependentCopy()
		{
			var col = new SummaryTableColumn("stat", true, "display", false, SummaryTableElement.Type.CsvStatAverage);
			col.SetValue(0, 10.0);
			col.SetValue(1, 20.0);

			var clone = col.Clone();
			Assert.Equal("stat", clone.name);
			Assert.Equal(10.0, clone.GetValue(0));
			Assert.Equal(20.0, clone.GetValue(1));

			// Modify original, verify clone is unaffected
			col.SetValue(0, 999.0);
			Assert.Equal(10.0, clone.GetValue(0));
		}

		#endregion
	}

	public class SummaryTableStaticTests
	{
		#region Element type prefix mapping

		[Fact]
		public void GetElementTypeStatPrefix_ReturnsCorrectPrefixes()
		{
			Assert.Equal("[metric]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.SummaryTableMetric));
			Assert.Equal("[csv]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.CsvStatAverage));
			Assert.Equal("[csvmin]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.CsvStatMin));
			Assert.Equal("[csvmax]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.CsvStatMax));
			Assert.Equal("[meta]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.CsvMetadata));
			Assert.Equal("[toolmeta]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.ToolMetadata));
			Assert.Equal("[extmeta]", SummaryTable.GetElementTypeStatPrefix(SummaryTableElement.Type.ExternalMetadata));
		}

		#endregion

		#region Qualified stat type parsing

		[Fact]
		public void GetQualifiedStatType_ParsesCorrectly()
		{
			Assert.Equal(SummaryTableElement.Type.SummaryTableMetric, SummaryTable.GetQualfiedStatType("[metric]fps"));
			Assert.Equal(SummaryTableElement.Type.CsvStatAverage, SummaryTable.GetQualfiedStatType("[csv]frametime"));
			Assert.Equal(SummaryTableElement.Type.CsvStatMin, SummaryTable.GetQualfiedStatType("[csvmin]frametime"));
			Assert.Equal(SummaryTableElement.Type.CsvStatMax, SummaryTable.GetQualfiedStatType("[csvmax]frametime"));
			Assert.Equal(SummaryTableElement.Type.CsvMetadata, SummaryTable.GetQualfiedStatType("[meta]platform"));
			Assert.Equal(SummaryTableElement.Type.ToolMetadata, SummaryTable.GetQualfiedStatType("[toolmeta]version"));
			Assert.Equal(SummaryTableElement.Type.ExternalMetadata, SummaryTable.GetQualfiedStatType("[extmeta]extra"));
		}

		[Fact]
		public void GetQualifiedStatType_NoPrefix_ReturnsCOUNT()
		{
			Assert.Equal(SummaryTableElement.Type.COUNT, SummaryTable.GetQualfiedStatType("frametime"));
		}

		[Fact]
		public void GetQualifiedStatType_WithOutputName_StripsPrefix()
		{
			SummaryTableElement.Type type = SummaryTable.GetQualfiedStatType("[metric]mvp", out string name);
			Assert.Equal(SummaryTableElement.Type.SummaryTableMetric, type);
			Assert.Equal("mvp", name);
		}

		#endregion

		#region Base stat name parsing

		[Theory]
		[InlineData("Avg frametime", "frametime")]
		[InlineData("Min frametime", "frametime")]
		[InlineData("Max frametime", "frametime")]
		[InlineData("frametime", "frametime")]
		public void GetBaseStatName_StripsPrefix(string input, string expected)
		{
			Assert.Equal(expected, SummaryTable.GetBaseStatName(input));
		}

		#endregion

		#region Row visibility with diff rows

		[Fact]
		public void IsRowVisible_NoDiffRows_AllVisible()
		{
			// When bShowOnlyDiffRows=false, ALL rows are visible regardless of diff status
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.None, false, 0));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.None, false, 1));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, false, 0));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, false, 2));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, false, 3));
		}

		[Fact]
		public void IsRowVisible_ShowOnlyDiffRows_ShowsDiffRowsOnly()
		{
			// Rows 0 and 1 are never diff rows. For Alternating, diff rows start at index 2.
			// IsDiffRow: row 2 = diff, row 3 = not diff, row 4 = diff
			Assert.False(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, true, 0));
			Assert.False(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, true, 1));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, true, 2));  // first diff row
			Assert.False(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, true, 3));
			Assert.True(SummaryTable.IsRowVisible(DiffRowFrequency.Alternating, true, 4));  // second diff row
		}

		[Fact]
		public void IsDiffRow_ReturnsCorrectly()
		{
			// Rows 0 and 1 are NEVER diff rows
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.None, 0));
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.None, 1));
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.Alternating, 0));
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.Alternating, 1));
			// Alternating: (rowIndex - 2) % 2 == 0 for diff rows
			Assert.True(SummaryTable.IsDiffRow(DiffRowFrequency.Alternating, 2));   // (0 % 2 == 0) = true
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.Alternating, 3));  // (1 % 2 == 0) = false
			Assert.True(SummaryTable.IsDiffRow(DiffRowFrequency.Alternating, 4));   // (2 % 2 == 0) = true
			// AfterEachPair: (rowIndex - 2) % 3 == 0 for diff rows
			Assert.True(SummaryTable.IsDiffRow(DiffRowFrequency.AfterEachPair, 2));   // (0 % 3 == 0)
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.AfterEachPair, 3));
			Assert.False(SummaryTable.IsDiffRow(DiffRowFrequency.AfterEachPair, 4));
			Assert.True(SummaryTable.IsDiffRow(DiffRowFrequency.AfterEachPair, 5));   // (3 % 3 == 0)
		}

		#endregion
	}

	public class SummaryTableDataTests
	{
		#region AddRowData and basic table operations

		[Fact]
		public void AddRowData_IncreasesCount()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");

			table.AddRowData(row, true, false, false, false);
			Assert.Equal(1, table.Count);

			table.AddRowData(row, true, false, false, false);
			Assert.Equal(2, table.Count);
		}

		[Fact]
		public void AddRowData_CreatesColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");

			table.AddRowData(row, true, false, false, false);

			Assert.NotNull(table.GetColumnByName("frametime"));
			Assert.NotNull(table.GetColumnByName("platform"));
		}

		[Fact]
		public void AddRowData_WithCsvStatMin_IncludesMinColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvStatMin, "frametime", 8.0);

			table.AddRowData(row, true, true, false, false);

			// Should have a column for the min value
			Assert.NotNull(table.GetColumnByName("frametime"));
		}

		[Fact]
		public void GetColumnByName_MissingColumn_ReturnsNull()
		{
			var table = new SummaryTable();
			Assert.Null(table.GetColumnByName("nonexistent"));
		}

		#endregion

		#region SortAndFilter

		[Fact]
		public void SortAndFilter_FiltersColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "drawcalls", 500.0);
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			table.AddRowData(row, true, false, false, false);

			// Only include frametime in the filter list
			var filtered = table.SortAndFilter(
				new List<string> { "frametime" },
				new List<string>(),
				false,
				null,
				null,
				false,
				TableColumnSortMode.Default);

			Assert.NotNull(filtered.GetColumnByName("frametime"));
			// drawcalls should be filtered out
			Assert.Null(filtered.GetColumnByName("drawcalls"));
		}

		[Fact]
		public void SortAndFilter_WildcardFilter_MatchesMultipleColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "render/drawcalls", 500.0);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "render/triangles", 1000.0);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "physics/time", 5.0);
			table.AddRowData(row, true, false, false, false);

			var filtered = table.SortAndFilter(
				new List<string> { "render/*" },
				new List<string>(),
				false,
				null,
				null,
				false,
				TableColumnSortMode.Default);

			Assert.NotNull(filtered.GetColumnByName("render/drawcalls"));
			Assert.NotNull(filtered.GetColumnByName("render/triangles"));
			Assert.Null(filtered.GetColumnByName("physics/time"));
		}

		[Fact]
		public void SortAndFilter_WithAdditionalFilters_AppliesFilters()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvStatAverage, "lowstat", 0.001);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "highstat", 100.0);
			table.AddRowData(row, true, false, false, false);

			var filters = new List<ISummaryTableColumnFilter>
			{
				new StatThresholdColumnFilter(1.0f)
			};

			var filtered = table.SortAndFilter(
				new List<string> { "lowstat", "highstat" },
				new List<string>(),
				false,
				null,
				filters,
				false,
				TableColumnSortMode.Default);

			// highstat should remain, lowstat should be filtered
			Assert.NotNull(filtered.GetColumnByName("highstat"));
			Assert.Null(filtered.GetColumnByName("lowstat"));
		}

		#endregion

		#region CollateSortedTable

		[Fact]
		public void CollateSortedTable_MergesRowsByKey()
		{
			var table = new SummaryTable();

			// Add two rows with same platform
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 20.0);
			table.AddRowData(row2, true, false, false, false);

			// Add one row with different platform
			var row3 = new SummaryTableRowData();
			row3.Add(SummaryTableElement.Type.CsvMetadata, "platform", "linux");
			row3.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 33.0);
			table.AddRowData(row3, true, false, false, false);

			// First sort by platform so same platforms are adjacent
			var sorted = table.SortRows(new List<string> { "platform" }, false);
			var collated = sorted.CollateSortedTable(
				new List<string> { "platform" },
				false,
				StringCollationVisibility.Auto,
				DateCollationVisibility.Newest, 
				CollateAverageMethod.Mean);

			// Should have 2 collated rows (linux and windows)
			Assert.Equal(2, collated.Count);
		}

		[Fact]
		public void CollateSortedTable_WithMinMaxColumns_AddsAggregates()
		{
			var table = new SummaryTable();

			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 20.0);
			table.AddRowData(row2, true, false, false, false);

			var sorted = table.SortRows(new List<string> { "platform" }, false);
			var collated = sorted.CollateSortedTable(
				new List<string> { "platform" },
				true, // addMinMaxColumns
				StringCollationVisibility.Auto,
				DateCollationVisibility.Newest,
				CollateAverageMethod.Mean);

			// Collated table should have at least 1 row and contain aggregate columns
			Assert.Equal(1, collated.Count);

			// The collated column names include aggregate prefixes: "Avg frametime", "Min frametime", "Max frametime"
			// GetColumnByName may use the full name or lookup key. Verify we have more than just the collateBy columns.
			// We can enumerate to verify aggregate columns exist
			var rows = collated.EnumerateTableDataByRow().ToList();
			Assert.True(rows.Count > 0, "Collated table should have rows");
		}

		#endregion

		#region WriteToCSV

		[Fact]
		[Trait("Category", "FileIO")]
		public void WriteToCSV_ProducesValidFile()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			table.AddRowData(row, true, false, false, false);

			string tempFile = Path.Combine(Path.GetTempPath(), "test_summary_" + Path.GetRandomFileName() + ".csv");
			try
			{
				table.WriteToCSV(tempFile);
				Assert.True(File.Exists(tempFile));
				string content = File.ReadAllText(tempFile);
				Assert.Contains("frametime", content);
			}
			finally
			{
				if (File.Exists(tempFile)) File.Delete(tempFile);
			}
		}

		#endregion

		#region SortRows

		[Fact]
		public void SortRows_SortsBySpecifiedColumn()
		{
			var table = new SummaryTable();

			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 33.0);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "android");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			table.AddRowData(row2, true, false, false, false);

			var row3 = new SummaryTableRowData();
			row3.Add(SummaryTableElement.Type.CsvMetadata, "platform", "linux");
			row3.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 22.0);
			table.AddRowData(row3, true, false, false, false);

			var sorted = table.SortRows(new List<string> { "platform" }, false);

			var platformCol = sorted.GetColumnByName("platform");
			Assert.NotNull(platformCol);

			// Should be sorted alphabetically: android, linux, windows
			Assert.Equal("android", platformCol.GetStringValue(0));
			Assert.Equal("linux", platformCol.GetStringValue(1));
			Assert.Equal("windows", platformCol.GetStringValue(2));
		}

		[Fact]
		public void SortRows_ReverseSort_ReversesOrder()
		{
			var table = new SummaryTable();

			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "a");
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "b");
			table.AddRowData(row2, true, false, false, false);

			var sorted = table.SortRows(new List<string> { "platform" }, true);
			var platformCol = sorted.GetColumnByName("platform");
			Assert.Equal("b", platformCol.GetStringValue(0));
			Assert.Equal("a", platformCol.GetStringValue(1));
		}

		#endregion

		#region ToFlatJsonDict

		/// <summary>Helper: add a row with optional csvid, platform metadata, and a frametime stat.</summary>
		private static void AddRow(SummaryTable table, string csvId, string platform, double frametime)
		{
			var row = new SummaryTableRowData();
			if (csvId != null) row.Add(SummaryTableElement.Type.CsvMetadata, "csvid", csvId);
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", platform);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", frametime);
			table.AddRowData(row, true, false, false, false);
		}

		/// <summary>Helper: sort by platform then collate.</summary>
		private static SummaryTable SortAndCollateByPlatform(SummaryTable table, bool addMinMax = false)
		{
			var sorted = table.SortRows(new List<string> { "platform" }, false);
			return sorted.CollateSortedTable(
				new List<string> { "platform" }, addMinMax,
				StringCollationVisibility.Auto, DateCollationVisibility.Newest, CollateAverageMethod.Mean);
		}

		/// <summary>Helper: build a 3-row table (2x windows + 1x linux).</summary>
		private static SummaryTable BuildThreeRowTable()
		{
			var table = new SummaryTable();
			AddRow(table, "run_001", "windows", 16.0);
			AddRow(table, "run_002", "windows", 20.0);
			AddRow(table, "run_003", "linux", 33.0);
			return table;
		}

		/// <summary>Helper: build a 2-row table (2x windows, same platform for collation).</summary>
		private static SummaryTable BuildTwoWindowsRowTable()
		{
			var table = new SummaryTable();
			AddRow(table, "run_001", "windows", 16.0);
			AddRow(table, "run_002", "windows", 20.0);
			return table;
		}

		[Fact]
		public void ToFlatJsonDict_KeysByCsvId()
		{
			var dict = BuildTwoWindowsRowTable().ToFlatJsonDict();

			Assert.Equal(2, dict.Count);
			Assert.True(dict.ContainsKey("run_001"));
			Assert.True(dict.ContainsKey("run_002"));
		}

		[Fact]
		public void ToFlatJsonDict_FallsBackToRowIndex_WhenNoCsvId()
		{
			var table = new SummaryTable();
			AddRow(table, null, "windows", 16.0);
			AddRow(table, null, "linux", 20.0);

			var dict = table.ToFlatJsonDict();

			Assert.Equal(2, dict.Count);
			Assert.True(dict.ContainsKey("0"));
			Assert.True(dict.ContainsKey("1"));
		}

		[Fact]
		public void ToFlatJsonDict_IncludesNumericAndStringValues()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "drawcalls", 500.0);
			table.AddRowData(row, true, false, false, false);

			var dict = table.ToFlatJsonDict();
			Dictionary<string, dynamic> rowDict = dict["run_001"];

			Assert.Equal("windows", (string)rowDict["platform"]);
			Assert.Equal(16.6m, (decimal)rowDict["frametime"]);
			Assert.Equal(500.0m, (decimal)rowDict["drawcalls"]);
		}

		[Fact]
		public void ToFlatJsonDict_OmitsMaxValueSentinels()
		{
			var table = new SummaryTable();
			// row1 has drawcalls, row2 doesn't — column will contain double.MaxValue sentinel
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "drawcalls", 500.0);
			table.AddRowData(row1, true, false, false, false);

			AddRow(table, "run_002", "windows", 20.0);

			var dict = table.ToFlatJsonDict();
			Dictionary<string, dynamic> row2Dict = dict["run_002"];

			Assert.True(row2Dict.ContainsKey("frametime"));
			Assert.False(row2Dict.ContainsKey("drawcalls"), "MaxValue sentinel should be omitted");
		}

		[Fact]
		public void ToFlatJsonDict_OmitsEmptyStrings()
		{
			var table = new SummaryTable();
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row1.Add(SummaryTableElement.Type.CsvMetadata, "config", "shipping");
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_002");
			table.AddRowData(row2, true, false, false, false);

			var dict = table.ToFlatJsonDict();
			Assert.False(dict["run_002"].ContainsKey("config"), "Empty string values should be omitted");
		}

		[Fact]
		public void ToFlatJsonDict_OmitsInvisibleColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "hidden_stat", 99.0);
			table.AddRowData(row, true, false, false, false);

			table.GetColumnByName("hidden_stat").isVisible = false;

			var dict = table.ToFlatJsonDict();
			Dictionary<string, dynamic> rowDict = dict["run_001"];

			Assert.True(rowDict.ContainsKey("frametime"));
			Assert.False(rowDict.ContainsKey("hidden_stat"), "Invisible columns should be omitted");
		}

		[Fact]
		public void ToFlatJsonDict_CollatedTable_KeysBySortColumns()
		{
			var dict = SortAndCollateByPlatform(BuildThreeRowTable()).ToFlatJsonDict();

			Assert.Equal(2, dict.Count);
			Assert.True(dict.ContainsKey("linux"), "Collated key should be the sort column value");
			Assert.True(dict.ContainsKey("windows"), "Collated key should be the sort column value");
		}

		[Fact]
		public void ToFlatJsonDict_CollatedTable_MultiSortKey_JoinsWithUnderscore()
		{
			var table = new SummaryTable();
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvMetadata, "config", "shipping");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_002");
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row2.Add(SummaryTableElement.Type.CsvMetadata, "config", "development");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 33.0);
			table.AddRowData(row2, true, false, false, false);

			var sorted = table.SortRows(new List<string> { "platform", "config" }, false);
			var collated = sorted.CollateSortedTable(
				new List<string> { "platform", "config" }, false,
				StringCollationVisibility.Auto, DateCollationVisibility.Newest, CollateAverageMethod.Mean);

			var dict = collated.ToFlatJsonDict();

			Assert.Equal(2, dict.Count);
			Assert.True(dict.ContainsKey("windows_development"));
			Assert.True(dict.ContainsKey("windows_shipping"));
		}

		[Fact]
		public void ToFlatJsonDict_CollatedTable_AveragesNumericValues()
		{
			var dict = SortAndCollateByPlatform(BuildTwoWindowsRowTable()).ToFlatJsonDict();
			// The avg of 16.0 and 20.0 is 18.0
			Assert.Equal(18.0m, (decimal)dict["windows"]["Avg frametime"]);
		}

		[Fact]
		public void ToFlatJsonDict_CollatedTable_WithMinMax_IncludesAggregateColumns()
		{
			var dict = SortAndCollateByPlatform(BuildTwoWindowsRowTable(), addMinMax: true).ToFlatJsonDict();
			Dictionary<string, dynamic> windowsRow = dict["windows"];

			Assert.Equal(18.0m, (decimal)windowsRow["Avg frametime"]);
			Assert.Equal(16.0m, (decimal)windowsRow["Min frametime"]);
			Assert.Equal(20.0m, (decimal)windowsRow["Max frametime"]);
		}

		[Fact]
		public void ToFlatJsonDict_AfterSortAndFilter_OnlyIncludesFilteredColumns()
		{
			var table = new SummaryTable();
			var row = new SummaryTableRowData();
			row.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			row.Add(SummaryTableElement.Type.CsvStatAverage, "drawcalls", 500.0);
			table.AddRowData(row, true, false, false, false);

			var filtered = table.SortAndFilter(
				new List<string> { "frametime" }, new List<string>(),
				false, null, null, false, TableColumnSortMode.Default);

			var dict = filtered.ToFlatJsonDict();
			Dictionary<string, dynamic> rowDict = dict["run_001"];

			Assert.True(rowDict.ContainsKey("frametime"));
			Assert.False(rowDict.ContainsKey("drawcalls"), "Filtered-out columns should not appear in JSON");
		}

		[Fact]
		public void ToFlatJsonDict_IncludesCountColumn_InCollatedTable()
		{
			var dict = SortAndCollateByPlatform(BuildThreeRowTable()).ToFlatJsonDict();

			Assert.Equal(2.0m, (decimal)dict["windows"]["Count"]);
			Assert.Equal(1.0m, (decimal)dict["linux"]["Count"]);
		}

		[Fact]
		public void ToFlatJsonDict_EmptyTable_ReturnsEmptyDict()
		{
			Assert.Empty(new SummaryTable().ToFlatJsonDict());
		}

		#endregion
	}

	public class SummaryTableJsonWriteTests : IDisposable
	{
		private readonly string _tempDir;

		public SummaryTableJsonWriteTests()
		{
			_tempDir = Path.Combine(Path.GetTempPath(), "prt_json_test_" + Path.GetRandomFileName());
			Directory.CreateDirectory(_tempDir);
		}

		public void Dispose()
		{
			if (Directory.Exists(_tempDir))
			{
				Directory.Delete(_tempDir, true);
			}
		}

		private static Dictionary<string, dynamic> MakeSampleDict()
		{
			return new Dictionary<string, dynamic>
			{
				["run_001"] = new Dictionary<string, dynamic>
				{
					["frametime"] = new decimal(16.6),
					["platform"] = "windows"
				}
			};
		}

		private string WriteAndRead(Dictionary<string, dynamic> dict, string name, bool stream = false, bool indent = true)
		{
			string filePath = Path.Combine(_tempDir, name);
			SummaryTableDataJsonWriteHelper.WriteJsonFile(stream, filePath, dict, indent);
			Assert.True(File.Exists(filePath));
			return File.ReadAllText(filePath);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void WriteJsonFile_ProducesValidJsonFile()
		{
			string content = WriteAndRead(MakeSampleDict(), "test_output.json");
			Assert.Contains("run_001", content);
			Assert.Contains("frametime", content);
			Assert.Contains("16.6", content);
			Assert.Contains("windows", content);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void WriteJsonFile_NoIndent_ProducesCompactJson()
		{
			string content = WriteAndRead(MakeSampleDict(), "compact.json", indent: false);
			Assert.DoesNotContain("\n", content.Trim());
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void WriteJsonFile_StreamMode_ProducesValidFile()
		{
			string content = WriteAndRead(MakeSampleDict(), "streamed.json", stream: true, indent: false);
			Assert.Contains("run_001", content);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void ToFlatJsonDict_RoundTrips_ThroughWriteJsonFile()
		{
			var table = new SummaryTable();
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.6);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_002");
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "linux");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 33.3);
			table.AddRowData(row2, true, false, false, false);

			string content = WriteAndRead(table.ToFlatJsonDict(), "roundtrip.json");
			Assert.Contains("run_001", content);
			Assert.Contains("run_002", content);
			Assert.Contains("windows", content);
			Assert.Contains("linux", content);
		}

		[Fact]
		[Trait("Category", "FileIO")]
		public void CollatedTable_RoundTrips_ThroughWriteJsonFile()
		{
			var table = new SummaryTable();
			var row1 = new SummaryTableRowData();
			row1.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_001");
			row1.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row1.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 16.0);
			table.AddRowData(row1, true, false, false, false);

			var row2 = new SummaryTableRowData();
			row2.Add(SummaryTableElement.Type.CsvMetadata, "csvid", "run_002");
			row2.Add(SummaryTableElement.Type.CsvMetadata, "platform", "windows");
			row2.Add(SummaryTableElement.Type.CsvStatAverage, "frametime", 20.0);
			table.AddRowData(row2, true, false, false, false);

			var sorted = table.SortRows(new List<string> { "platform" }, false);
			var collated = sorted.CollateSortedTable(
				new List<string> { "platform" }, true,
				StringCollationVisibility.Auto, DateCollationVisibility.Newest, CollateAverageMethod.Mean);

			string content = WriteAndRead(collated.ToFlatJsonDict(), "collated.json");
			Assert.Contains("windows", content);
			Assert.Contains("Avg frametime", content);
			Assert.Contains("Min frametime", content);
			Assert.Contains("Max frametime", content);
		}
	}

	public class TableUtilTests
	{
		[Fact]
		public void FormatStatName_AddsWordBreakAfterSlash()
		{
			Assert.Equal("render/<wbr>drawcalls", TableUtil.FormatStatName("render/drawcalls"));
		}

		[Fact]
		public void SanitizeHtmlString_EscapesAngleBrackets()
		{
			Assert.Equal("&lt;script&gt;", TableUtil.SanitizeHtmlString("<script>"));
		}

		[Fact]
		public void SafeTruncateHtmlTableValue_TruncatesLongString()
		{
			string result = TableUtil.SafeTruncateHtmlTableValue("This is a very long string", 10);
			Assert.EndsWith("...", result);
		}

		[Fact]
		public void SafeTruncateHtmlTableValue_LinkTemplate_NotTruncated()
		{
			string input = "{LinkTemplate:Report:csvid123}";
			Assert.Equal(input, TableUtil.SafeTruncateHtmlTableValue(input, 5));
		}
	}
}
