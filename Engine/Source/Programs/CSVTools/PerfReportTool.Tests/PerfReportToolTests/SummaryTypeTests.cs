// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Linq;
using System.Xml.Linq;
using CSVStats;
using PerfReportTool;
using PerfSummaries;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class StandardSummaryTests
	{
		[Fact]
		public void GetName_ReturnsStandard()
		{
			var summary = new StandardSummary();
			Assert.Equal("standard", summary.GetName());
		}

		[Fact]
		public void WriteSummaryData_ComputesTotalTime()
		{
			var summary = new StandardSummary();
			// 1000 frames at 16.67ms each = 16670ms = 16.67s
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 1000, 16.67f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("total time (s)"));
			double totalTime = rowData.Get("total time (s)").numericValue;
			// 1000 * 16.67ms / 1000.0 = 16.67s
			Assert.Equal(16.67, totalTime, 0.01);
		}

		[Fact]
		public void WriteSummaryData_DoesNotOverwriteExisting()
		{
			var summary = new StandardSummary();
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 10.0f);
			var rowData = new SummaryTableRowData();
			rowData.Add(SummaryTableElement.Type.SummaryTableMetric, "Total Time (s)", 999.0);

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			// Should not overwrite the existing value
			Assert.Equal(999.0, rowData.Get("total time (s)").numericValue);
		}

		[Fact]
		public void WriteSummaryData_NullRowData_DoesNotThrow()
		{
			var summary = new StandardSummary();
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 10.0f);
			// Should not throw with null rowData
			summary.WriteSummaryData(false, csvStats, csvStats, false, null, null);
		}
	}

	public class FPSChartSummaryTests
	{
		private static FPSChartSummary CreateFPSChartSummary(int fps = 30, float hitchThreshold = 150.0f, bool useEngineHitchMetric = false, int recommendedFps = 0, float slowFrameTimeBuffer = 0)
		{
			string ht = hitchThreshold.ToString(CultureInfo.InvariantCulture);
			string sfb = slowFrameTimeBuffer.ToString(CultureInfo.InvariantCulture);
			string xml = $@"<summary type='fpschart' fps='{fps}' hitchThreshold='{ht}' useEngineHitchMetric='{useEngineHitchMetric.ToString().ToLower()}' recommendedFps='{recommendedFps}' slowFrameTimeBuffer='{sfb}'>
				<stats/>
			</summary>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();
			return new FPSChartSummary(element, vars, "");
		}

		[Fact]
		public void GetName_ReturnsFpschart()
		{
			var summary = new FPSChartSummary();
			Assert.Equal("fpschart", summary.GetName());
		}

		[Fact]
		public void WriteSummaryData_ComputesMVP()
		{
			var summary = CreateFPSChartSummary(fps: 30, hitchThreshold: 150.0f);
			// Create 30fps-equivalent frameTimes (33.33ms each) for 10 seconds (300 frames)
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 300, 33.33f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("mvp"));
			double mvp = rowData.Get("mvp").numericValue;
			// At exact 30fps, MVP should be ~0 (no missed frames)
			Assert.True(mvp < 5.0, $"MVP should be near 0 for exact target frame rate, got {mvp}");
		}

		[Fact]
		public void WriteSummaryData_ComputesHitchesPerMinute()
		{
			var summary = CreateFPSChartSummary(fps: 30, hitchThreshold: 50.0f);
			// Create frames: mostly 33ms, but every 30th frame is a 100ms hitch
			float[] frameTimes = new float[300];
			for (int i = 0; i < 300; i++)
			{
				frameTimes[i] = (i % 30 == 0) ? 100.0f : 33.33f;
			}
			var csvStats = new CsvStats();
			var stat = CsvTestHelper.CreateStatSamples("frametime", frameTimes);
			csvStats.AddStat(stat);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("hitches/min"));
			double hitchesPerMin = rowData.Get("hitches/min").numericValue;
			Assert.True(hitchesPerMin > 0, "Should detect hitches");
		}

		[Fact]
		public void WriteSummaryData_ComputesTargetFPS()
		{
			var summary = CreateFPSChartSummary(fps: 60, hitchThreshold: 150.0f);
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 16.67f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("targetfps"));
			Assert.Equal(60.0, rowData.Get("targetfps").numericValue);
		}

		[Fact]
		public void WriteSummaryData_ComputesTotalTime()
		{
			var summary = CreateFPSChartSummary(fps: 30, hitchThreshold: 150.0f);
			// 100 frames at 33.33ms = ~3.33 seconds
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 33.33f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("total time (s)"));
			double totalTime = rowData.Get("total time (s)").numericValue;
			// 99 frames counted (last frame skipped) * 33.33ms / 1000
			Assert.True(totalTime > 3.0 && totalTime < 4.0, $"Expected ~3.3s, got {totalTime}");
		}

		[Fact]
		public void WriteSummaryData_NoHitches_ZeroHitchTimePercent()
		{
			var summary = CreateFPSChartSummary(fps: 30, hitchThreshold: 150.0f);
			// All frames at 33ms - well below 150ms hitch threshold
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 33.33f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("hitchtimepercent"));
			double htp = rowData.Get("hitchtimepercent").numericValue;
			Assert.Equal(0.0, htp, 0.01);
		}

		[Fact]
		public void WriteSummaryData_SlowFrameTimePercent_WithRecommendedFps()
		{
			var summary = CreateFPSChartSummary(fps: 30, hitchThreshold: 150.0f, recommendedFps: 30, slowFrameTimeBuffer: 0);
			// Half frames fast, half slow
			float[] frameTimes = new float[100];
			for (int i = 0; i < 100; i++)
			{
				frameTimes[i] = (i < 50) ? 20.0f : 50.0f; // 50ms >> 33.33ms target
			}
			var csvStats = new CsvStats();
			var stat = CsvTestHelper.CreateStatSamples("frametime", frameTimes);
			csvStats.AddStat(stat);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			Assert.True(rowData.Contains("slowframetimepercent"));
			double sftp = rowData.Get("slowframetimepercent").numericValue;
			Assert.True(sftp > 0, "Should have some slow frame time percent");
		}

		[Fact]
		public void WriteSummaryData_EngineHitchMetric_CountsHitchesDifferently()
		{
			var summarySimple = CreateFPSChartSummary(fps: 30, hitchThreshold: 50.0f, useEngineHitchMetric: false);
			var summaryEngine = CreateFPSChartSummary(fps: 30, hitchThreshold: 50.0f, useEngineHitchMetric: true);

			// Create frames with a slow section
			float[] frameTimes = new float[100];
			for (int i = 0; i < 100; i++)
			{
				frameTimes[i] = (i >= 10 && i <= 15) ? 80.0f : 33.33f;
			}
			var csvStats = new CsvStats();
			csvStats.AddStat(CsvTestHelper.CreateStatSamples("frametime", frameTimes));

			var rowDataSimple = new SummaryTableRowData();
			var rowDataEngine = new SummaryTableRowData();

			summarySimple.WriteSummaryData(false, csvStats, csvStats, false, rowDataSimple, null);
			summaryEngine.WriteSummaryData(false, csvStats, csvStats, false, rowDataEngine, null);

			double hitchesSimple = rowDataSimple.Get("hitches/min").numericValue;
			double hitchesEngine = rowDataEngine.Get("hitches/min").numericValue;

			// Simple mode should detect the 80ms frames as hitches (above 50ms threshold)
			Assert.True(hitchesSimple > 0, $"Simple hitch counting should detect hitches, got {hitchesSimple}");
			// The two modes should produce different hitch counts
			Assert.NotEqual(hitchesSimple, hitchesEngine);
		}
	}

	public class HitchSummaryTests
	{
		private static HitchSummary CreateHitchSummary(string thresholds = "50,100,150,200")
		{
			string xml = $@"<summary type='hitches'>
				<hitchThresholds>{thresholds}</hitchThresholds>
				<stats>frametime</stats>
			</summary>";
			var element = XElement.Parse(xml);
			var vars = new XmlVariableMappings();
			return new HitchSummary(element, vars, "");
		}

		[Fact]
		public void GetName_ReturnsHitches()
		{
			var summary = new HitchSummary();
			Assert.Equal("hitches", summary.GetName());
		}

		[Fact]
		public void Constructor_ParsesThresholds()
		{
			var summary = CreateHitchSummary("50,100,150,200");
			Assert.Equal(4, summary.HitchThresholds.Length);
			Assert.Equal(50.0, summary.HitchThresholds[0]);
			Assert.Equal(200.0, summary.HitchThresholds[3]);
		}

		[Fact]
		public void WriteSummaryData_AddsHitchMetrics_100msIntervals()
		{
			var summary = CreateHitchSummary();
			// Some frames with hitches
			float[] frameTimes = new float[100];
			for (int i = 0; i < 100; i++)
			{
				frameTimes[i] = (i == 50) ? 500.0f : 16.67f; // One 500ms hitch
			}
			var csvStats = new CsvStats();
			csvStats.AddStat(CsvTestHelper.CreateStatSamples("frametime", frameTimes));
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			// Should have Hitches>100ms through Hitches>1000ms entries
			Assert.True(rowData.Contains("hitches>100ms"));
			Assert.True(rowData.Contains("hitches>500ms"));
			Assert.True(rowData.Contains("hitches>1000ms"));

			// A 500ms frame is strictly >100ms through >400ms, but not >500ms (strict greater-than)
			double hitches100 = rowData.Get("hitches>100ms").numericValue;
			double hitches500 = rowData.Get("hitches>500ms").numericValue;
			double hitches600 = rowData.Get("hitches>600ms").numericValue;

			Assert.True(hitches100 >= 1, "Should have at least 1 hitch >100ms");
			Assert.Equal(0.0, hitches500);
			Assert.Equal(0.0, hitches600);
		}

		[Fact]
		public void WriteSummaryData_NoHitches_AllZero()
		{
			var summary = CreateHitchSummary();
			var csvStats = CsvTestHelper.CreateSimpleCsvStats(new[] { "frametime" }, 100, 16.67f);
			var rowData = new SummaryTableRowData();

			summary.WriteSummaryData(false, csvStats, csvStats, false, rowData, null);

			for (int i = 1; i <= 10; i++)
			{
				string key = $"hitches>{i * 100}ms";
				Assert.True(rowData.Contains(key), $"Missing {key}");
				Assert.Equal(0.0, rowData.Get(key).numericValue);
			}
		}
	}

	public class SummaryBaseClassTests
	{
		[Fact]
		public void StandardSummary_CanBeInstantiated()
		{
			var summary = new StandardSummary();
			Assert.Equal("standard", summary.GetName());
		}
	}

}
