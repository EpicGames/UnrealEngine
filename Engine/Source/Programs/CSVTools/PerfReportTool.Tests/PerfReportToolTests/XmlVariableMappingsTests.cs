using PerfReportTool;
using PerfReportTool.Tests.TestHelpers;
using Xunit;

namespace PerfReportTool.Tests.PerfReportToolTests
{
	public class XmlVariableMappingsTests
	{
		[Fact]
		public void SetVariable_ValidName_StoresCorrectly()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("myVar", "myValue");

			string result = vars.ResolveVariables("${myVar}");
			Assert.Equal("myValue", result);
		}

		[Fact]
		public void SetVariable_WithDot_StoresCorrectly()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("my.var", "dotValue");

			string result = vars.ResolveVariables("${my.var}");
			Assert.Equal("dotValue", result);
		}

		[Fact]
		public void SetVariable_WithUnderscore_StoresCorrectly()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("my_var", "underscoreValue");

			string result = vars.ResolveVariables("${my_var}");
			Assert.Equal("underscoreValue", result);
		}

		[Fact]
		public void SetVariable_InvalidName_ThrowsException()
		{
			var vars = new XmlVariableMappings();
			Assert.Throws<System.Exception>(() => vars.SetVariable("invalid name!", "value"));
		}

		[Fact]
		public void SetMetadataVariables_SetsMetaPrefix()
		{
			var vars = new XmlVariableMappings();
			var metadata = CsvTestHelper.CreateMetadata(("platform", "windows"), ("config", "test"));
			vars.SetMetadataVariables(metadata);

			Assert.Equal("windows", vars.ResolveVariables("${meta.platform}"));
			Assert.Equal("test", vars.ResolveVariables("${meta.config}"));
		}

		[Fact]
		public void ResolveVariables_NoVariables_ReturnsUnchanged()
		{
			var vars = new XmlVariableMappings();
			string input = "no variables here";
			Assert.Equal(input, vars.ResolveVariables(input));
		}

		[Fact]
		public void ResolveVariables_SimpleVariable_Resolves()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("name", "World");

			string result = vars.ResolveVariables("Hello ${name}!");
			Assert.Equal("Hello World!", result);
		}

		[Fact]
		public void ResolveVariables_MultipleVariables_ResolvesAll()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("a", "1");
			vars.SetVariable("b", "2");

			string result = vars.ResolveVariables("${a} and ${b}");
			Assert.Equal("1 and 2", result);
		}

		[Fact]
		public void ResolveVariables_MissingVariable_ReturnsEmptyString()
		{
			var vars = new XmlVariableMappings();
			string result = vars.ResolveVariables("before ${missing} after");
			Assert.Equal("before  after", result);
		}

		[Fact]
		public void ResolveVariables_ArrayIndexVariable_ResolvesElement()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("colors", "red,green,blue");

			Assert.Equal("red", vars.ResolveVariables("${colors[0]}"));
			Assert.Equal("green", vars.ResolveVariables("${colors[1]}"));
			Assert.Equal("blue", vars.ResolveVariables("${colors[2]}"));
		}

		[Fact]
		public void ResolveVariables_OverwrittenVariable_UsesLatestValue()
		{
			var vars = new XmlVariableMappings();
			vars.SetVariable("x", "first");
			Assert.Equal("first", vars.ResolveVariables("${x}"));

			vars.SetVariable("x", "second");
			Assert.Equal("second", vars.ResolveVariables("${x}"));
		}

		[Fact]
		public void ResolveVariables_NoDollarSign_SkipsProcessing()
		{
			var vars = new XmlVariableMappings();
			// No $ means fast path - no processing
			string result = vars.ResolveVariables("simple text");
			Assert.Equal("simple text", result);
		}

		[Fact]
		public void ResolveVariables_IncompleteSyntax_ReturnsPartially()
		{
			var vars = new XmlVariableMappings();
			// Incomplete variable syntax - ${... without closing brace
			string result = vars.ResolveVariables("${unclosed");
			Assert.Equal("${unclosed", result);
		}
	}
}
