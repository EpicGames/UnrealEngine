using System;
using System.IO;

namespace PerfReportTool.Tests
{
	public class TestDataFixture : IDisposable
	{
		/// <summary>Path to the in-project TestData directory (always available).</summary>
		public string TestDataDir { get; }
		public bool IsAvailable { get; }

		public TestDataFixture()
		{
			// 1. Check environment variable override (supports absolute or relative paths)
			string envPath = Environment.GetEnvironmentVariable("PERFREPORT_TEST_DATA_DIR");
			if (!string.IsNullOrEmpty(envPath))
			{
				string resolved = Path.GetFullPath(envPath);
				if (Directory.Exists(resolved))
				{
					TestDataDir = resolved;
					IsAvailable = true;
					return;
				}
			}

			// 2. Find the in-project TestData directory by walking up from the output directory
			string dir = AppContext.BaseDirectory;
			for (int i = 0; i < 10; i++)
			{
				string candidate = Path.Combine(dir, "TestData");
				if (Directory.Exists(candidate))
				{
					TestDataDir = candidate;
					IsAvailable = true;
					return;
				}
				string parent = Path.GetDirectoryName(dir);
				if (parent == null || parent == dir)
				{
					break;
				}
				dir = parent;
			}

			TestDataDir = null;
			IsAvailable = false;
		}

		public string GetFile(string relativePath)
		{
			if (TestDataDir == null)
			{
				throw new InvalidOperationException(
					"TestDataDir is not available. Call RequireTestData() before GetFile().");
			}
			return Path.Combine(TestDataDir, relativePath);
		}

		public string GetPerfCsvDir()
		{
			return TestDataDir;
		}

		/// <summary>
		/// Call at the start of any test that requires test data files.
		/// </summary>
		public void RequireTestData()
		{
			if (!IsAvailable)
			{
				throw new InvalidOperationException(
					"Test data not available. Ensure the TestData directory exists in the test project, " +
					"or set PERFREPORT_TEST_DATA_DIR environment variable.");
			}
		}

		public void Dispose()
		{
		}
	}
}
