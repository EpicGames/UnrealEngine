using System.Collections.Generic;
using CSVStats;
using PerfSummaries;

namespace PerfReportTool.Tests.TestHelpers
{
	internal static class CsvTestHelper
	{
		public static CSVStats.CsvStats CreateSimpleCsvStats(string[] statNames, int sampleCount, float defaultValue = 0.0f)
		{
			var csvStats = new CSVStats.CsvStats();
			foreach (string name in statNames)
			{
				var stat = new StatSamples(name);
				for (int i = 0; i < sampleCount; i++)
				{
					stat.samples.Add(defaultValue);
				}
				stat.ComputeAverageAndTotal();
				csvStats.AddStat(stat);
			}
			return csvStats;
		}

		public static CsvMetadata CreateMetadata(params (string key, string value)[] pairs)
		{
			var metadata = new CsvMetadata();
			foreach (var (key, value) in pairs)
			{
				metadata.Values[key] = value;
			}
			return metadata;
		}

		public static StatSamples CreateStatSamples(string name, float[] values)
		{
			var stat = new StatSamples(name);
			stat.samples.AddRange(values);
			stat.ComputeAverageAndTotal();
			return stat;
		}

		public static ColourThresholdList CreateDefaultThresholds(double red, double orange, double yellow, double green)
		{
			return new ColourThresholdList(red, orange, yellow, green);
		}
	}
}
