using System.IO;
using System.Reflection;

namespace PerfReportTool.Tests.TestHelpers
{
	internal static class ReportXmlTestHelper
	{
		/// <summary>
		/// Gets the directory containing the PerfReportTool output (where ReportTypes.xml etc. are copied).
		/// </summary>
		public static string GetPerfReportToolOutputDir()
		{
			// The PerfReportTool assembly is loaded as a dependency. Find its location.
			string assemblyLocation = typeof(PerfReportTool.Version).Assembly.Location;
			return Path.GetDirectoryName(assemblyLocation);
		}

		public static string GetReportTypesXmlPath()
		{
			return Path.Combine(GetPerfReportToolOutputDir(), "ReportTypes.xml");
		}

		public static string GetReportGraphsXmlPath()
		{
			return Path.Combine(GetPerfReportToolOutputDir(), "ReportGraphs.xml");
		}

		public static string GetLlmReportTypesXmlPath()
		{
			return Path.Combine(GetPerfReportToolOutputDir(), "LLMReportTypes.xml");
		}

		public static string GetLlmReportGraphsXmlPath()
		{
			return Path.Combine(GetPerfReportToolOutputDir(), "LLMReportGraphs.xml");
		}
	}
}
