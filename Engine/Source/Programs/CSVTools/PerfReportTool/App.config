<?xml version="1.0" encoding="utf-8"?>
<configuration>
    <startup> 
        <supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.8" />
    </startup>
  <runtime>
    <gcConcurrent enabled="false" />
    <assemblyBinding xmlns="urn:schemas-microsoft-com:asm.v1">
      <dependentAssembly>
        <assemblyIdentity name="System.Text.Encodings.Web" publicKeyToken="cc7b13ffcd2ddd51" culture="neutral" />
        <bindingRedirect oldVersion="0.0.0.0-5.0.0.1" newVersion="5.0.0.1" />
      </dependentAssembly>
    </assemblyBinding>
  </runtime>
</configuration>
