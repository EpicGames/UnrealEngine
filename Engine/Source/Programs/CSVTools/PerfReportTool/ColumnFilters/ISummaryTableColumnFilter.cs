using PerfSummaries;

namespace PerfReportTool
{
	internal interface ISummaryTableColumnFilter
	{
		bool ShouldFilter(SummaryTableColumn column, SummaryTable table);
	}
}
