using PerfSummaries;
using System.Collections.Generic;
using System.Linq;

namespace PerfReportTool
{
	// Filters out metadata columns.
	internal class MetadataColumnFilter : ISummaryTableColumnFilter
	{
		// Metadata columns to not filter out.
		private HashSet<string> ColumnsToIgnore;

		public MetadataColumnFilter(List<string> columnsToNotFilter)
		{
			ColumnsToIgnore = columnsToNotFilter.ToHashSet();
		}

		public bool ShouldFilter(SummaryTableColumn column, SummaryTable table)
		{
			if ((column.elementType == SummaryTableElement.Type.CsvMetadata ||
				 column.elementType == SummaryTableElement.Type.ToolMetadata) &&
				 !ColumnsToIgnore.Contains(column.name))
			{
				column.DebugMarkAsFiltered(GetType().ToString(), "Column is metadata");
				return true;
			}
			return false;
		}
	}
}
