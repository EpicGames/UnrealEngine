using PerfSummaries;
using System;
using System.Collections.Generic;
using System.Linq;

namespace PerfReportTool
{
	// Filters out any columns where the stat hasn't regressed in the most row group.
	internal class RegressionColumnFilter : ISummaryTableColumnFilter
	{
		// The stat name to join rows together that should be aggregated for sake of regression testing.
		private string JoinRowsByStatName = null;
		// Number of standard deviations from the mean the latest row group must be outside of to be considered a regression.
		private double StdDevTreshold = 2.0;
		// Number of standard deviations from the mean a value must be to be considered an outlier.
		private double OutlierStdDevThreshold = 4.0;
		// When the % of rows without a value is over this amount we just filter out the whole column due to lack of data.
		private const double MissingRowThresholdPercent = 0.6;

		public RegressionColumnFilter(string joinRowsByName, float stdDevThreshold, float outlierStdDevThreshold)
		{
			JoinRowsByStatName = joinRowsByName;
			StdDevTreshold = stdDevThreshold;
			OutlierStdDevThreshold = outlierStdDevThreshold;
		}

		// Overview of how this works:
		// 1. If too many values are missing from the column we filter it out.
		// 2. We calculate the mean and standard deviation of all rows except those belonging to the first row group.
		// 3. We remove any rows whose values are outside the (aggressive) outlier std dev threshold as these will skew our mean.
		// 4. We re-calculate the mean and std deviation of the rows now that the outliers are gone.
		// 5. We calculate the mean and std dev of the rows belonging to the first row group.
		// 6. If the mean of the first row group - the mean of the rest of the rows (minus outliers) is outside our std dev threshold then there's a regression.
		public bool ShouldFilter(SummaryTableColumn column, SummaryTable table)
		{
			if (!column.isNumeric ||
				(column.elementType != SummaryTableElement.Type.CsvStatAverage &&
				 column.elementType != SummaryTableElement.Type.CsvStatMin &&
				 column.elementType != SummaryTableElement.Type.CsvStatMax &&
				 column.elementType != SummaryTableElement.Type.SummaryTableMetric))
			{
				return false;
			}

			int rowCount = table.Count;
			if (rowCount == 0)
			{
				column.DebugMarkAsFiltered(GetType().ToString(), "Column doesn't have any rows.");
				return true;
			}

			int firstRowGroupCount = GetNumRowsInFirstRowGroup(table);
			if (firstRowGroupCount == rowCount)
			{
				column.DebugMarkAsFiltered(GetType().ToString(), "All rows of data belong to the same group so there's nothing to compare against.");
				return true;
			}

			List<Tuple<int, double>> allValuesExcludingFirstRowGroup = GetNonEmptyRows(column, firstRowGroupCount, rowCount);

			// If too many rows are missing values then just ignore the whole column.
			double percentMissing = 1.0 - ((double)allValuesExcludingFirstRowGroup.Count / (double)rowCount);
			if (percentMissing > MissingRowThresholdPercent)
			{
				column.DebugMarkAsFiltered(GetType().ToString(), $"Column is missing {percentMissing * 100.0}% of its values, which is more than the threshold {MissingRowThresholdPercent * 100.0}%.");
				return true;
			}

			bool highIsBad = true;
			if (column.formatInfo != null)
			{
				highIsBad = column.formatInfo.autoColorizeMode == AutoColorizeMode.HighIsBad;
			}

			// Calculate the mean and std dev for all the rows except the ones belonging to the first row group.
			double mean, stdDev;
			CalculateStdDevAndMeanForRange(allValuesExcludingFirstRowGroup, out mean, out stdDev);

			// Use the more aggressive std dev threshold to remove any outliers as these will skew our mean to the point where it may hide new regressions.
			var rowsWithoutOutliers = new List<Tuple<int, double>>();
			foreach (Tuple<int, double> row in allValuesExcludingFirstRowGroup)
			{
				int index = row.Item1;
				double value = row.Item2;
				if (!IsValueOutsideThreshold(value, mean, stdDev, OutlierStdDevThreshold, highIsBad))
				{
					rowsWithoutOutliers.Add(row);
				}
				else
				{
					column.DebugMarkRowInvalid(index, "RegressionFilter identified this value as an outlier.");
				}
			}

			// Re-calculate with the outliers removed
			CalculateStdDevAndMeanForRange(rowsWithoutOutliers, out mean, out stdDev);

			// Grab the rows for the first group.
			List<Tuple<int, double>> firstRowGroupValues = GetNonEmptyRows(column, 0, firstRowGroupCount);
			if (firstRowGroupValues.Count == 0)
			{
				column.DebugMarkAsFiltered(GetType().ToString(), "Missing values for the first row group.");
				return true;
			}

			// Calculate the mean and std dev of the first row group.
			double firstRowGroupMean, firstRowGroupStdDev;
			CalculateStdDevAndMeanForRange(firstRowGroupValues, out firstRowGroupMean, out firstRowGroupStdDev);

			// Check if the first row group is within the std dev threshold of the rest of the historical data excluding outliers.
			if (IsValueOutsideThreshold(firstRowGroupMean, mean, stdDev, StdDevTreshold, highIsBad))
			{
				return false;
			}

			double modifier = highIsBad ? 1.0 : -1.0;
			column.DebugMarkAsFiltered(GetType().ToString(),
				$"First row group mean ({firstRowGroupMean:0.000}) is within the std dev threshold ({stdDev:0.000} * {StdDevTreshold:0.0} = {stdDev * StdDevTreshold:0.000}) of the total mean ({mean:0.000}). Difference = {(firstRowGroupMean - mean)* modifier:0.000}.");
			return true;
		}

		private bool IsValueOutsideThreshold(double value, double mean, double stdDev, double numStdDevations, bool highIsBad)
		{
			double modifier = highIsBad ? 1.0 : -1.0;
			double difference = (value - mean) * modifier;
			double threshold = stdDev * numStdDevations;
			return difference > threshold;
		}

		private void CalculateStdDevAndMeanForRange(List<Tuple<int, double>> rows, out double mean, out double stdDev)
		{
			var values = rows.Select(tuple => tuple.Item2).ToList();
			CalculateStdDevAndMeanForRange(values, out mean, out stdDev);
		}

		private void CalculateStdDevAndMeanForRange(List<double> values, out double mean, out double stdDev)
		{
			int count = values.Count;
			double total = values.Sum();
			double avg = total / (double)count;

			List<double> distFromMean = values.Select(value => Math.Pow(value - avg, 2)).ToList();
			double totalDistFromMean = distFromMean.Sum();
			stdDev = Math.Sqrt(totalDistFromMean / count);
			mean = avg;
		}

		private List<Tuple<int, double>> GetNonEmptyRows(SummaryTableColumn column, int start, int end)
		{
			var values = new List<Tuple<int, double>>();
			for (int i = start; i < end; ++i)
			{
				double value = column.GetValue(i);
				if (value < double.MaxValue)
				{
					values.Add(Tuple.Create(i, value));
				}
			}
			return values;
		}

		private int GetNumRowsInFirstRowGroup(SummaryTable table)
		{
			if (JoinRowsByStatName == null)
			{
				// If no name is provided we treat each row distinctly.
				return 1;
			}

			SummaryTableColumn groupByColumn = table.GetColumnByName(JoinRowsByStatName);
			if (groupByColumn == null)
			{
				Console.Error.WriteLine($"[Error] -regressionJoinRowsByName doesn't refer to a real column name: {JoinRowsByStatName}. Will not be applying any grouping to rows.");
				return 1;
			}

			string firstRowValue = groupByColumn.GetStringValue(0);
			for (int count = 1; count < table.Count; ++count)
			{
				if (groupByColumn.GetStringValue(count) != firstRowValue)
				{
					return count;
				}
			}
			return table.Count;
		}
	}
}
