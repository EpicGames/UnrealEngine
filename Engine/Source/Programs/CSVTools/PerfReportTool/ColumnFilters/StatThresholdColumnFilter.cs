using PerfSummaries;
using System.Collections.Generic;
using System.Linq;

namespace PerfReportTool
{
	// Filter out csv stat or metric columns below the specified threshold
	internal class StatThresholdColumnFilter : ISummaryTableColumnFilter
	{
		private float StatThreshold;
		public StatThresholdColumnFilter(float statThreshold)
		{
			StatThreshold = statThreshold;
		}

		public bool ShouldFilter(SummaryTableColumn column, SummaryTable table)
		{
			if (StatThreshold <= 0.0f)
			{
				return false;
			}

			if (!column.isNumeric ||
				(column.elementType != SummaryTableElement.Type.CsvStatAverage &&
				 column.elementType != SummaryTableElement.Type.CsvStatMin &&
				 column.elementType != SummaryTableElement.Type.CsvStatMax &&
				 column.elementType != SummaryTableElement.Type.SummaryTableMetric &&
				 column.elementType != SummaryTableElement.Type.ExternalMetadata))
			{
				return false;
			}

			if (column.AreAllValuesOverThreshold((double)StatThreshold))
			{
				return false;
			}

			column.DebugMarkAsFiltered(GetType().ToString(), $"Not all column values are over thershold {StatThreshold}");
			return true;
		}
	}
}
