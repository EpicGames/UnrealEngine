// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Linq;
using System.IO;
using System.Diagnostics;
using CSVStats;
using System.Collections;
using System.Collections.Concurrent;
using System.Threading;
using System.Security.Cryptography;

using PerfSummaries;

namespace PerfReportTool
{
	class CachedCsvFile
	{
		enum FileType
		{
			TextCsv,
			BinaryCsv,
			SummaryTableCachePrc
		};
		public CachedCsvFile(string inFilename, bool useCacheFiles, DerivedMetadataMappings inDerivedMetadataMappings, DerivedCsvStatDefinition[] inDerivedCsvStatDefinitions)
		{
			if (inFilename.ToLower().EndsWith(".csv"))
			{
				fileType = FileType.TextCsv;
			}
			else if (inFilename.ToLower().EndsWith(".csv.bin"))
			{
				fileType = FileType.BinaryCsv;
			}
			else if (inFilename.ToLower().EndsWith(".prc"))
			{
				fileType = FileType.SummaryTableCachePrc;
			}
			else
			{
				throw new Exception("File extension not supported for file " + inFilename);
			}
			string cacheFilename = inFilename + ".cache";
			if (useCacheFiles && File.Exists(cacheFilename))
			{
				string[] fileLines = File.ReadAllLines(cacheFilename);

				// Put the stats and metadata lines in the standard order
				if (fileLines.Length >= 3)
				{
					string metadataLine = fileLines[1];
					string statsLine = fileLines[2];
					fileLines[1] = statsLine;
					fileLines[2] = metadataLine;
				}
				dummyCsvStats = CsvStats.ReadCSVFromLines(fileLines, null, 0, true);
			}
			else
			{
				if (fileType == FileType.BinaryCsv)
				{
					dummyCsvStats = CsvStats.ReadBinFile(inFilename, null, 0, true);
				}
				else if (fileType == FileType.TextCsv)
				{
					textCsvLines = File.ReadAllLines(inFilename);
					if (textCsvLines.Length > 0)
					{
						dummyCsvStats = CsvStats.ReadCSVFromLines(textCsvLines, null, 0, true);
					}
					else
					{
						Console.WriteLine("CSV file " + inFilename + " contains no lines!");
						dummyCsvStats = new CsvStats();
					}
				}
				else if (fileType == FileType.SummaryTableCachePrc)
				{
					// Read just the Csv metadata so we can filter
					metadata = null;
					cachedSummaryTableRowData = SummaryTableRowData.TryReadFromCacheFile(inFilename, true);
					if (cachedSummaryTableRowData != null)
					{
						metadata = cachedSummaryTableRowData.ReadCsvMetadata();
					}
					else
					{
						Console.WriteLine("Invalid PRC file detected: " + inFilename);
					}
				}
			}
			filename = inFilename;

			// Setup initial metadata variable mappings derived metadata (which can read from the initial variableMappings). 
			// Note: we skip for PRCs (dummyCsvStats==null), since these already have mappings cached
			if (dummyCsvStats != null)
			{
				xmlVariableMappings = new XmlVariableMappings();

				// Note: We could apply the global variable set here, but that is slow and leads to complexity and correctness issues due to dependency order
				if (dummyCsvStats.metaData != null)
				{
					metadata = dummyCsvStats.metaData;
					xmlVariableMappings.SetMetadataVariables(metadata);
					inDerivedMetadataMappings.ApplyMapping(metadata, xmlVariableMappings);
				}

				// Add the derived csv stats to dummyStats
				derivedCsvStatDefinitions = inDerivedCsvStatDefinitions;
				foreach (DerivedCsvStatDefinition derivedCsvStatDefinition in derivedCsvStatDefinitions)
				{
					dummyCsvStats.AddStat(new StatSamples(derivedCsvStatDefinition.name, 0, true));
				}
			}
		}

		private void AddDerivedMetadataToCsv(CsvStats csvStats)
		{
			bool bVerboseErrors = CommandLineTool.GetGlobalCommandLine().GetBoolArg("derivedStatVerboseErrors", false);
			foreach (DerivedCsvStatDefinition derivedStat in derivedCsvStatDefinitions)
			{
				finalCsv.AddDerivedStatFromExpression(derivedStat.getExpressionStr(), bVerboseErrors);
			}
		}

		public void PrepareCsvData()
		{
			if (fileType == FileType.BinaryCsv)
			{
				finalCsv = CsvStats.ReadBinFile(filename);
				AddDerivedMetadataToCsv(finalCsv);
				// If we already have metadata then use it. This avoids the need to remap it multiple times
				if (metadata != null)
				{
					finalCsv.metaData = metadata;
				}
			}
			else
			{
				if (textCsvLines == null)
				{
					textCsvLines = File.ReadAllLines(filename);
				}
			}
		}

		public CsvStats GetFinalCsv()
		{
			if (finalCsv != null)
			{
				return finalCsv;
			}
			if (fileType == FileType.TextCsv)
			{
				// TODO: Investigate moving this to the PrepareCsvData function so it can be done in parallel like with binary CSVs
				// Not sure why it's not like that currently - maybe mem usage was the issue, or it was faster for some reason
				finalCsv = CsvStats.ReadCSVFromLines(textCsvLines, null);
				AddDerivedMetadataToCsv(finalCsv);

				// If we already have metadata then use it. This avoids the need to remap it multiple times
				if ( metadata != null )
				{
					finalCsv.metaData = metadata;
				}
				return finalCsv;
			}
			return null;
		}

		public bool DoesMetadataMatchQuery(QueryExpression metadataQuery)
		{
			if (metadataQuery == null)
			{
				return true;
			}
			if (metadata == null)
			{
				Console.WriteLine("CSV " + filename + " has no metadata");
				return false;
			}
			return metadataQuery.Evaluate(metadata);
		}

		public void ComputeSummaryTableCacheId(string reportTypeId)
		{
			// If the CSV has an embedded ID, use that. Otherwise generate one. 
			string csvId;
			if (metadata != null && metadata.Values.ContainsKey("csvid"))
			{
				csvId = metadata.Values["csvid"];
			}
			else
			{
				// Fall back to absolute path if the CSVID metadata doesn't exist
				// Note that using the filename like this means moving the file will result in a new entry for each location
				StringBuilder sb = new StringBuilder();
				sb.Append("CSVFILENAME={" + Path.GetFullPath(filename).ToLower() + "}\n");
				if (metadata != null)
				{
					foreach (string key in metadata.Values.Keys)
					{
						sb.Append("{" + key + "}={" + metadata.Values[key] + "}\n");
					}
				}
				csvId = HashHelper.StringToHashStr(sb.ToString());
			}
			if (reportTypeId == null)
			{
				summaryTableCacheId = csvId;
			}
			else
			{
				summaryTableCacheId = csvId + "_" + reportTypeId;
			}
		}


		public string filename;
		public string summaryTableCacheId;
		public string[] textCsvLines;
		public CsvStats dummyCsvStats;
		public CsvStats finalCsv;
		public CsvMetadata metadata;
		public SummaryTableRowData cachedSummaryTableRowData;
		public ReportTypeInfo reportTypeInfo;
		public XmlVariableMappings xmlVariableMappings;
		public DerivedCsvStatDefinition[] derivedCsvStatDefinitions;

		FileType fileType;
	};

	class ExternalMetadata : Dictionary<string, string>
	{

	}

	class ExternalMetadataCache
	{
		public ExternalMetadataCache(List<string> inExternalMetadataSourceStrings)
		{
			externalMetadataSourceStrings = inExternalMetadataSourceStrings;
		}
		public void InjectExternalMetadata(SummaryTableRowData summaryTableRowData)
		{
			foreach (string filename in externalMetadataSourceStrings)
			{
				string resolvedFilename = ResolveExternalMetadataFilename(filename, summaryTableRowData);
				if (resolvedFilename == null)
				{
					continue;
				}
				ExternalMetadata fileDict = LoadExternalMetadataFile(resolvedFilename);
				foreach (KeyValuePair<string, string> entry in fileDict)
				{
					// Add metadata if it doesn't already exist
					if (!summaryTableRowData.Contains(entry.Key.ToLower()))
					{ 
						summaryTableRowData.Add(SummaryTableElement.Type.ExternalMetadata, entry.Key, entry.Value);
					}
				}
			}
		}

		ExternalMetadata LoadExternalMetadataFile(string filename)
		{
			lock (cachedExternalMetadataFiles)
			{
				if (cachedExternalMetadataFiles.TryGetValue(filename, out ExternalMetadata cachedDict))
				{
					return cachedDict;
				}
			}
			ExternalMetadata dict = new ExternalMetadata();
			if (System.IO.File.Exists(filename))
			{
				string[] lines = System.IO.File.ReadAllLines(filename);
				foreach (string line in lines)
				{
					int idx = line.IndexOf('=');
					if (idx == -1)
					{
						break;
					}
					string key = line[..idx];
					string value = line[(idx + 1)..];
					dict[key] = value;
				}
			}
			// Add to the cache if it wasn't added while we were reading the file
			lock (cachedExternalMetadataFiles)
			{
				if (!cachedExternalMetadataFiles.ContainsKey(filename))
				{
					cachedExternalMetadataFiles[filename] = dict;
				}				
			}			
			return dict;
		}


		// Converts an external metadata file path (including {} metadata entries) to an actual filename by resolving the metadata entries
		// Returns null on failure
		static string ResolveExternalMetadataFilename(string sourceFilename, SummaryTableRowData summaryTableRowData)
		{
			string filenameOut = "";
			// Replace {meta} entries with actual metadata
			int readIndex = 0;
			int startIndex = sourceFilename.IndexOf('{');
			while (startIndex >= 0)
			{
				int endIndex = sourceFilename.IndexOf('}', startIndex);
				if (endIndex == -1)
				{
					throw new Exception("Warning: bad external metadata filename (mismatched {): " + sourceFilename);
				}
				string key = sourceFilename[(startIndex+1)..endIndex].ToLowerInvariant();
				
				SummaryTableElement sourceElement = summaryTableRowData.Get(key);
				if (sourceElement == null || sourceElement.type != SummaryTableElement.Type.CsvMetadata)
				{
					// If we fail to find the metadata, that's probably fine. Just return null. Ignore other types
					return null;
				}
				string value = sourceElement.value;
				filenameOut += sourceFilename[readIndex..startIndex] + value;
				readIndex = endIndex+1;

				startIndex = sourceFilename.IndexOf('{', endIndex);
			}
			filenameOut += sourceFilename[readIndex..];
			return filenameOut;
		}


		Dictionary<string, ExternalMetadata> cachedExternalMetadataFiles = new Dictionary<string, ExternalMetadata>();
		List<string> externalMetadataSourceStrings;
	};

	class CsvFileCache
	{
		public CsvFileCache(
			string[] inCsvFilenames,
			int inLookaheadCount,
			int inThreadCount,
			bool inUseCacheFiles,
			QueryExpression inMetadataQuery,
			ReportXML inReportXml,
			ReportTypeParams inReportTypeParams,
			bool inBulkMode,
			bool inSummaryTableCacheOnlyMode,
			bool inSummaryTableCacheUseOnlyCsvID,
			bool inRemoveDuplicates,
			bool inRequireMetadata,
			string inSummaryTableCacheDir,
			bool inListFilesMode,
			ExternalMetadataCache inExternalMetadataCache)
		{
			csvFileInfos = new CsvFileInfo[inCsvFilenames.Length];
			for (int i = 0; i < inCsvFilenames.Length; i++)
			{
				csvFileInfos[i] = new CsvFileInfo(inCsvFilenames[i]);
			}
			fileCache = this;
			writeIndex = 0;
			useCacheFiles = inUseCacheFiles;
			readIndex = 0;
			lookaheadCount = inLookaheadCount;
			countFreedSinceLastGC = 0;
			reportXml = inReportXml;
			bulkMode = inBulkMode;
			summaryTableCacheOnlyMode = inSummaryTableCacheOnlyMode;
			summaryTableCacheUseOnlyCsvID = inSummaryTableCacheUseOnlyCsvID;
			metadataQuery = inMetadataQuery;
			derivedMetadataMappings = inReportXml.derivedMetadataMappings;
			summaryTableCacheDir = inSummaryTableCacheDir;
			reportTypeParams = inReportTypeParams;
			bRemoveDuplicates = inRemoveDuplicates;
			bRequireMetadata = inRequireMetadata;
			bListFilesMode = inListFilesMode;
			derivedCsvStatDefinitions = reportXml.derivedCsvStatDefinitions.Values.ToArray();

			csvIdToFilename = new Dictionary<string, string>();
			outCsvQueue = new BlockingCollection<CsvFileInfo>(lookaheadCount);

			externalMetadataCache = inExternalMetadataCache;

			// Kick off the workers (must be done last)
			if (inLookaheadCount > 0)
			{
				Console.WriteLine("Kicking off " + inThreadCount + " precache threads with lookahead " + lookaheadCount);
				precacheThreads = new Thread[inThreadCount];
				precacheJobs = new ThreadStart[inThreadCount];
				for (int i = 0; i < precacheThreads.Length; i++)
				{
					precacheJobs[i] = new ThreadStart(PrecacheThreadRun);
					precacheThreads[i] = new Thread(precacheJobs[i]);
					precacheThreads[i].Start();
				}
			}
		}

		public CachedCsvFile GetNextCachedCsvFile()
		{
			CachedCsvFile file = null;
			if (readIndex >= csvFileInfos.Length)
			{
				// We're done
				return null;
			}

			// Find the next valid fileinfo
			if (precacheThreads == null)
			{
				CsvFileInfo fileInfo = csvFileInfos[readIndex];
				file = new CachedCsvFile(fileInfo.filename, useCacheFiles, derivedMetadataMappings, derivedCsvStatDefinitions);
				if (!file.DoesMetadataMatchQuery(metadataQuery)) // TODO do we need to check this here and in the thread?
				{
					return null;
				}
			}
			else
			{
				CsvFileInfo fileInfo = outCsvQueue.Take();
				file = fileInfo.cachedFile;
				fileInfo.cachedFile = null;
				countFreedSinceLastGC++;
				// Periodically GC
				if (countFreedSinceLastGC > 16 && summaryTableCacheOnlyMode == false)
				{
					GC.Collect();
					GC.WaitForPendingFinalizers();
					countFreedSinceLastGC = 0;
				}
				if (!fileInfo.isValid)
				{
					file = null;
				}
			}
			readIndex++;
			return file;
		}

		static void PrecacheThreadRun()
		{
			fileCache.ThreadRun();
		}

		void ThreadRun()
		{
			int threadWriteIndex = 0;
			while (true)
			{
				threadWriteIndex = Interlocked.Increment(ref writeIndex) - 1;
				if (threadWriteIndex >= csvFileInfos.Length)
				{
					// We're done
					break;
				}
				CsvFileInfo fileInfo = csvFileInfos[threadWriteIndex];

				// Process the file
				CachedCsvFile file = new CachedCsvFile(fileInfo.filename, useCacheFiles, derivedMetadataMappings, derivedCsvStatDefinitions);

				bool bProcessThisFile = true;
				if (bRequireMetadata && file.metadata == null)
				{
					Console.WriteLine("CSV has no metadata. Skipping: " + fileInfo.filename);
					bProcessThisFile = false;
				}
				else if (bRemoveDuplicates && file.metadata != null && file.metadata.Values.ContainsKey("csvid"))
				{
					string csvId = file.metadata.Values["csvid"];
					lock (csvIdToFilename)
					{
						if (csvIdToFilename.ContainsKey(csvId))
						{
							Console.WriteLine("Duplicate CSV found: " + fileInfo.filename);
							Console.WriteLine("   First version   : " + csvIdToFilename[csvId]);
							bProcessThisFile = false;
							duplicateCount++;
						}
						else
						{
							csvIdToFilename[csvId] = fileInfo.filename;
						}
					}
				}

				if (bProcessThisFile && file.DoesMetadataMatchQuery(metadataQuery))
				{
					if (bListFilesMode)
					{
						Console.WriteLine("File: " + fileInfo.filename);
					}
					else if (summaryTableCacheOnlyMode)
					{
						// Assume the report type is valid if we have cached metadata, since we don't actually need the reporttype in metadata only mode
						if (file.cachedSummaryTableRowData != null)
						{
							// Just read the full cached metadata
							file.cachedSummaryTableRowData = SummaryTableRowData.TryReadFromCacheFile(fileInfo.filename);
							fileInfo.isValid = true;
						}
					}
					else
					{
						file.reportTypeInfo = GetCsvReportTypeInfo(file, bulkMode);
						if (file.reportTypeInfo != null)
						{
							string reportTypeHash = summaryTableCacheUseOnlyCsvID ? null : file.reportTypeInfo.GetSummaryTableCacheID();
							file.ComputeSummaryTableCacheId(reportTypeHash);
							if (summaryTableCacheDir != null)
							{
								// If a summary metadata cache is specified, try reading from it instead of reading the whole CSV
								// Note that this will be disabled if we're not in bulk mode
								file.cachedSummaryTableRowData = SummaryTableRowData.TryReadFromCache(summaryTableCacheDir, file.summaryTableCacheId);
								if (file.cachedSummaryTableRowData == null)
								{
									Console.WriteLine("Failed to read summary metadata from cache for CSV: " + fileInfo.filename);
								}
							}
							if (file.cachedSummaryTableRowData == null)
							{
								file.PrepareCsvData();
							}

							// Only read the full file data if the metadata matches
							fileInfo.isValid = true;
						}
					}
					// Inject external metadata into the cached row data
					if (file.cachedSummaryTableRowData != null)
					{
						externalMetadataCache.InjectExternalMetadata(file.cachedSummaryTableRowData);
					}

				}
				fileInfo.cachedFile = file;
				// TODO: only add valid files to the queue? Need alternative stopping condition
				outCsvQueue.Add(fileInfo);
			}
		}

		ReportTypeInfo GetCsvReportTypeInfo(CachedCsvFile csvFile, bool bBulkMode)
		{
			try
			{
				return reportXml.GetReportTypeInfo(reportTypeParams.reportTypeOverride, csvFile, bBulkMode, reportTypeParams.forceReportType);
			}
			catch (Exception e)
			{
				if (bBulkMode)
				{
					Console.Error.WriteLine("[ERROR] : " + e.Message);
					return null;
				}
				else
				{
					// If we're not in bulk mode, exceptions are fatal
					throw;
				}
			}
		}


		ThreadStart[] precacheJobs;
		Thread[] precacheThreads;
		int writeIndex;
		int readIndex;
		int lookaheadCount;
		int countFreedSinceLastGC;
		bool useCacheFiles;
		bool bulkMode;
		bool bRequireMetadata;
		bool summaryTableCacheOnlyMode;
		bool summaryTableCacheUseOnlyCsvID;
		QueryExpression metadataQuery;
		string summaryTableCacheDir;
		ReportXML reportXml;
		ReportTypeParams reportTypeParams;
		DerivedMetadataMappings derivedMetadataMappings;
		DerivedCsvStatDefinition[] derivedCsvStatDefinitions;

		Dictionary<string, string> csvIdToFilename;
		public int duplicateCount = 0;
		bool bRemoveDuplicates;
		bool bListFilesMode;

		static CsvFileCache fileCache;
		BlockingCollection<CsvFileInfo> outCsvQueue;

		class CsvFileInfo
		{
			public CsvFileInfo(string inFilename)
			{
				filename = inFilename;
				cachedFile = null;
				isValid = false;
			}
			public CachedCsvFile cachedFile;
			public string filename;
			public bool isValid;
		}
		CsvFileInfo[] csvFileInfos;

		ExternalMetadataCache externalMetadataCache;
	}
}