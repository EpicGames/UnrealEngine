<?xml version="1.0" encoding="UTF-8"?>

<!-- Please use the XML schemas in "Engine/Source/Programs/CsvTools/PerfReportTool/XmlSchemas" to validate -->

<!-- This structure describes how to graph out the different graphs specified in ReportTypes.xml
       The base settings are the main settings that all of the other graphs use unless they specifically override them.
       The main base settings are:
         - maxY:           This is the maximum value that the graph will represent, so make sure they covers the whole dataset.
         - stacked:        Indicates whether the output graph will be stacked.
         - hideStatPrefix: This indicates the prefix that should be hidden in the graph legend.
         
       The graph node can take the same attributes as the baseSettings node (if it does it will override them, otherwise, it will use the baseSettings settings.
       The graph has a title attribute, this is the title of the graph, this MUST match the appropriate graph node title in ReportTypes.xml.
       The graph also has a subnode called statString, this is the string that is passed to the CSVToSVG tool, therefore * will graph all of the stats in the CSV,
       while something like "name1 name2" will graph only the two stats with headings name1 and name2.
       
  -->
<graphGroups xmlns="https://www.unrealengine.com/PerfReportTool" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance">
	<baseSettings thickness="1" smooth="0" smoothKernelSize="-1" smoothKernelPercent="1" compression="0.1"
				  width="1800" height="600" stacked="0" maxHierarchyDepth="2" hideStatPrefix="" showEvents="MapSetup*"/> 
 
	<!-- a graph group is a convenient way to apply common settings to multiple graphs, avoiding duplication -->
	<graphGroup name="LLMBase">
		<baseSettings maxHierarchyDepth="4" stacked="0" hideStatPrefix="Meshes/;Audio/;" requiresDetailedStats="0" thickness="100" compression="0.05" />

		<graph title="Total">
			<statString>Total,Untracked,TrackedTotal</statString> 
		</graph>

		<!-- ignoreStats should include all parent LLM tags -->
		<graph title="Memory Allocated" stacked="1" thickness="100" maxHierarchyDepth="1" ignoreStats="TrackedTotal;Audio;Meshes" mainStat="Total">
			<statString>*</statString>
		</graph>

		<!-- top level LLM summary graphs -->
		<graph title="Content Summary" stacked="1" thickness="100" maxHierarchyDepth="-1">
			<statString>Shaders,Materials,MaterialInstance,Meshes/*,Audio/AudioSoundWaves,Animation</statString>
		</graph>
		<graph title="Misc Summary">
			<statString>EngineInit,EnginePreInit,EngineMisc,TaskGraphMiscTasks,FName,GC,Localization,AssetRegistry,ConfigSystem</statString>
		</graph>
		<graph title="Rendering Summary">
			<statString>RenderingThread,RenderTargets,PSO,SceneRender,RHIMisc,Graphics</statString>
		</graph>
		<graph title="Development Summary">
			<statString>Stats,CsvProfiler</statString>
		</graph>	

		<!-- general graphs -->
		<graph title="Tracked Total" maxY="8000">
			<statString>TrackedTotal</statString>
		</graph>
		<graph title="Untagged" maxY="1000">
			<statString>Untagged</statString>
		</graph>
		<graph title="Untracked">
			<statString>Untracked</statString>
		</graph>
		<graph title="ProgramSize">
			<statString>ProgramSize</statString>
		</graph>
		<graph title="EnginePreInitMemory" maxY="350">
			<statString>EnginePreInit</statString>
		</graph>
		<graph title="Stats" maxY="15">
			<statString>Stats</statString>
		</graph>
		<graph title="UObject" maxY="1000">
			<statString>UObject</statString>
		</graph>
		<graph title="RHI Misc" maxY="250">
			<statString>RHIMisc</statString>
		</graph>
		<graph title="Shaders" maxY="500">
			<statString>Shaders</statString>
		</graph>
		<graph title="Render Targets" maxY="1400">
			<statString>RenderTargets</statString>
		</graph>
		<graph title="Textures" maxY="1400">
			<statString>Textures</statString>
		</graph>
		<graph title="AsyncLoading" maxY="300">
			<statString>AsyncLoading</statString>
		</graph>
		<graph title="Materials" maxY="1000">
			<statString>Materials</statString>
		</graph>
		<graph title="Audio">
			<statString>Audio</statString>
		</graph>
		<graph title="Audio Detail">
			<statString>Audio/*</statString>
		</graph>
		<graph title="Meshes Detail">
			<statString>Meshes/*</statString>
		</graph>
		<graph title="EngineInitMemory" maxY="100">
			<statString>EngineInit</statString>
		</graph>
		<graph title="Animation" maxY="50">
			<statString>Animation</statString>
		</graph>
		<graph title="RenderingThreadMemory" maxY="200">
			<statString>RenderingThread</statString>
		</graph>
		<graph title="LoadMapMemory" maxY="100">
			<statString>LoadMapMisc</statString>
		</graph>
		<graph title="PhysX" maxY="100">
			<statString>PhysX</statString>
		</graph>
		<graph title="EngineMisc" maxY="250">
			<statString>EngineMisc</statString>
		</graph>
		<graph title="StreamingManager" maxY="25">
			<statString>StreamingManager</statString>
		</graph>
	</graphGroup>

	<graphGroup name="LLMPlatformBase">
		<baseSettings maxHierarchyDepth="4" maxY="1024" stacked="0" hideStatPrefix="" requiresDetailedStats="0" thickness="100" compression="0.05" />

		<graph title="Platform FMalloc" maxY="2500">
			<statString>FMalloc</statString>
		</graph>
		<graph title="Platform GenericPlatformMallocCrash" maxY="5">
			<statString>GenericPlatformMallocCrash</statString>
		</graph>
		<graph title="Platform Graphics" maxY="2000">
			<statString>Graphics</statString>
		</graph>
		<graph title="Platform Thread Stack" maxY="10">
			<statString>ThreadStack</statString>
		</graph>
	</graphGroup>
</graphGroups>
