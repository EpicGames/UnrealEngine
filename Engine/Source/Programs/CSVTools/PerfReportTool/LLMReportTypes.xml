<?xml version="1.0" encoding="UTF-8"?>

<!-- Please use the XML schemas in "Engine/Source/Programs/CsvTools/PerfReportTool/XmlSchemas" to validate -->

<!-- The report type works by having a summary type and different graphs.
       The summary type must match one of the types in PerfReportTool, currently we only have:
         - Peak
         - Histogram
       
       To create a new type just subclass Summary and override its functions. See PeakSummary or HistogramSummary as examples.
       
       Then each graph has a title, budget, inSummary, inMainSummary and ignoreList.
        - title:         Used to link the graph to the graph in ReportGraphs.xml, so they must be the same.
        - budget:        This is the value that we are going to compare to. How much we can use.
        - inSummary:     This indicated whether this should be included in the HTML report produced.
        - ignoreList:    This is mainly used with wildcards, if you have stats in the CSV you want ignored for this graph, add them to this list.
                         It's a semi-colon separated list as such: "name1;name2;..."
       
       NOTE: Each graph needs an entry in the appropriate graphGroup node in ReportGraphs.xml, otherwise the program crashes.
  -->
<root xmlns="https://www.unrealengine.com/PerfReportTool" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance">

	<summaryTables>
		<summaryTable name="default" rowSort="CSV File">
			<filter>Total Max,Untracked Avg,CSV File,Report,frameCount,</filter>
		</summaryTable>

	</summaryTables>

	<reporttypes reportGraphsFile="LLMReportGraphs.xml">
 
		<!-- LLM Report -->
		<reporttype name="LLM" title="LLM Report" vsync="0" ignoreList="">

			<autodetection requiredstats="EngineInit"> 
			</autodetection>
 
			<summary type="peak"/>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="4500"    inSummary="1" />
			<graph title="Content Summary"             budget="0"       inSummary="0"/>
			<graph title="Rendering Summary"           budget="0"       inSummary="0"/>
			<graph title="Misc Summary"                budget="0"       inSummary="0"/>
			<graph title="Development Summary"         budget="0"       inSummary="0"/>
			
			<graph title="Tracked Total"               budget="4000"    inSummary="0"/> 
			<!-- Sum of below -->
			<graph title="Animation"                   budget="0"       inSummary="1"/>
			<graph title="AsyncLoading"                budget="0"       inSummary="1"/>
			<graph title="Audio"                       budget="0"       inSummary="1"/>
			<graph title="Audio Detail"                budget="0"       inSummary="0"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="0"       inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="0"       inSummary="1"/>
			<graph title="EngineMisc"                  budget="0"       inSummary="1"/>
			<graph title="LoadMapMemory"               budget="0"       inSummary="1"/>
			<graph title="Materials"                   budget="0"       inSummary="1"/>
			<graph title="PhysX"                       budget="0"       inSummary="1"/>
			<graph title="ProgramSize"                 budget="0"       inSummary="1"/>
			<graph title="Render Targets"              budget="0"       inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="0"       inSummary="1"/>
			<graph title="RHI Misc"                    budget="0"       inSummary="1"/>
			<graph title="Shaders"                     budget="0"       inSummary="1"/>
			<graph title="Stats"                       budget="0"       inSummary="1"/>
			<graph title="StreamingManager"            budget="0"       inSummary="1"/>
			<graph title="Textures"                    budget="0"       inSummary="1"/>
			<graph title="UObject"                     budget="0"       inSummary="1"/>
			<graph title="Untagged"                    budget="0"       inSummary="1"/>
			
			<graph title="Memory Allocated"            budget="0"       inSummary="0"/>
		</reporttype>


		<!-- LLM Report -->
		<reporttype name="LLMPlatformBase" title="LLM Platform Base Report" vsync="0" ignoreList="">
			<autodetection requiredstats="LLMOverhead">
			</autodetection>
			
			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<graph title="Total"                                budget="4500"     inSummary="1"/>
			<graph title="Memory Allocated"                     budget="4500"     inSummary="0"/>
			<graph title="Untagged"                             budget="0"        inSummary="1"/>
			<graph title="Platform Graphics"                    budget="0"        inSummary="1"/>
			<graph title="ProgramSize"                          budget="0"        inSummary="1"/>
			<graph title="Platform FMalloc"                     budget="0"        inSummary="1"/>
			<graph title="Platform GenericPlatformMallocCrash"  budget="0"        inSummary="1"/>
			<graph title="Platform Thread Stack"                budget="0"        inSummary="1"/>

		</reporttype>

	</reporttypes>
</root>