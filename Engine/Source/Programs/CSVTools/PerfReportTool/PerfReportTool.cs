// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Diagnostics;
using CSVStats;
using System.Collections;
using System.Security.Cryptography;
using System.Threading.Tasks;
using System.Threading;

using PerfSummaries;
using CSVTools;

using System.Text.Json;
using EpicGames.Analytics.PerformanceTrends;

namespace PerfReportTool
{
	class Version
	{
		// Format: Major.Minor.Bugfix
		private static string VersionString = "4.274.0";

		public static string Get() { return VersionString; }
	};

	[Flags]
	enum SummaryTableOutputFormat
	{
		Html		= 1 << 0,
		Csv			= 1 << 1,
		Json		= 1 << 2
	}

	class HashHelper
	{
		public static string StringToHashStr(string strIn, int maxCharsOut = -1)
		{
			HashAlgorithm algorithm = SHA256.Create();
			StringBuilder sb = new StringBuilder();

			byte[] hash = algorithm.ComputeHash(Encoding.UTF8.GetBytes(strIn));
			StringBuilder sbOut = new StringBuilder();
			foreach (byte b in hash)
			{
				sbOut.Append(b.ToString("X2"));
			}
			string strOut = sbOut.ToString();
			if (maxCharsOut > 0)
			{
				return strOut.Substring(0, maxCharsOut);
			}
			return strOut;
		}
	}

	class SummaryTableCacheStats
	{
		public int WriteCount = 0;
		public int HitCount = 0;
		public int MissCount = 0;
		public int PurgeCount = 0;

		public void LogStats()
		{
			Console.WriteLine("Summary Table Cache stats:");
			Console.WriteLine("  Cache hits      : " + HitCount);
			Console.WriteLine("  Cache misses    : " + MissCount);
			Console.WriteLine("  Cache writes    : " + WriteCount);
			if (PurgeCount > 0)
			{
				Console.WriteLine("  Files purged    : " + PurgeCount);
			}
			if (HitCount > 0 || MissCount > 0)
			{
				Console.WriteLine("  Hit percentage  : " + ((float)HitCount * 100.0f / ((float)MissCount + (float)HitCount)).ToString("0.0") + "%");
			}
		}
	};

	class Program : CommandLineTool
	{
		static string formatString =
			"PerfReportTool v" + Version.Get() + "\n" +
			"\n" +
			"Format: \n" +
			"  -csv <filename> or -csvdir <directory path> or -summaryTableCacheIn <directory path> or\n" +
			"  -csvList <comma separated> or -prcList <comma separated>\n" +
			"  -o <dir name>: output directory (will be created if necessary)\n" +
			"\n" +
			"Optional Args:\n" +
			"  -reportType <e.g. flythrough, playthrough, playthroughmemory>\n" +
			"  -reportTypeCompatCheck : do a compatibility if when specifying a report type (rather than forcing)\n" +
			"  -graphXML <xmlfilename>\n" +
			"  -reportXML <xmlfilename>\n" +
			"  -reportxmlbasedir <folder>\n" +
			"  -title <name> - title for detailed reports\n" +
			"  -summaryTitle <name> - title for summary tables\n" +
			"  -maxy <value> - forces all graphs to use this value\n" +
			"  -writeSummaryCsv : if specified, a csv file containing summary information will be generated.\n" +
			"     Not available in bulk mode.\n" +
			"  -noWatermarks : don't embed the commandline or version in reports\n" +
			"  -cleanCsvOut <filename> : write a standard format CSV after event stripping with metadata stripped out.\n" +
			"     Not available in bulk mode.\n" +
			"  -noSmooth : disable smoothing on all graphs\n" +
			"  -listSummaryTables: lists available summary tables from the current report XML\n" +
			"\n" +
			"Performance args:\n" +
			"  -perfLog : output performance logging information\n" +
			"  -graphThreads : use with -batchedGraphs to control the number of threads per CsvToSVG instance \n" +
			"                  (default: PC core count/2)\n" +
			"  -csvToSvgSequential : Run CsvToSvg sequentially\n" +
			"  -useEmbeddedGraphUrl : Insert a script to fetch the graph from the specified endpoint on page load rather than\n"+
			"                         embedding the full graph\n" +
			"  -embeddedGraphUrlRoot : The url to fetch the graph from if -useEmbeddedGraphUrl is specified. CsvToSvg graph args\n"+
			"                          are provided as get params\n" +
			"Deprecated performance args:\n" +
			"  -csvToSvgProcesses : Use separate processes for csvToSVG instead of threads (slower)\n" +
			"  -embedGraphCommandline : if -csvToSvgProcesses is specified, embeds the commandline for debugging purposes\n" +
			"  -noBatchedGraphs : disable batched/multithreaded graph generation (use with -csvToSvgProcesses. Default is enabled)\n" +
			"\n" +
			"Options to truncate or filter source data:\n" +
			"Warning: these options disable Summary Table caching\n" +
			"  -minx <frameNumber>\n" +
			"  -maxx <frameNumber>\n" +
			"  -beginEvent <event> : strip data before this event\n" +
			"  -endEvent <event> : strip data after this event\n" +
			"  -noStripEvents : if specified, don't strip out samples between excluded events from the stats\n" +
			"\n" +
			"Optional bulk mode args: (use with -csvdir, -summaryTableCacheIn, -csvList, -prcList)\n" +
			"  -recurse \n" +
			"  -searchpattern <pattern>, e.g -searchpattern csvprofile*\n" +
			"  -summaryTableOutputFormats <html|csv|json>\n : determines the output format(s) of the summary tables (default=html)" +
			"  -customTable <comma separated fields> - sets or overrides the summary table metrics filter\n" +
			"  -customTableSort <comma separated field row sort order> - overrides the summary table row sort\n" +
			"  -noDetailedReports : skips individual report generation\n" +
			"  -noReports : skips generating reports\n" +
			"  -collateTable : writes a collated table in addition to the main one, merging by row sort\n" +
			"  -collateTableOnly : as -collateTable, but doesn't write the standard summary table.\n" +
			"  -emailTable : writes a condensed email-friendly table (see the 'condensed' summary table)\n" +
			"  -csvTable : (Deprecated, use -summaryTableOutputFormats csv) writes the summary table in CSV format instead of html\n" +
			"  -summaryTableXML <XML filename>\n" +
			"  -summaryTable <name> :\n" +
			"     Selects a custom summary table type from the list in reportTypes.xml \n" +
			"     (if not specified, 'default' will be used)\n" +
			"  -condensedSummaryTable <name> :\n" +
			"     Selects a custom condensed summary table type from the list in reportTypes.xml \n" +
			"     (if not specified, 'condensed' will be used)\n" +
			"  -summaryTableFilename <name> : use the specified filename for the summary table (instead of SummaryTable.html)\n" +
			"  -metadataFilter <query> or <key0=value0,key1=value1...>: filters based on CSV metadata,\n" +
			"     e.g \"platform=ps4 AND deviceprofile=ps4_60\" \n" +
			"  -spreadsheetfriendly: outputs a single quote before non-numeric entries in summary tables\n" +
			"  -noSummaryMinMax: don't make min/max columns for each stat in a condensed summary\n" +
			"  -reverseTable [0|1]: Reverses the order of summary tables (set 0 to force off)\n" +
			"  -sortTrailingDigitsAsNumeric : detects trailing digits and pads them when sorting summary table\n" +
			"  -scrollableTable [0|1]: makes the summary table scrollable, with frozen first rows and columns (set 0 to force off)\n" +
			"  -colorizeTable <off|budget|auto>: selects the table colorization mode. If omitted, uses the default in the summary\n" +
			"     xml table if set.\n" +
			"  -maxSummaryTableStringLength <n>: strings longer than this will get truncated\n" +
			"  -allowDuplicateCSVs : doesn't remove duplicate CSVs (Note: can cause summary table cache file locking issues)\n" +
			"  -requireMetadata : ignores CSVs without metadata\n" +
			"  -listFiles : just list all files that pass the metadata query. Don't generate any reports.\n" +
			"  -reportLinkRootPath <path> : Make report links relative to this\n" +
			"  -csvLinkRootPath <path> : Make CSV file links relative to this\n" +
			"  -linkTemplates : insert templates in place of relative links that can be replaced later\n" +
			"     e.g {{LinkTemplate:Report:<CSV ID>}}\n" +
			"  -weightByColumn : weight collated table averages by this column (overrides value specified in the report XML)\n" +
			"  -noWeightedAvg : Don't use weighted averages for the collated table\n" +
			"  -collateAverageMethod <mean,median>\n" +
			"  -minFrameCount <n> : ignore CSVs without at least this number of valid frames\n" +
			"  -maxFileAgeDays <n> : max file age in days. CSV or PRC files older than this will be ignored\n" +
			"  -summaryTableStatThreshold <n> : stat/metric columns in the summarytable will be filtered out if all values are\n" +
			"     less than the threshold\n" +
			"  -summaryTableXmlSubst <find1>=<replace1>;<find2>=<replace2>... : replace summarytable XML row and filter entries\n" +
			"                                                                   Note: one to many is supported (using , separators)\n" +
			"  -summaryTableXmlAppend <list,of,stats> : append these stats to the summary table's filter list\n" +
			"  -summaryTableXmlRowSortAppend <list,of,stats> : append these stats to the summary table's row sort list\n" +
			"  -transposeTable : write the summary tables transposed\n" +
			"  -transposeCollatedTable : write the collated summary table transposed (disables min/max columns)\n" +
			"  -externalMetadataSources <list of paths>: specify paths for external metadata key=value text files\n" +
			"	   these resolve from existing metadata to form the final path. e.g D:\\ExtMeta\\{{platform}}\\{{buildversion}}.txt\n" +
			"  -collatedStringVisibility <show|hide|auto> : sets visibility of strings in collated summary tables (default=auto)\n" +
			"  -collatedDateVisibility <hide|newest|oldest> : sets visibility of dates in collated summary tables (default=newest)\n" +
			"  -columnSortMode <default|sortByMax|sortByAvg|wildCardSortByMax|wildCardSortByAvg> : sorting mode for columns\n" +
			"  -summaryTableJsData : embeds javascript row and column data in the <script> section to help with interop\n" +
			"  -derivedStatVerboseErrors : output verbose errors for derived stat expressions\n" +
			"\n" +
			"CSV stat filtering" +
			"  -readStatAvg or -readAllStats(legacy) : exposes CSV stat averages to the summary tables (not just those exposed\n"+
			"         via report summaries)\n" +
			"  -readStatMin : reads stat min values from CSVs and exposes them to summary tables\n" +
			"  -readStatMax : reads stat max values from CSVs and exposes them to summary tables\n" +
			"  -readStatMinMax : equivalent to -readStatMin -readStatMax\n" +
			"  -showHiddenStats : shows stats which have been automatically hidden (typically duplicate csv unit stats)\n" +
			"  Note: by default, min/max values are not processed, so can't be used in summary tables generated from PRCs unless"+
			"        -readStatMin/Max/MinMax was specified when the PRCs were generated originally"+
			"\n" +
			"Diff rows\n" +
			"  -addDiffRows : adds diff rows after the first two rows\n" +
			"  -sortColumnsByDiff : sorts columns by the max of its diff scores (use with -addDiffRows).\n" +
			"      Notes: Diff score corresponds to the value of a column's diff row; the sign is reversed if LowIsBad\n" +
			"             Stats are sorted by prefixes first, e.g GPU/\n" +
			"  -columnDiffDisplayThreshold <value> : if specified, hides columns with max diff value below this threshold\n" +
			"  -diffRowsAlternating 0|1. If enabled, every other column has a diff. Otherwise diffs are only generated for pairs\n" +
			"  -showOnlyDiffRows: If enabled, non-diff rows are hidden. Common text fields are combined.\n" +
			"\n" +
			"Optional Column Filters\n" +
			"  -hideMetadataColumns : filters out metadata columns from the table (excluding those used in row sort).\n" +
			"\n" +
			"Regression Column Filtering\n" +
			"  -onlyShowRegressedColumns : enables regression filtering. Only shows columns where the most recent row group\n" +
			"    (see -regressionJoinRowsByName) is outside the given stddiv threshold from the mean of the previous rows.\n" +
			"  -regressionJoinRowsByName <statName> : a stat name to join rows by for aggregation. (default: no aggregation)\n" +
			"  -regressionStdDevThreshold <n> (default = 2) : the stddiv threshold for filtering \n" +
			"  -regressionOutlierStdDevThreshold <n> (default = 4) : stddiv threshold for outliers (these are ignored)\n" +
			"\n" +
			"Json serialization:\n" +
			"  -summaryTableToJson <path> : path (usually a json filename) to write summary table row data to. This will write out the full csvs as json.\n" +
			"								Use `-summaryTableOutputFormats json` to output a filtered summary table containing only the stats in the provided summary table.\n" +
			"  -summaryTableToJsonSeparateFiles : writes separate files. -summaryTableToJson specifies the directory name\n" +
			"  -summaryTableToJsonFastMode : exit after serializing json data (skips making summary tables).\n" +
			"  -summaryTableToJsonWriteAllElementData : write all element data, including tooltips, flags\n" +
			"  -summaryTableToJsonMetadataOnly : only write CsvMetadata elements to json\n" +
			"  -summaryTableToJsonFileStream : use a file stream to write Json. Experimental but can avoid OOMs (also applies to `-summaryTableOutputFormats json`)\n" +
			"  -summaryTableToJsonNoIndent : don't indent json output files (also applies to `-summaryTableOutputFormats json`)\n" +
			"  -jsonToPrcs <json filename> : write PRCs. PRC files will be written to -summaryTableCache folder\n" +
			"Telemetry Emission:\n" +
			"  -summaryTableDataToTelemetry : publish the summary table as a telemetry event.\n" +
			"  -summaryTableDataToTelemetryVerbose : publish the verbose form of every row data (without summary table filter).\n" +
			"     Will also emit summary table data.\n" +
			"  -telemetrySessionLabel <label>: \n" +
			"     Optional label to include on all telemetry emissions.\n" +
			"  -telemetryTestIdentity <testIdentity>: \n" +
			"     Optional data source to include on all telemetry emissions.\n" +
			"\n" +
			"Variable dumping\n" +
			"  -dumpVariables | -dumpVariablesAll : dumps variables to the log for each CSV (all includes metadata)\n" +
			"  -dumpVariablesToJson | -dumpAllVariablesToJson <path> : path (usually a json filename) to write variables\n" +
			"                         (like budgets) to json (all includes metadata)\n" +
			"  -metadataProxy : when dumping variables can be used in place of a csv, proxy data can be specified in a\n" +
			"                   csvMetadataProxies section in report XML\n" +
			"  -overrideMetadata : when dumping variables, use to specify/override metadata\n" +
			"                      e.g: -overrideMetadata platform=abc,targetframerate=60\n" +
			"\n" +
			"Performance args for bulk mode:\n" +
			"  -precacheCount <n> : number of CSV files to precache in the lookahead cache (0 for no precache)\n" +
			"  -precacheThreads <n> : number of threads to use for the CSV lookahead cache (default 8)\n" +
			"  -summaryTableCache <dir> : specifies a directory for summary table data to be cached.\n" +
			"     This avoids processing csvs on subsequent runs when -noDetailedReports is specified\n" +
			"  -summaryTableCacheInvalidate : regenerates summary table disk cache entries (ie write only)\n" +
			"  -summaryTableCacheReadOnly : only read from the cache, never write\n" +
			"  -summaryTableCachePurgeInvalid : Purges invalid PRCs from the cache folder\n" +
			"  -summaryTableCacheIn <dir> : reads data directly from the summary table cache instead of from CSVs\n" +
			"  -summaryTableCacheUseOnlyCsvID : only use the CSV ID for the summary table cacheID, ignoringthe report type hash\n" +
			"     Use this if you want to avoid cache data being invalidated by report changes\n" +
			"  -noCsvCacheFiles: disables usage of .csv.cache files. Cache files can be much faster if filtering on metadata\n" +
			"";

		Dictionary<string, StatDisplayInfo> statDisplayNameMapping;
		ReportXML reportXML;

		string GetBaseDirectory()
		{
			string location = System.Reflection.Assembly.GetEntryAssembly().Location.ToLower();
			return Path.GetDirectoryName(location);
		}

		void Run(string[] args)
		{
			System.Globalization.CultureInfo.DefaultThreadCurrentCulture = System.Globalization.CultureInfo.InvariantCulture;

			// Read the command line
			if (args.Length < 1)
			{
				WriteLine("Invalid args");
				WriteLine(formatString);
				return;
			}
			WriteLine("PerfReportTool v" + Version.Get());

			ReadCommandLine(args);
			PerfLog perfLog = new PerfLog(GetBoolArg("perfLog"));

			SummaryFactory.Init();

			// Handle converting json to PRCs if requested
			string jsonToPrcFilename = GetArg("jsonToPrcs", null);
			if (jsonToPrcFilename != null)
			{
				string prcOutputDir = GetArg("summaryTableCache", null);
				if (prcOutputDir == null)
				{
					throw new Exception("-jsonToPRCs requires -summaryTableCache!");
				}
				ConvertJsonToPrcs(jsonToPrcFilename, prcOutputDir);
				return;
			}

			// Load the report + graph XML data
			// Do before commandline processing, -metadataProxyName needs info from report XML
			reportXML = new ReportXML(
				GetArg("graphxml", false),
				GetArg("reportxml", false),
				GetArg("reportxmlbasedir", false),
				GetArg("summaryTableXml", false),
				GetArg("summaryTableXmlSubst", false),
				GetArg("summaryTableXmlAppend", false),
				GetArg("summaryTableXmlRowSortAppend", false)
				);

			bool cleanupTempCsv = false;
			string tempCsv = Path.GetTempPath() + "dummy.csv";
			string csvDir = null;

			bool bBulkMode = false;
			bool bSummaryTableCacheOnlyMode = false;
			// Read CSV filenames from a directory or list
			string[] csvFilenames;
			if (args.Length == 1)
			{
				// Simple mode: just pass a csv name
				csvFilenames = new string[] { args[0] };
			}
			else
			{
				csvDir = GetArg("csvDir");
				int maxFileAgeDays = GetIntArg("maxFileAgeDays", -1);
				string summaryTableCacheInDir = GetArg("summaryTableCacheIn");
				string csvListStr = GetArg("csvList");
				string prcListStr = GetArg("prcList");
				string metadataProxyName = GetArg("metadataProxy");
				string explicitMetadata = GetArg("overrideMetadata");
				if (csvDir.Length > 0)
				{
					bool recurse = GetBoolArg("recurse");
					string searchPattern = GetArg("searchPattern", false);
					if (searchPattern == "")
					{
						searchPattern = "*.csv;*.csv.bin";
					}
					else if (!searchPattern.Contains('.'))
					{
						searchPattern += ".csv;*.csv.bin";
					}

					System.IO.FileInfo[] files = GetFilesWithSearchPattern(csvDir, searchPattern, recurse, maxFileAgeDays);
					csvFilenames = new string[files.Length];
					int i = 0;
					foreach (FileInfo csvFile in files)
					{
						csvFilenames[i] = csvFile.FullName;
						i++;
					}
					// We don't write summary CSVs in bulk mode
					bBulkMode = true;
					perfLog.LogTiming("DirectoryScan");
				}
				else if (summaryTableCacheInDir.Length > 0)
				{
					bool recurse = GetBoolArg("recurse");
					System.IO.FileInfo[] files = GetFilesWithSearchPattern(summaryTableCacheInDir, "*.prc", recurse, maxFileAgeDays);
					csvFilenames = new string[files.Length];
					int i = 0;
					foreach (FileInfo csvFile in files)
					{
						csvFilenames[i] = csvFile.FullName;
						i++;
					}
					bBulkMode = true;
					bSummaryTableCacheOnlyMode = true;
					perfLog.LogTiming("DirectoryScan");
				}
				else if (csvListStr.Length > 0)
				{
					csvFilenames = csvListStr.Split(',');
					bBulkMode = true;
				}
				else if (prcListStr.Length > 0)
				{
					csvFilenames = prcListStr.Split(',');
					bBulkMode = true;
					bSummaryTableCacheOnlyMode = true;
				}
				else if (metadataProxyName.Length > 0 || explicitMetadata.Length > 0)
				{
					string proxyMetadata = "";

					// Look up a set of meta data from the proxy name if supplied
					if (metadataProxyName.Length > 0)
					{
						proxyMetadata = reportXML.GetMetadataProxyInfo(metadataProxyName);
						if (proxyMetadata.Length == 0)
						{
							Console.WriteLine("Warning: Unable to find metadataProxy named " + metadataProxyName + " in report xml csvMetadataProxies section.");
						}
						else if (explicitMetadata.Length > 0)
						{
							proxyMetadata += ",";
						}
					}

					// Append with any supplied metadata from commandline, see -overrideMetadata
					proxyMetadata += explicitMetadata;

					// Process each key=value pair, later entries will override previous ones
					Dictionary<string, string> metaDict = new Dictionary<string, string>();
					if (proxyMetadata.Length > 0)
					{
						// This allows flexibility on the commandline, can use commas or semicolons (and spaces since they are mapped to semicolons in GetArg)
						string[] entries = proxyMetadata.Split(',', ';');
						foreach (string entry in entries)
						{
							string[] data = entry.Split("=");
							if (data.Length == 2)
							{
								metaDict[data[0]] = data[1];
							}
						}
					}

					// If we have a set of proxy metadata, save it out to a dummy.csv for processing
					if (metaDict.Count > 0)
					{
						string fakeValues = "FrameTime\n1";
						string metaDataString = "";
						string leadingComma = "";
						foreach (var (key, value) in metaDict)
						{
							metaDataString += leadingComma;
							metaDataString += "[" + key + "]," + value;
							leadingComma = ",";
						}

						csvFilenames = new string[] { tempCsv };
						string[] lines = { fakeValues, metaDataString };
						File.Delete(csvFilenames[0]);
						File.AppendAllLines(csvFilenames[0], lines);
						cleanupTempCsv = true;
					}
					else
					{
						throw new Exception("Valid -metadataProxy and/or -overrideMetadata not found.");
					}
				}
				else
				{
					string csvFilenamesStr = GetArg("csv");
					if (csvFilenamesStr.Length == 0)
					{
						csvFilenamesStr = GetArg("csvs", true);
						if (csvFilenamesStr.Length == 0)
						{
							if (!GetBoolArg("listSummaryTables"))
							{
								System.Console.Write(formatString);
								return;
							}
						}
					}
					csvFilenames = csvFilenamesStr.Split(';');
				}
			}

			if (GetBoolArg("listSummaryTables"))
			{
				Console.WriteLine("Listing summary tables:");
				List<string> summaryTableNames = reportXML.GetSummaryTableNames();

				foreach (string name in summaryTableNames)
				{
					Console.WriteLine("  " + name);
				}
				return;
			}
			statDisplayNameMapping = reportXML.GetDisplayNameMapping();

			// If we're outputting row data to json, create the dict
			SummaryTableDataJsonWriteHelper summaryTableJsonHelper = null;
			string summaryJsonOutPath = GetArg("summaryTableToJson", null);

			bool summaryTableDataToTelemetryVerbose = GetBoolArg("summaryTableDataToTelemetryVerbose");
			bool summaryTableDataToTelemetry = GetBoolArg("summaryTableDataToTelemetry") || summaryTableDataToTelemetryVerbose;
			string telemetrySessionLabel = GetArg("telemetrySessionLabel", null);
			string telemetryTestIdentity = GetArg("telemetryTestIdentity", null);

			if (summaryJsonOutPath != null)
			{
				summaryTableJsonHelper = new SummaryTableDataJsonWriteHelper(summaryJsonOutPath,
					GetBoolArg("summaryTableToJsonSeparateFiles"),
					GetBoolArg("summaryTableToJsonMetadataOnly"),
					GetBoolArg("summaryTableToJsonWriteAllElementData"),
					!GetBoolArg("summaryTableToJsonNoIndent"));
			}

			SummaryTableCacheStats summaryTableCacheStats = new SummaryTableCacheStats();

			perfLog.LogTiming("Initialization");

			string summaryTableCacheDir = null;
			if (bBulkMode)
			{
				summaryTableCacheDir = GetArg("summaryTableCache", null);
				if (summaryTableCacheDir != null)
				{
					// Check for incompatible options. Could just feed these into the metadata key eventually
					string incompatibleOptionsStr = "minx,maxx,beginevent,endevent,noStripEvents";
					string[] incompatibleOptionsList = incompatibleOptionsStr.Split(',');
					List<string> badOptions = new List<string>();
					foreach (string option in incompatibleOptionsList)
					{
						if (GetArg(option, null) != null)
						{
							badOptions.Add(option);
						}
					}
					if (badOptions.Count > 0)
					{
						Console.WriteLine("Warning: Summary Table cache disabled due to incompatible options (" + string.Join(", ", badOptions) + "). See help for details.");
						summaryTableCacheDir = null;
					}
					else
					{
						Console.WriteLine("Using summary table cache: " + summaryTableCacheDir);
						Directory.CreateDirectory(summaryTableCacheDir);

						if (GetBoolArg("summaryTableCachePurgeInvalid"))
						{
							Console.WriteLine("Purging invalid data from the summary table cache.");
							DirectoryInfo di = new DirectoryInfo(summaryTableCacheDir);
							FileInfo[] files = di.GetFiles("*.prc", SearchOption.TopDirectoryOnly);
							int numFilesDeleted = 0;
							foreach (FileInfo file in files)
							{
								if (SummaryTableRowData.TryReadFromCache(summaryTableCacheDir, file.Name.Substring(0, file.Name.Length - 4)) == null)
								{
									File.Delete(file.FullName);
									numFilesDeleted++;
								}
							}
							summaryTableCacheStats.PurgeCount = numFilesDeleted;
							Console.WriteLine(numFilesDeleted + " of " + files.Length + " cache entries deleted");
							perfLog.LogTiming("PurgeSummaryTableCache");
						}
					}
				}
			}

			// Create the output directory if requested
			string outputDir = GetArg("o", false);
			if (!string.IsNullOrEmpty(outputDir))
			{
				if (!Directory.Exists(outputDir))
				{
					Directory.CreateDirectory(outputDir);
				}
			}

			int precacheCount = GetIntArg("precacheCount", 8);
			int precacheThreads = GetIntArg("precacheThreads", 8);
			bool bBatchedGraphs = true;
			if (GetBoolArg("noBatchedGraphs"))
			{
				bBatchedGraphs = false;
			}
			if (bBatchedGraphs)
			{
				WriteLine("Batched graph generation enabled.");
			}
			else
			{
				WriteLine("Batched graph generation disabled.");
			}

			// Read the metadata filter string
			string metadataFilterString = GetArg("metadataFilter", null);
			QueryExpression metadataQuery = null;
			if (metadataFilterString != null)
			{
				metadataQuery = MetadataQueryBuilder.BuildQueryExpressionTree(metadataFilterString);
			}

			bool writeDetailedReports = !GetBoolArg("noDetailedReports");
			bool writeReports = !GetBoolArg("noReports");
			bool bReadStatAverages = GetBoolArg("readStatAvg") || GetBoolArg("readAllStats");
			bool bReadStatMin = GetBoolArg("readStatMin") || GetBoolArg("readStatMinMax");
			bool bReadStatMax = GetBoolArg("readStatMax") || GetBoolArg("readStatMinMax");

			bool bSummaryTableCacheReadonly = GetBoolArg("summaryTableCacheReadOnly");
			bool bSummaryTableCacheInvalidate = GetBoolArg("summaryTableCacheInvalidate");
			string cleanCsvOutputFilename = GetArg("cleanCsvOut", null);
			if (cleanCsvOutputFilename != null && bBulkMode)
			{
				throw new Exception("-cleanCsvOut is not compatible with bulk mode. Pass one csv with -csv <filename>");
			}

			bool bShowHiddenStats = GetBoolArg("showHiddenStats");
			string customSummaryTableFilter = GetArg("customTable", null);
			if (customSummaryTableFilter != null)
			{
				bShowHiddenStats = true;
			}

			string summaryTableCacheForRead = summaryTableCacheDir;
			if (bSummaryTableCacheInvalidate || writeDetailedReports)
			{
				// Don't read from the summary metadata cache if we're generating full reports
				summaryTableCacheForRead = null;
			}

			if (bSummaryTableCacheOnlyMode)
			{
				// Override these options in summaryTableCacheOnly mode
				bSummaryTableCacheReadonly = true;
				summaryTableCacheForRead = null;
				bSummaryTableCacheInvalidate = false;
			}

			ReportTypeParams reportTypeParams = new ReportTypeParams
			{
				reportTypeOverride = GetArg("reportType", false).ToLower(),
				forceReportType = !GetBoolArg("reportTypeCompatCheck")
			};

			bool bRemoveDuplicates = !GetBoolArg("allowDuplicateCSVs");
			bool bSummaryTableCacheUseOnlyCsvID = GetBoolArg("summaryTableCacheUseOnlyCsvID");
			bool bRequireMetadata = GetBoolArg("requireMetadata");
			bool bListFilesMode = GetBoolArg("listFiles");
			int frameCountThreshold = GetIntArg("minFrameCount", 0);
			if (bListFilesMode)
			{
				writeDetailedReports = false;
			}

			List<string> externalMetadataSources = GetListArg("externalMetadataSources");
			ExternalMetadataCache externalMetadataCache = new ExternalMetadataCache(externalMetadataSources);

			CsvFileCache csvFileCache = new CsvFileCache(
				csvFilenames,
				precacheCount,
				precacheThreads,
				!GetBoolArg("noCsvCacheFiles"),
				metadataQuery,
				reportXML,
				reportTypeParams,
				bBulkMode,
				bSummaryTableCacheOnlyMode,
				bSummaryTableCacheUseOnlyCsvID,
				bRemoveDuplicates,
				bRequireMetadata,
				summaryTableCacheForRead,
				bListFilesMode,
				externalMetadataCache);

			SummaryTable summaryTable = new SummaryTable();
			bool bWriteToSummaryTableCache = summaryTableCacheDir != null && !bSummaryTableCacheReadonly;

			int csvCount = csvFilenames.Length;
			for (int i = 0; i < csvCount; i++)
			{
				try
				{
					CachedCsvFile cachedCsvFile = csvFileCache.GetNextCachedCsvFile();
					if (cachedCsvFile == null)
					{
						continue;
					}
					Console.WriteLine("-------------------------------------------------");
					Console.WriteLine("CSV " + (i + 1) + "/" + csvFilenames.Length);
					Console.WriteLine(cachedCsvFile.filename);

					perfLog.LogTiming("  CsvCacheRead");
					if (cachedCsvFile == null)
					{
						Console.WriteLine("Skipped!");
					}
					else
					{
						SummaryTableRowData rowData = cachedCsvFile.cachedSummaryTableRowData;
						if (rowData == null)
						{
							if (summaryTableCacheForRead != null)
							{
								summaryTableCacheStats.MissCount++;
							}
							if (bBulkMode)
							{
								rowData = new SummaryTableRowData();
							}
							if (cleanCsvOutputFilename != null)
							{
								WriteCleanCsv(cachedCsvFile, cleanCsvOutputFilename, cachedCsvFile.reportTypeInfo);
								perfLog.LogTiming("  WriteCleanCsv");
							}
							else
							{
								if (writeReports)
								{
									GenerateReport(cachedCsvFile, outputDir, bBulkMode, rowData, bBatchedGraphs, writeDetailedReports, bReadStatMin, bReadStatMax, cachedCsvFile.reportTypeInfo, csvDir);
									perfLog.LogTiming("  GenerateReport");
								}

								if (cachedCsvFile.xmlVariableMappings != null)
								{
									if ((GetBoolArg("dumpVariables") || GetBoolArg("dumpVariablesAll")))
									{
										Console.WriteLine("\nDumping variables for " + cachedCsvFile.filename + "\n");
										cachedCsvFile.xmlVariableMappings.DumpToLog(GetBoolArg("dumpVariablesAll"));
									}

									string variablesJsonOutPath = GetArg("dumpVariablesToJson", null);
									if (variablesJsonOutPath != null)
									{
										cachedCsvFile.xmlVariableMappings.SerializeToJson(variablesJsonOutPath, "", "meta.");
									}

									string allVariablesJsonOutPath = GetArg("dumpAllVariablesToJson", null);
									if (allVariablesJsonOutPath != null)
									{
										cachedCsvFile.xmlVariableMappings.SerializeToJson(allVariablesJsonOutPath, "", "");
									}

								}

								if (rowData != null)
								{
									// Inject external metadata if we have it
									externalMetadataCache.InjectExternalMetadata(rowData);
									if (bWriteToSummaryTableCache)
									{
										if (rowData.WriteToCache(summaryTableCacheDir, cachedCsvFile.summaryTableCacheId))
										{
											Console.WriteLine("Cached summary rowData for CSV: " + cachedCsvFile.filename);
											summaryTableCacheStats.WriteCount++;
											perfLog.LogTiming("  WriteSummaryTableCache");
										}
									}
								}
							}
						}
						else
						{
							summaryTableCacheStats.HitCount++;
						}

						if (rowData != null)
						{
							// Filter row based on framecount if minFrameCount is specified
							bool bIncludeRowData = true;
							if (frameCountThreshold > 0 && rowData.GetFrameCount() < frameCountThreshold)
							{
								Console.WriteLine("CSV frame count below the threshold. Excluding from summary table:" + cachedCsvFile.filename);
								bIncludeRowData = false;
							}
							if (bIncludeRowData)
							{
								summaryTable.AddRowData(rowData, bReadStatAverages, bReadStatMin, bReadStatMax, bShowHiddenStats);
								if (summaryTableJsonHelper != null)
								{

									summaryTableJsonHelper.AddRowData(rowData);
								}
							}

							// Emit telemetry if enabled
							if (summaryTableDataToTelemetryVerbose)
							{
								Telemetry.EnqueuePostSummaryTableMetricEvent(rowData, telemetrySessionLabel, telemetryTestIdentity);
							}

							perfLog.LogTiming("  AddRowData");
						}
					}
				}
				catch (Exception e)
				{
					if (bBulkMode)
					{
						Console.Error.WriteLine("[ERROR] : " + e.Message);
					}
					else
					{
						// If we're not in bulk mode, exceptions are fatal
						throw;
					}
				}
			}

			if (cleanupTempCsv)
			{
				File.Delete(tempCsv);
			}

			if (summaryTableJsonHelper != null)
			{
				summaryTableJsonHelper.WriteToJson(GetBoolArg("summaryTableToJsonFileStream"));
				perfLog.LogTiming("WriteSummaryDataJson");

				if (GetBoolArg("summaryTableToJsonFastMode"))
				{
					if (summaryTableDataToTelemetry)
					{
						Telemetry.Flush();
						Telemetry.Dispose();
					}

					perfLog.LogTotalTiming();
					return;
				}
			}

			Console.WriteLine("-------------------------------------------------");

			// Write out the summary table, if there is one
			if (summaryTable.Count > 0)
			{
				// Pre-sort the summary table to ensure determinism
				summaryTable = summaryTable.SortRows(new List<string>(new string[] { "csvfilename" }), true);
				perfLog.LogTiming("PreSort Summary table");

				string summaryTableFilename = GetArg("summaryTableFilename", "SummaryTable");
				if (summaryTableFilename.ToLower().EndsWith(".html"))
				{
					summaryTableFilename = summaryTableFilename.Substring(0, summaryTableFilename.Length - 5);
				}
				bool bCollateTable = GetBoolArg("collateTable");
				bool bCollateTableOnly = GetBoolArg("collateTableOnly");

				SummaryTableOutputFormat outputFormats = GetEnumArg<SummaryTableOutputFormat>("summaryTableOutputFormats", SummaryTableOutputFormat.Html);
				if (GetBoolArg("csvTable"))
				{
					// Force exclusive for backwards compatiblity with previous behavior.
					outputFormats = SummaryTableOutputFormat.Csv;
				}

				bCollateTable |= bCollateTableOnly;
				string collatedTableFilename = summaryTableFilename + (bCollateTableOnly ? "" : "_Collated");

				bool bSpreadsheetFriendlyStrings = GetBoolArg("spreadsheetFriendly");
				string weightByColumnName = GetArg("weightByColumn", null);

				string customSummaryTableRowSort = GetArg("customTableSort", null);
				string summaryTableName = GetArg("summaryTable", null);
				SummaryTableInfo tableInfo = GetTableInfo(summaryTableName, customSummaryTableFilter, customSummaryTableRowSort);
				if (!bCollateTableOnly)
				{
					WriteSummaryTableReport(outputDir, summaryTableFilename, summaryTable, tableInfo, false, outputFormats, bSpreadsheetFriendlyStrings, null, telemetrySessionLabel, telemetryTestIdentity);
				}
				if (bCollateTable)
				{
					WriteSummaryTableReport(outputDir, collatedTableFilename, summaryTable, tableInfo, true, outputFormats, bSpreadsheetFriendlyStrings, weightByColumnName, telemetrySessionLabel, telemetryTestIdentity);
				}

				// EmailTable is hardcoded to use the condensed type
				string condensedSummaryTableName = GetArg("condensedSummaryTable", null);
				if (GetBoolArg("emailSummary") || GetBoolArg("emailTable") || condensedSummaryTableName != null)
				{
					if (condensedSummaryTableName == null)
					{
						condensedSummaryTableName = "condensed";
					}
					tableInfo = GetTableInfo(condensedSummaryTableName, customSummaryTableFilter, customSummaryTableRowSort);
					WriteSummaryTableReport(outputDir, summaryTableFilename + "_Email", summaryTable, tableInfo, true, SummaryTableOutputFormat.Html, bSpreadsheetFriendlyStrings, weightByColumnName);
				}
				perfLog.LogTiming("WriteSummaryTable");
			}

			if (summaryTableDataToTelemetry)
			{
				Telemetry.Flush();
				Telemetry.Dispose();
			}

			if (summaryTableCacheDir != null)
			{
				summaryTableCacheStats.LogStats();
			}
			Console.WriteLine("Duplicate CSVs skipped: " + csvFileCache.duplicateCount);
			perfLog.LogTotalTiming();
		}

		SummaryTableInfo GetTableInfo(string summaryTableName, string customSummaryTableFilterString = null, string customSummaryTableRowSortString = null)
		{
			SummaryTableInfo tableInfo;

			// Use the default summary table name if one isn't provided, unless we're making a custom table			
			if (summaryTableName == null && customSummaryTableFilterString == null)
			{
				summaryTableName = "default";
			}

			if (summaryTableName != null)
			{
				tableInfo = reportXML.GetSummaryTable(summaryTableName);
			}
			else
			{
				tableInfo = new SummaryTableInfo();
			}
			if (customSummaryTableFilterString != null)
			{
				tableInfo.columnFilterList = customSummaryTableFilterString.Split(',').ToList();

				// If we're doing a custom sort but don't have a provided row sort (or preset summary table), set a default
				if (summaryTableName == null && customSummaryTableRowSortString == null)
				{
					customSummaryTableRowSortString = "buildversion,deviceprofile";
				}
			}
			if (customSummaryTableRowSortString != null)
			{
				tableInfo.rowSortList = customSummaryTableRowSortString.Split(',').ToList();
			}
			return tableInfo;
		}

		void WriteSummaryTableReport(string outputDir, string filenameWithoutExtension, SummaryTable table, List<string> columnFilterList, List<string> rowSortList, bool bCollated, SummaryTableOutputFormat outputFormats, bool bSpreadsheetFriendlyStrings, List<SummarySectionBoundaryInfo> sectionBoundaries, string weightByColumnName)
		{
			SummaryTableInfo tableInfo = new SummaryTableInfo();
			tableInfo.columnFilterList = columnFilterList;
			tableInfo.rowSortList = rowSortList;
			WriteSummaryTableReport(outputDir, filenameWithoutExtension, table, tableInfo, bCollated, outputFormats, bSpreadsheetFriendlyStrings, weightByColumnName);
		}

		void WriteSummaryTableReport(string outputDir, string filenameWithoutExtension, SummaryTable table, SummaryTableInfo tableInfo, bool bCollated, SummaryTableOutputFormat outputFormats, bool bSpreadsheetFriendlyStrings, string weightByColumnNameOverride, string telemetrySessionLabel = null, string telemetryTestIdentity = null)
		{
			string weightByColumnName = weightByColumnNameOverride != null ? weightByColumnNameOverride : tableInfo.weightByColumn;
			if (GetBoolArg("noWeightedAvg"))
			{
				weightByColumnName = null;
			}

			CollateAverageMethod collateAverageMethod = GetEnumArg<CollateAverageMethod>("collateAverageMethod", tableInfo.collateAverageMethod);

			bool bTransposeFullSummaryTable = GetBoolArg("transposeTable");
			bool bTransposeCollatedSummaryTable = bTransposeFullSummaryTable | GetBoolArg("transposeCollatedTable");

			// Check params and any commandline overrides
			bool bReverseTable = tableInfo.bReverseSortRows;
			bool? bReverseTableOption = GetOptionalBoolArg("reverseTable");
			if (bReverseTableOption != null)
			{
				bReverseTable = (bool)bReverseTableOption;
			}

			bool bScrollableTable = tableInfo.bScrollableFormatting;
			bool? bScrollableTableOption = GetOptionalBoolArg("scrollableTable");
			if (bScrollableTableOption != null)
			{
				bScrollableTable = (bool)bScrollableTableOption;
			}

			// The colorize mode is initially set to whatever is in the summary table xml file.
			// We then override that value if -colorizeTable is set to auto, off or budget.
			// If -colorizeTable isn't specified then we use the default from the xml.
			// If -colorizeTable isn't specified and it's not set in the summary table xml it uses the default set in the class initializer.
			TableColorizeMode colorizeMode = tableInfo.tableColorizeMode;
			if (GetBoolArg("autoColorizeTable")) // Legacy support for the -autoColorizeTable arg
			{
				colorizeMode = TableColorizeMode.Auto;
			}
			colorizeMode = GetEnumArg("colorizeTable", colorizeMode);

			TableColumnSortMode tableColumnSortMode = GetEnumArg("columnSortMode", tableInfo.columnSortMode);

			bool addMinMaxColumns = !GetBoolArg("noSummaryMinMax") && !bTransposeCollatedSummaryTable;

			if (!string.IsNullOrEmpty(outputDir))
			{
				filenameWithoutExtension = Path.Combine(outputDir, filenameWithoutExtension);
			}

			IEnumerable<ISummaryTableColumnFilter> additionalColumnFilters = MakeAdditionalColumnFilters(tableInfo);

			// Set format info for the columns as some of the info is needed for the filters.
			// TODO: would be better if we could determine HighIsBad without the format info and store it directly in the column.
			table.SetColumnFormatInfo(reportXML.columnFormatInfoList);

			bool bSortTrailingDigitsAsNumeric = GetBoolArg("sortTrailingDigitsAsNumeric");
			SummaryTable filteredTable = table.SortAndFilter(tableInfo.columnFilterList, tableInfo.rowSortList, bReverseTable, weightByColumnName, additionalColumnFilters, bSortTrailingDigitsAsNumeric, tableColumnSortMode);
			if (bCollated)
			{
				StringCollationVisibility stringCollationVisibility = GetEnumArg("collatedStringVisibility", tableInfo.stringCollationVisibility);
				DateCollationVisibility dateCollationVisibility = GetEnumArg("collatedDateVisibility", tableInfo.dateCollationVisibility);
				filteredTable = filteredTable.CollateSortedTable(tableInfo.rowSortList, addMinMaxColumns, stringCollationVisibility, dateCollationVisibility, collateAverageMethod);
			}

			if (outputFormats.HasFlag(SummaryTableOutputFormat.Csv))
			{
				filteredTable.WriteToCSV(filenameWithoutExtension + ".csv");
			}

			if (outputFormats.HasFlag(SummaryTableOutputFormat.Html))
			{
				filteredTable.ApplyDisplayNameMapping(statDisplayNameMapping);
				string VersionString = GetBoolArg("noWatermarks") ? "" : Version.Get();
				string summaryTitle = GetArg("summaryTitle", null);
				if (GetBoolArg("addDiffRows"))
				{
					filteredTable.AddDiffRows(GetBoolArg("sortColumnsByDiff"), GetFloatArg("columnDiffDisplayThreshold", 0.0f), GetBoolArg("showOnlyDiffRows"), GetBoolArg("diffRowsAlternating", true));
				}

				// Run again to add format info for any new columns that were added (eg. count).
				filteredTable.SetColumnFormatInfo(reportXML.columnFormatInfoList);

				filteredTable.WriteToHTML(
					filenameWithoutExtension + ".html",
					VersionString,
					bSpreadsheetFriendlyStrings,
					tableInfo.sectionBoundaries,
					bScrollableTable,
					colorizeMode,
					addMinMaxColumns,
					tableInfo.hideStatPrefix,
					GetIntArg("maxSummaryTableStringLength", Int32.MaxValue),
					weightByColumnName,
					collateAverageMethod,
					summaryTitle,
					bCollated ? bTransposeCollatedSummaryTable : bTransposeFullSummaryTable,
					tableInfo.summaryTableStyle,
					GetBoolArg("summaryTableJsData")
				);
			}

			if (outputFormats.HasFlag(SummaryTableOutputFormat.Json))
			{
				Dictionary<string, dynamic> jsonDict = filteredTable.ToFlatJsonDict();
				SummaryTableDataJsonWriteHelper.WriteJsonFile(GetBoolArg("summaryTableToJsonFileStream"), filenameWithoutExtension + ".json", jsonDict, !GetBoolArg("summaryTableToJsonNoIndent"));
			}

			bool summaryTableDataToTelemetry = GetBoolArg("summaryTableDataToTelemetry") || GetBoolArg("summaryTableDataToTelemetryVerbose");

			// Emit telemetry if enabled
			if (summaryTableDataToTelemetry)
			{
				// Reconstitute the table to include our standard metadata columns for each bundle of data.
				SummaryTable reconstitutedTable = table.SortAndFilter(tableInfo.columnFilterList.Union(StandardTelemetryMetadata.StandardMetadataKeys).ToList(), tableInfo.rowSortList, bReverseTable, weightByColumnName, additionalColumnFilters, bSortTrailingDigitsAsNumeric, tableColumnSortMode);

				if (bCollated)
				{
					StringCollationVisibility stringCollationVisibility = GetEnumArg("collatedStringVisibility", tableInfo.stringCollationVisibility);
					DateCollationVisibility dateCollationVisibility = GetEnumArg("collatedDateVisibility", tableInfo.dateCollationVisibility);
					reconstitutedTable = reconstitutedTable.CollateSortedTable(tableInfo.rowSortList, addMinMaxColumns, stringCollationVisibility, dateCollationVisibility, collateAverageMethod);
				}

				string summaryTableName = GetArg("summaryTable", null);
				Telemetry.EnqueuePostSummaryTableMetricEvent(summaryTableName, reconstitutedTable, bCollated, telemetrySessionLabel, telemetryTestIdentity);
			}
		}

		IEnumerable<ISummaryTableColumnFilter> MakeAdditionalColumnFilters(SummaryTableInfo tableInfo)
		{
			List<ISummaryTableColumnFilter> additionalColumnFilters = new List<ISummaryTableColumnFilter>();
			additionalColumnFilters.Add(new StatThresholdColumnFilter(GetFloatArg("summaryTableStatThreshold", tableInfo.statThreshold)));

			if (GetBoolArg("hideMetadataColumns"))
			{
				additionalColumnFilters.Add(new MetadataColumnFilter(tableInfo.rowSortList));
			}

			if (GetBoolArg("onlyShowRegressedColumns"))
			{
				string joinByStatName = GetArg("regressionJoinRowsByName");
				float stdDevThreshold = GetFloatArg("regressionStdDevThreshold", 2.0f);
				float outlierStdDevThreshold = GetFloatArg("regressionOutlierStdDevThreshold", 4.0f);
				additionalColumnFilters.Add(new RegressionColumnFilter(joinByStatName, stdDevThreshold, outlierStdDevThreshold));
			}

			return additionalColumnFilters;
		}

		string ReplaceFileExtension(string path, string newExtension)
		{
			// Special case for .bin.csv
			if (path.ToLower().EndsWith(".csv.bin"))
			{
				return path.Substring(0, path.Length - 8) + newExtension;
			}

			int lastDotIndex = path.LastIndexOf('.');
			if (path.EndsWith("\""))
			{
				newExtension = newExtension + "\"";
				if (lastDotIndex == -1)
				{
					lastDotIndex = path.Length - 1;
				}
			}
			else if (lastDotIndex == -1)
			{
				lastDotIndex = path.Length;
			}

			return path.Substring(0, lastDotIndex) + newExtension;
		}

		static Dictionary<string, bool> UniqueHTMLFilemameLookup = new Dictionary<string, bool>();

		void WriteCleanCsv(CachedCsvFile csvFile, string outCsvFilename, ReportTypeInfo reportTypeInfo)
		{
			if (File.Exists(outCsvFilename))
			{
				throw new Exception("Clean csv file " + outCsvFilename + " already exists!");
			}
			Console.WriteLine("Writing clean (standard format, event stripped) csv file to " + outCsvFilename);
			int minX = GetIntArg("minX", 0);
			int maxX = GetIntArg("maxX", Int32.MaxValue);

			// Check if we're stripping stats
			bool bStripStatsByEvents = reportTypeInfo.bStripEvents;
			if (GetBoolArg("noStripEvents"))
			{
				bStripStatsByEvents = false;
			}

			int numFramesStripped;
			CsvStats csvStatsUnstripped;
			CsvStats csvStats = ProcessCsv(csvFile, out numFramesStripped, out csvStatsUnstripped, minX, maxX, null, bStripStatsByEvents);
			csvStats.WriteToCSV(outCsvFilename, false);
		}

		CsvStats ProcessCsv(CachedCsvFile csvFile, out int numFramesStripped, out CsvStats csvStatsUnstripped, int minX = 0, int maxX = Int32.MaxValue, PerfLog perfLog = null, bool bStripStatsByEvents = true)
		{
			numFramesStripped = 0;
			CsvStats csvStats = ReadCsvStats(csvFile, minX, maxX);
			csvStatsUnstripped = csvStats;
			if (perfLog != null)
			{
				perfLog.LogTiming("    ReadCsvStats");
			}
			if (bStripStatsByEvents)
			{
				CsvStats strippedCsvStats = StripCsvStatsByEvents(csvStatsUnstripped, out numFramesStripped);
				csvStats = strippedCsvStats;
			}
			if (perfLog != null)
			{
				perfLog.LogTiming("    FilterStats");
			}
			return csvStats;
		}

		// Represents how a graph should be written to the file.
		private class CsvSvgInfo
		{
			public enum GraphFormat
			{
				// The full graph html/script is written directly into the report file.
				Inline,
				// A script to fetch the graph dynamically on page load is inserted into the report file.
				Url
			}

			public ReportGraph Graph { get; private set; }
			public GraphFormat Format { get; private set; }
			public string SvgFilename { get; private set; } = null;

			public static CsvSvgInfo CreateInlineGraphInfo(ReportGraph graph, string svgFilename)
			{
				return new CsvSvgInfo()
				{
					Graph = graph,
					Format = GraphFormat.Inline,
					SvgFilename = svgFilename
				};
			}

			public static CsvSvgInfo CreateEmbeddedUrlGraphInfo(ReportGraph graph)
			{
				return new CsvSvgInfo()
				{
					Graph = graph,
					Format = GraphFormat.Url,
					SvgFilename = null
				};
			}
		}

		string ExpandGraphStatListDerivedStats(string originalGraphStatNamesStr, CsvStats csvStats)
		{
			string[] statNames = originalGraphStatNamesStr.ToLowerInvariant().Split(',');
			List<string> newStatNames = new List<string>();
			bool bHasDerivedStats = false;
			for (int i = 0; i < statNames.Length; i++)
			{
				List<StatSamples> matchingStats = csvStats.GetStatsMatchingString(statNames[i]);
				foreach (StatSamples stat in matchingStats)
				{
					DerivedCsvStatDefinition derivedStatDefinition = stat.bIsDerived ? reportXML.GetDerivedCsvStatDefinition(stat.Name) : null;
					if (derivedStatDefinition != null)
					{
						newStatNames.Add(derivedStatDefinition.getExpressionStr());
						bHasDerivedStats = true;
					}
					else
					{
						newStatNames.Add(stat.Name);
					}
				}
			}
			return bHasDerivedStats ? String.Join(',', newStatNames) : originalGraphStatNamesStr;
		}

		void GenerateReport(CachedCsvFile csvFile, 
				string outputDir, 
				bool bBulkMode, 
				SummaryTableRowData rowData, 
				bool bBatchedGraphs, 
				bool writeDetailedReport, 
				bool bReadStatMinValues,
				bool bReadStatMaxValues,
				ReportTypeInfo reportTypeInfo, 
				string csvDir)
		{
			PerfLog perfLog = new PerfLog(GetBoolArg("perfLog"));
			string shortName = ReplaceFileExtension(MakeShortFilename(csvFile.filename), "");
			string title = GetArg("title", false);
			if (title.Length == 0)
			{
				title = shortName;
			}

			char c = title[0];
			c = char.ToUpper(c);
			title = c + title.Substring(1);

			int minX = GetIntArg("minx", 0);
			int maxX = GetIntArg("maxx", Int32.MaxValue);

			string htmlFilename = null;
			if (writeDetailedReport)
			{
				htmlFilename = shortName;
				// Make sure the HTML filename is unique
				if (bBulkMode)
				{
					int index = 0;
					while (UniqueHTMLFilemameLookup.ContainsKey(htmlFilename.Trim().ToLower()))
					{
						if (htmlFilename.EndsWith("]") && htmlFilename.Contains('['))
						{
							int strIndex = htmlFilename.LastIndexOf('[');
							htmlFilename = htmlFilename.Substring(0, strIndex);
						}
						htmlFilename += "[" + index.ToString() + "]";
						index++;
					}
					UniqueHTMLFilemameLookup.Add(htmlFilename.Trim().ToLower(), true);
				}
				htmlFilename += ".html";
			}

			bool bCsvToSvgProcesses = GetBoolArg("csvToSvgProcesses");
			bool bCsvToSvgMultiThreaded = !GetBoolArg("csvToSvgSequential");
			bool useEmbeddedGraphUrl = GetBoolArg("useEmbeddedGraphUrl");

			int minWorker, minIOC;
			// Get the current settings.
			ThreadPool.GetMinThreads(out minWorker, out minIOC);


			float thickness = 1.0f;
			List<string> csvToSvgCommandlines = new List<string>();
			List<CsvSvgInfo> csvSvgInfoList = new List<CsvSvgInfo>();
			string responseFilename = null;
			List<Process> csvToSvgProcesses = new List<Process>();
			List<Task> csvToSvgTasks = new List<Task>();

			if (writeDetailedReport)
			{
				GraphGenerator graphGenerator = null;
				if (!bCsvToSvgProcesses && !useEmbeddedGraphUrl)
				{
					graphGenerator = new GraphGenerator(csvFile.GetFinalCsv(), csvFile.filename);
				}

				// Generate all the graphs asynchronously
				foreach (ReportGraph graph in reportTypeInfo.graphs)
				{
					if (graph.settings.statString.isSet == false)
					{
						throw new Exception("Graph " + graph.title + " has no <statString> element");
					}

					bool bShowGraph = graph.ShouldShow(csvFile.finalCsv ?? csvFile.dummyCsvStats);
					if (bShowGraph)
					{
						// If we're not generating graphs in this process then expand any referenced derived stats with the full expression so that CsvToSvg can compute them itself
						// (We've already computed them in this tool, but CsvToSvg only has the original Csv file)
						if (graphGenerator == null && reportXML.derivedCsvStatDefinitions.Count > 0)
						{
							graph.settings.statString.value = ExpandGraphStatListDerivedStats(graph.settings.statString.value, csvFile.finalCsv ?? csvFile.dummyCsvStats);
						}

						if (useEmbeddedGraphUrl)
						{
							csvSvgInfoList.Add(CsvSvgInfo.CreateEmbeddedUrlGraphInfo(graph));
						}
						else
						{
							string svgFilename = GetTempFilename(csvFile.filename) + ".svg";
							if (graphGenerator != null)
							{
								GraphParams graphParams = GetCsvToSvgGraphParams(graph, thickness, minX, maxX, false, csvSvgInfoList.Count);
								if (bCsvToSvgMultiThreaded)
								{
									csvToSvgTasks.Add(graphGenerator.MakeGraphAsync(graphParams, svgFilename, true, false));
								}
								else
								{
									graphGenerator.MakeGraph(graphParams, svgFilename, true, false);
								}
							}
							else
							{
								string args = GetCsvToSvgArgs(csvFile.filename, svgFilename, graph, thickness, minX, maxX, false, csvSvgInfoList.Count, CsvToSvgArgFormat.CommandLine);
								if (bBatchedGraphs)
								{
									csvToSvgCommandlines.Add(args);
								}
								else
								{
									Process csvToSvgProcess = LaunchCsvToSvgAsync(args);
									csvToSvgProcesses.Add(csvToSvgProcess);
								}
							}

							csvSvgInfoList.Add(CsvSvgInfo.CreateInlineGraphInfo(graph, svgFilename));
						}
					}
				}

				if (bCsvToSvgProcesses && bBatchedGraphs)
				{
					// Save the response file
					responseFilename = GetTempFilename(csvFile.filename) + "_response.txt";
					System.IO.File.WriteAllLines(responseFilename, csvToSvgCommandlines);
					Process csvToSvgProcess = LaunchCsvToSvgAsync("-batchCommands \"" + responseFilename + "\" -mt " + GetIntArg("graphThreads", Environment.ProcessorCount / 2).ToString());
					csvToSvgProcesses.Add(csvToSvgProcess);
				}
			}
			perfLog.LogTiming("    Initial Processing");

			// Check if we're stripping stats
			bool bStripStatsByEvents = reportTypeInfo.bStripEvents;
			if (GetBoolArg("noStripEvents"))
			{
				bStripStatsByEvents = false;
			}

			if (writeDetailedReport && csvToSvgTasks.Count > 0)
			{
				// wait on the graph tasks to complete
				// Note that we have to do this before we can call ProcessCSV, since this modifies the CsvStats object the graph tasks are reading
				foreach (Task task in csvToSvgTasks)
				{
					task.Wait();
				}
				perfLog.LogTiming("    WaitForAsyncGraphs");
			}

			// Read the full csv while we wait for the graph processes to complete (this is only safe for CsvToSVG processes, not task threads)
			int numFramesStripped;
			CsvStats csvStatsUnstripped;
			CsvStats csvStats = ProcessCsv(csvFile, out numFramesStripped, out csvStatsUnstripped, minX, maxX, perfLog, bStripStatsByEvents);

			if (writeDetailedReport && csvToSvgProcesses.Count > 0)
			{
				// wait on the graph processes to complete
				foreach (Process process in csvToSvgProcesses)
				{
					process.WaitForExit();
				}
				perfLog.LogTiming("    WaitForAsyncGraphs");
			}


			// Generate CSV metadata
			if (rowData != null)
			{
				Uri currentDirUri = new Uri(Directory.GetCurrentDirectory() + "/", UriKind.Absolute);
				if (outputDir.Length > 0 && !outputDir.EndsWith("/"))
				{
					outputDir += "/";
				}
				Uri optionalDirUri = new Uri(outputDir, UriKind.RelativeOrAbsolute);

				// Make a Csv URI that's relative to the report directory
				Uri finalDirUri;
				if (optionalDirUri.IsAbsoluteUri)
				{
					finalDirUri = optionalDirUri;
				}
				else
				{
					finalDirUri = new Uri(currentDirUri, outputDir);
				}
				Uri csvFileUri = new Uri(csvFile.filename, UriKind.Absolute);
				Uri relativeCsvUri = finalDirUri.MakeRelativeUri(csvFileUri);
				string csvPath = relativeCsvUri.ToString();

				bool bLinkTemplates = GetBoolArg("linkTemplates");
				string csvId = null;
				if (csvStats.metaData != null)
				{
					csvId = csvStats.metaData.GetValue("csvid", null);
				}

				// re-root the CSV path if requested
				string csvLinkRootPath = GetArg("csvLinkRootPath", null);
				if (csvDir != null && csvLinkRootPath != null)
				{
					string csvDirFinal = csvDir.Replace("\\", "/");
					csvDirFinal += csvDirFinal.EndsWith("/") ? "" : "/";
					Uri csvDirUri = new Uri(csvDirFinal, UriKind.Absolute);
					Uri csvRelativeToCsvDirUri = csvDirUri.MakeRelativeUri(csvFileUri);
					csvPath = Path.Combine(csvLinkRootPath, csvRelativeToCsvDirUri.ToString());
					csvPath = new Uri(csvPath, UriKind.Absolute).ToString();
				}

				string csvLink = "<a href='" + csvPath + "'>" + shortName + ".csv" + "</a>";
				if (bLinkTemplates)
				{
					if (!csvPath.StartsWith("http://") && !csvPath.StartsWith("https://"))
					{
						csvLink = "{LinkTemplate:Csv:" + (csvId ?? "0") + "}";
					}
				}

				rowData.Add(SummaryTableElement.Type.ToolMetadata, "Csv File", csvLink, null, csvPath);
				rowData.Add(SummaryTableElement.Type.ToolMetadata, "ReportType", reportTypeInfo.name);
				rowData.Add(SummaryTableElement.Type.ToolMetadata, "ReportTypeID", reportTypeInfo.summaryTableCacheID);
				if (htmlFilename != null)
				{
					string htmlUrl = htmlFilename;
					string reportLinkRootPath = GetArg("reportLinkRootPath", null);
					if (reportLinkRootPath != null)
					{
						htmlUrl = reportLinkRootPath + htmlFilename;
					}
					string reportLink = "<a href='" + htmlUrl + "'>Link</a>";
					if (bLinkTemplates)
					{
						if (!htmlUrl.StartsWith("http://") && !htmlUrl.StartsWith("https://"))
						{
							reportLink = "{LinkTemplate:Report:" + (csvId ?? "0") + "}";
						}
					}
					rowData.Add(SummaryTableElement.Type.ToolMetadata, "Report", reportLink);
				}
				// Pass through all the metadata from the CSV
				if (csvStats.metaData != null)
				{
					foreach (KeyValuePair<string, string> pair in csvStats.metaData.Values.ToList())
					{
						rowData.Add(SummaryTableElement.Type.CsvMetadata, pair.Key.ToLower(), pair.Value);
					}
				}

				// Add every stat avg value to the metadata, and min/max values if requested
				foreach (StatSamples stat in csvStats.Stats.Values)
				{
					rowData.Add(SummaryTableElement.Type.CsvStatAverage, stat.Name, (double)stat.average);
					if (bReadStatMinValues)
					{
						rowData.Add(SummaryTableElement.Type.CsvStatMin, stat.Name + " Min", (double)stat.ComputeMinValue());
					}
					if (bReadStatMaxValues)
					{
						rowData.Add(SummaryTableElement.Type.CsvStatMax, stat.Name + " Max", (double)stat.ComputeMaxValue());
					}
				}
			}

			if (htmlFilename != null && !string.IsNullOrEmpty(outputDir))
			{
				htmlFilename = Path.Combine(outputDir, htmlFilename);
			}

			// Write the report
			WriteReport(htmlFilename, title, csvSvgInfoList, reportTypeInfo, csvStats, csvStatsUnstripped, numFramesStripped, minX, maxX, bBulkMode, rowData);
			perfLog.LogTiming("    WriteReport");

			// Delete the temp files
			foreach (CsvSvgInfo csvSvgInfo in csvSvgInfoList)
			{
				if (csvSvgInfo.SvgFilename != null && File.Exists(csvSvgInfo.SvgFilename))
				{
					File.Delete(csvSvgInfo.SvgFilename);
				}
			}
			if (responseFilename != null && File.Exists(responseFilename))
			{
				File.Delete(responseFilename);
			}
		}
		CsvStats ReadCsvStats(CachedCsvFile csvFile, int minX, int maxX)
		{
			CsvStats csvStats = csvFile.GetFinalCsv();
			if (csvStats.metaData == null)
			{
				csvStats.metaData = new CsvMetadata();
			}
			csvStats.metaData.Values.Add("csvfilename", csvFile.filename);

			// Adjust min/max x based on the event delimiters
			string beginEventStr = GetArg("beginEvent").ToLower();
			if (beginEventStr != "")
			{
				foreach (CsvEvent ev in csvStats.Events)
				{
					if (ev.Name.ToLower() == beginEventStr)
					{
						minX = Math.Max(minX, ev.Frame);
						break;
					}
				}
			}
			string endEventStr = GetArg("endEvent").ToLower();
			if (endEventStr != "")
			{
				for (int i = csvStats.Events.Count - 1; i >= 0; i--)
				{
					CsvEvent ev = csvStats.Events[i];
					if (ev.Name.ToLower() == endEventStr)
					{
						maxX = Math.Min(maxX, ev.Frame);
						break;
					}
				}
			}

			// Strip out all stats with a zero total
			List<StatSamples> allStats = new List<StatSamples>();
			foreach (StatSamples stat in csvStats.Stats.Values)
			{
				allStats.Add(stat);
			}
			csvStats.Stats.Clear();
			foreach (StatSamples stat in allStats)
			{
				if (stat.total != 0.0f)
				{
					csvStats.AddStat(stat);
				}
			}

			// Crop the stats to the range
			csvStats.CropStats(minX, maxX);
			return csvStats;
		}

		CsvStats StripCsvStatsByEvents(CsvStats csvStats, out int numFramesStripped)
		{
			numFramesStripped = 0;
			List<CsvEventStripInfo> eventsToStrip = reportXML.GetCsvEventsToStrip();
			// We want to run the mask apply in parallel if -nodetailedreports is specified. Otherwise leave cores free for graph generation
			bool doParallelMaskApply = GetBoolArg("noDetailedReports");
			CsvStats strippedStats = csvStats;

			if (eventsToStrip != null)
			{
				BitArray sampleMask = null;
				foreach (CsvEventStripInfo eventStripInfo in eventsToStrip)
				{
					csvStats.ComputeEventStripSampleMask(eventStripInfo.beginName, eventStripInfo.endName, ref sampleMask);
				}
				if (sampleMask != null)
				{
					numFramesStripped = sampleMask.Cast<bool>().Count(l => !l);
					strippedStats = csvStats.ApplySampleMask(sampleMask, doParallelMaskApply);
				}
			}

			if (numFramesStripped > 0)
			{
				Console.WriteLine("CSV frames excluded : " + numFramesStripped);
			}
			return strippedStats;
		}




		void WriteReport(string htmlFilename, string title, List<CsvSvgInfo> csvSvgInfoList, ReportTypeInfo reportTypeInfo, CsvStats csvStats, CsvStats csvStatsUnstripped, int numFramesStripped, int minX, int maxX, bool bBulkMode, SummaryTableRowData summaryRowData)
		{
			string titleStr = reportTypeInfo.title + " : " + title;
			System.IO.StreamWriter htmlFile = null;

			if (htmlFilename != null)
			{
				htmlFile = new System.IO.StreamWriter(htmlFilename);
				htmlFile.WriteLine("<html>");
				htmlFile.WriteLine("  <head>");
				htmlFile.WriteLine("    <meta http-equiv='X-UA-Compatible' content='IE=edge'/>");
				htmlFile.WriteLine("    <meta charset='UTF-8'/>");

				if (GetBoolArg("noWatermarks"))
				{
					htmlFile.WriteLine("    <![CDATA[ \nCreated with PerfReportTool");
				}
				else
				{
					htmlFile.WriteLine("    <![CDATA[ \nCreated with PerfReportTool " + Version.Get() + " with commandline:");
					htmlFile.WriteLine(commandLine.GetCommandLine());
				}
				htmlFile.WriteLine("    ]]>");
				htmlFile.WriteLine("    <title>" + titleStr + "</title>");

				// Scripting for collapsibles
				htmlFile.WriteLine("    <script>");
				htmlFile.WriteLine("        document.addEventListener('DOMContentLoaded', function (event) { setupCollapsibles(); })");

				htmlFile.WriteLine("        function setupCollapsibles() {");
				htmlFile.WriteLine("            var collapsibles = document.getElementsByClassName('collapsibleHeading');");
				htmlFile.WriteLine("            var i;");
				htmlFile.WriteLine("            for (i = 0; i < collapsibles.length; i++) {");
				htmlFile.WriteLine("                collapsibles[i].addEventListener('click', function() {");
				htmlFile.WriteLine("                    this.classList.toggle('expanded');");
				htmlFile.WriteLine("                    this.nextElementSibling.classList.toggle('expanded');");
				htmlFile.WriteLine("             	});");
				htmlFile.WriteLine("            }");
				htmlFile.WriteLine("        }");

				htmlFile.WriteLine("        function collapseAll() {");
				htmlFile.WriteLine("            var collapsibles = document.getElementsByClassName('collapsibleHeading');");
				htmlFile.WriteLine("            var i;");
				htmlFile.WriteLine("            for (i = 0; i < collapsibles.length; i++) {");
				htmlFile.WriteLine("                setSectionExpanded( collapsibles[i], false );");
				htmlFile.WriteLine("            }");
				htmlFile.WriteLine("        }");

				htmlFile.WriteLine("        function expandAll() {");
				htmlFile.WriteLine("            var collapsibles = document.getElementsByClassName('collapsibleHeading');");
				htmlFile.WriteLine("            var i;");
				htmlFile.WriteLine("            for (i = 0; i < collapsibles.length; i++) {");
				htmlFile.WriteLine("                setSectionExpanded( collapsibles[i], true );");
				htmlFile.WriteLine("            }");
				htmlFile.WriteLine("        }");


				htmlFile.WriteLine("        function setSectionExpanded(collapsible, bExpanded) {");
				htmlFile.WriteLine("            if (bExpanded) {");
				htmlFile.WriteLine("                collapsible.classList.add('expanded');");
				htmlFile.WriteLine("                collapsible.nextElementSibling.classList.add('expanded');");
				htmlFile.WriteLine("            }");
				htmlFile.WriteLine("            else {");
				htmlFile.WriteLine("                collapsible.classList.remove('expanded');");
				htmlFile.WriteLine("                collapsible.nextElementSibling.classList.remove('expanded');");
				htmlFile.WriteLine("            }");
				htmlFile.WriteLine("        }");

				htmlFile.WriteLine("        function setSectionExpandedById(id, bExpanded) {");
				htmlFile.WriteLine("            setSectionExpanded(document.getElementById(id), bExpanded);");
				htmlFile.WriteLine("        }");

				htmlFile.WriteLine("    </script>");

				// CSS
				htmlFile.WriteLine("    <style type='text/css'>");

				htmlFile.WriteLine("      table, th, td { border: 2px solid black; border-collapse: collapse; padding: 3px; vertical-align: top; font-family: 'Verdana', Times, serif; font-size: 12px;}");
				htmlFile.WriteLine("      p {  font-family: 'Verdana', Times, serif; font-size: 12px }");
				htmlFile.WriteLine("      ul {  font-family: 'Verdana', Times, serif; font-size: 14px }");
				htmlFile.WriteLine("      h1 {  font-family: 'Verdana', Times, serif; font-size: 20px; padding-top:10px }");
				htmlFile.WriteLine("      h2 {  font-family: 'Verdana', Times, serif; font-size: 18px; padding-top:5px; padding-bottom:0px; margin-block-start: 0.4em; margin-block-end: 0.4em }");
				htmlFile.WriteLine("      h3 {  font-family: 'Verdana', Times, serif; font-size: 16px; padding-top:5px; margin-block-start: 0.3em; margin-block-end: 0.3em }");
				htmlFile.WriteLine("      hr {  margin-top:15px }");
				htmlFile.WriteLine("      a {  font-family: 'Verdana', Times, serif; font-size: 12px }");

				// Collapsibles
				htmlFile.WriteLine("      .collapsibleHeading { background-color: #ffffff; cursor: pointer; width: fit-content; border: none; text-align: left; outline: none; display: flex; justify-content: flex-start; align-items: flex-end; flex-direction: row; flex-wrap: nowrap; }");
				htmlFile.WriteLine("      .expanded { background-color: #fff; }");
				htmlFile.WriteLine("      .collapsibleHeading:hover { background-color: #c4c4c4; }");
				htmlFile.WriteLine("      .collapsibleHeading:after { content: '\u25B2'; font-size: 14px; color: #4d4d4d; font-variant: normal; float: right; margin-left: 14px;}");
				htmlFile.WriteLine("      .collapsibleHeading.expanded:after { content: '\u25BC'; }");
				htmlFile.WriteLine("      .collapsibleSection { display: grid; grid-template-rows: 0fr; transition: grid-template-rows 0.2s ease-out }");
				htmlFile.WriteLine("      .collapsibleSection.expanded { grid-template-rows: 1fr; }");
				htmlFile.WriteLine("      .collapsibleSectionInner { overflow: auto; overflow-y: hidden }");

				// Section anchors
				htmlFile.WriteLine("      .sectionAnchorLink { color:#808080; text-decoration:none;display: inline-block; vertical-align:-4px; margin-right:-2px; margin-left:-2px }");
				htmlFile.WriteLine("      .sectionAnchorText-h1 {font-size:16px}");
				htmlFile.WriteLine("      .sectionAnchorText-h2 {font-size:14px}");
				htmlFile.WriteLine("      .sectionAnchorText-h3 {font-size:12px}");

				htmlFile.WriteLine("    </style>");
				htmlFile.WriteLine("  </head>");  
				htmlFile.WriteLine("  <body>");
				htmlFile.WriteLine("  <h1 name='top'>" + titleStr + "</h1>");

				// show the range
				if (minX > 0 || maxX < Int32.MaxValue)
				{
					htmlFile.WriteLine("<br><br><font size='1.5'>(CSV cropped to range " + minX + "-");
					if (maxX < Int32.MaxValue)
					{
						htmlFile.WriteLine(maxX);
					}
					htmlFile.WriteLine(")</font>");
				}

				// Output the metadata table
				htmlFile.WriteLine("<table style='width:800'>");

				if (reportTypeInfo.metadataToShowList != null)
				{
					Dictionary<string, StatDisplayInfo> displayNameMapping = reportXML.GetDisplayNameMapping();

					foreach (string metadataStr in reportTypeInfo.metadataToShowList)
					{
						string value = csvStats.metaData.GetValue(metadataStr, null);
						if (value != null)
						{
							string friendlyName = metadataStr;
							if (displayNameMapping.TryGetValue(metadataStr.ToLower(), out StatDisplayInfo displayInfo))
							{
								if (displayInfo.displayName != null)
								{
									friendlyName = displayInfo.displayName;
								}
							}
							htmlFile.WriteLine("<tr><td bgcolor='#F0F0F0'>" + friendlyName + "</td><td><b>" + value + "</b></td></tr>");
						}
					}
				}
				htmlFile.WriteLine("<tr><td bgcolor='#F0F0F0'>Frame count</td><td>" + csvStats.SampleCount + " (" + numFramesStripped + " excluded)</td></tr>");
				htmlFile.WriteLine("</table>");

				// Output the top level nav
				htmlFile.WriteLine("<br>");
				htmlFile.WriteLine("<div style='width: 100%; background-color: #ffffff'>");
				htmlFile.WriteLine("<button type='button' onclick='collapseAll()' style='margin-right:10px'>Collapse all</button>");
				htmlFile.WriteLine("<button type='button' onclick='expandAll()' style='margin-right:10px'>Expand all</button>");
				htmlFile.WriteLine("<button type='button' onclick='location.href=\"#graphList\";'>Graphs</button>");
				htmlFile.WriteLine("</div>");
				htmlFile.WriteLine("<br>");
			}

			if (summaryRowData != null)
			{
				summaryRowData.Add(SummaryTableElement.Type.ToolMetadata, "framecount", csvStats.SampleCount.ToString());
				if (numFramesStripped > 0)
				{
					summaryRowData.Add(SummaryTableElement.Type.ToolMetadata, "framecountExcluded", numFramesStripped.ToString());
				}
			}

			bool bWriteSummaryCsv = GetBoolArg("writeSummaryCsv") && !bBulkMode;

			List<Summary> summaries = new List<Summary>(reportTypeInfo.summaries);
			bool bExtraLinksSummary = GetBoolArg("extraLinksSummary");
			if (bExtraLinksSummary)
			{
				bool bLinkTemplates = GetBoolArg("linkTemplates");
				summaries.Insert(0, new ExtraLinksSummary(null, reportTypeInfo.vars, null, bLinkTemplates));
			}
			// Add the standard summary last
			summaries.Add(new StandardSummary());

			// Assign indices to the final summary list
			for (int i=0; i<summaries.Count; i++)
			{
				summaries[i].SetReportSummaryIndex(i);
			}

			// If the reporttype has summary info, then write out the summary]
			foreach (Summary summary in summaries)
			{
				bool bWriteHtml = htmlFile != null && !summary.bHideInDetailedReport;
				HtmlSection htmlSection = summary.WriteSummaryData(bWriteHtml, summary.useUnstrippedCsvStats ? csvStatsUnstripped : csvStats, csvStatsUnstripped, bWriteSummaryCsv, summaryRowData, htmlFilename);
				if (htmlSection != null)
				{
					htmlSection.WriteToFile(htmlFile);
				}
			}

			if (htmlFile != null)
			{
				// Output the list of graphs
				htmlFile.WriteLine("<hr><a name='graphList'></a>");
				htmlFile.WriteLine("<h2>Graphs</h2>");

				// TODO: support sections for graphs
				List<string> sections = new List<string>();

				//// We have to at least have the empty string in this array so that we can print the list of links.
				if (sections.Count() == 0)
				{
					sections.Add("");
				}

				for (int index = 0; index < sections.Count; index++)
				{
					htmlFile.WriteLine("<ul>");
					string currentCategory = sections[index];
					if (currentCategory.Length > 0)
					{
						htmlFile.WriteLine("<h4>" + currentCategory + " Graphs</h4>");
					}

					foreach (CsvSvgInfo csvSvgInfo in csvSvgInfoList)
					{
						string svgTitle = csvSvgInfo.Graph.title;
						// TODO: Check if this graph belongs in this section.
						htmlFile.WriteLine("<li><a href='#" + StripSpaces(svgTitle) + "' onclick='setSectionExpandedById(\"" + StripSpaces(svgTitle) + "\", true)'>" + svgTitle + "</a></li>");
					}

					htmlFile.WriteLine("</ul>");
				}
				htmlFile.WriteLine("<a href='#top'>Back to top \u2191</a>");


				// Output the Graphs
				for (int svgFileIndex = 0; svgFileIndex < csvSvgInfoList.Count; svgFileIndex++)
				{
					CsvSvgInfo csvSvgInfo = csvSvgInfoList[svgFileIndex];
					ReportGraph graph = csvSvgInfo.Graph;

					string svgTitle = graph.title;
					HtmlSection htmlSection = new HtmlSection(svgTitle, false, StripSpaces(svgTitle), 3);

					if (csvSvgInfo.Format == CsvSvgInfo.GraphFormat.Inline)
					{
						string[] svgLines = ReadLinesFromFile(csvSvgInfo.SvgFilename);
						foreach (string line in svgLines)
						{
							string modLine = line.Replace("__MAKEUNIQUE__", "U_" + svgFileIndex.ToString());
							htmlSection.WriteLine(modLine);
						}
					}
					else if (csvSvgInfo.Format == CsvSvgInfo.GraphFormat.Url)
					{
						string graphArgs = GetCsvToSvgArgs(null, null, graph, 1.0, minX, maxX, false, svgFileIndex, CsvToSvgArgFormat.Url);
						string csvId = csvStats.metaData?.GetValue("csvid", null);
						if (csvId == null)
						{
							throw new Exception("Failed to generate embeddedGraphUrl since no valid csvId was found.");
						}

						string graphUrlRoot = GetArg("embeddedGraphUrlRoot", mandatory: true);
						string graphUrl = $"{graphUrlRoot}?csvs={csvId}&{graphArgs}";
						string script = $"<div id=\"graph_{svgFileIndex}\" style=\"width:{graph.settings.width.value}px;height:{graph.settings.height.value}px\"></div>\n";
						script += $"<script>" +
							// On page load, fetch the graph from the end point
							$"fetch('{graphUrl}')\n" +
							".then(response => response.text())\n" +
							".then(html => {\n" +
								// Insert the html into the div
								$"const graphDiv = document.getElementById('graph_{svgFileIndex}');\n" +
								"graphDiv.innerHTML = html;\n" +
								// Find all the nested scripts and make them executable.
								"graphDiv.querySelectorAll('script').forEach((script) => { \n" +
									// We need to copy the script into a new node to make it executable as scripts
									// assigned via innerHTML cannot be run. For some reason the text/innerHTML of scripts nested in an svg tag
									// cannot be read/assigned to a new node directly, so we must serialize the node to xml, then parse that into
									// a local dom tree. From there we can grab the script node and read its text data.
									"const scriptNodeText = new XMLSerializer().serializeToString(script);\n" +
									"const parsedScriptDom = new DOMParser().parseFromString(scriptNodeText, \"text/xml\");\n" +
									"const scriptText = parsedScriptDom.querySelector('script').firstChild.data;" +

									"const clonedScriptNode = document.createElement('script');\n" +
									"clonedScriptNode.text = scriptText;\n" +
									"script.parentNode.replaceChild(clonedScriptNode, script);\n" +
								"});\n" +
							"})\n" +
							".catch(err => console.log(err));" +
							"</script>";
						htmlSection.WriteLine(script);

						string expandedGraphUrl = graphUrl + "&tohtml=true";
						htmlSection.WriteLine($"<a href=\"{expandedGraphUrl}\" style=\"text-decoration:none\">View <b>&#x26F6;</b></a>");
					}
					else
					{
						throw new Exception("Unsupported graph output format.");
					}

					htmlSection.WriteToFile(htmlFile);
				}

				htmlFile.WriteLine("<br><a href='#top'>Back to top \u2191</a>");


				if (GetBoolArg("noWatermarks"))
				{
					htmlFile.WriteLine("<p style='font-size:8'>Created with PerfReportTool</p>");
				}
				else
				{
					htmlFile.WriteLine("<p style='font-size:8'>Created with PerfReportTool " + Version.Get() + "</p>");
				}
				htmlFile.WriteLine("  </body>");
				htmlFile.WriteLine("</html>");
				htmlFile.Close();
				string ForEmail = GetArg("foremail", false);
				if (ForEmail != "")
				{
					WriteEmail(htmlFilename, title, csvSvgInfoList, reportTypeInfo, csvStats, csvStatsUnstripped, minX, maxX, bBulkMode);
				}
			}
		}


		void WriteEmail(string htmlFilename, string title, List<CsvSvgInfo> csvSvgInfoList, ReportTypeInfo reportTypeInfo, CsvStats csvStats, CsvStats csvStatsUnstripped, int minX, int maxX, bool bBulkMode)
		{
			if (htmlFilename == null)
			{
				return;
			}
			ReportGraph[] graphs = reportTypeInfo.graphs.ToArray();
			string titleStr = reportTypeInfo.title + " : " + title;
			System.IO.StreamWriter htmlFile;
			htmlFile = new System.IO.StreamWriter(htmlFilename + "email");
			htmlFile.WriteLine("<html>");
			htmlFile.WriteLine("  <head>");
			htmlFile.WriteLine("    <meta http-equiv='X-UA-Compatible' content='IE=edge'/>");
			if (GetBoolArg("noWatermarks"))
			{
				htmlFile.WriteLine("    <![CDATA[ \nCreated with PerfReportTool");
			}
			else
			{
				htmlFile.WriteLine("    <![CDATA[ \nCreated with PerfReportTool " + Version.Get() + " with commandline:");
				htmlFile.WriteLine(commandLine.GetCommandLine());
			}
			htmlFile.WriteLine("    ]]>");
			htmlFile.WriteLine("    <title>" + titleStr + "</title>");
			htmlFile.WriteLine("  </head>");
			htmlFile.WriteLine("  <body><font face='verdana'>");
			htmlFile.WriteLine("  <h1>" + titleStr + "</h1>");

			// show the range
			if (minX > 0 || maxX < Int32.MaxValue)
			{
				htmlFile.WriteLine("<br><br><font size='1.5'>(CSV cropped to range " + minX + "-");
				if (maxX < Int32.MaxValue)
				{
					htmlFile.WriteLine(maxX);
				}
				htmlFile.WriteLine(")</font>");
			}


			htmlFile.WriteLine("<a href=\"[Report Link Here]\">Click here for Report w/ interactive SVGs.</a>");
			htmlFile.WriteLine("  <h2>Summary</h2>");

			htmlFile.WriteLine("Overall Runtime: [Replace Me With Runtime]");

			bool bWriteSummaryCsv = GetBoolArg("writeSummaryCsv") && !bBulkMode;

			// If the reporttype has summary info, then write out the summary]
			foreach (Summary summary in reportTypeInfo.summaries)
			{
				bool bWriteHtml = htmlFile != null && !summary.bHideInDetailedReport;
				HtmlSection htmlSection = summary.WriteSummaryData(bWriteHtml, csvStats, csvStatsUnstripped, bWriteSummaryCsv, null, htmlFilename);
				if (htmlSection != null)
				{
					htmlSection.WriteToFile(htmlFile);
				}
			}

			htmlFile.WriteLine("  </font></body>");
			htmlFile.WriteLine("</html>");
			htmlFile.Close();

		}
		string StripSpaces(string str)
		{
			return str.Replace(" ", "");
		}

		string GetTempFilename(string csvFilename)
		{
			string shortFileName = MakeShortFilename(csvFilename).Replace(" ", "_");
			return Path.Combine(Path.GetTempPath(), shortFileName + "_" + Guid.NewGuid().ToString().Substring(26));
		}

		enum CsvToSvgArgFormat
		{
			CommandLine,
			Url
		}

		string GetCsvToSvgArgs(string csvFilename, string svgFilename, ReportGraph graph, double thicknessMultiplier, int minx, int maxx, bool multipleCSVs, int graphIndex, CsvToSvgArgFormat argFormat, float scaleby = 1.0f)
		{
			string title = graph.title;

			GraphSettings graphSettings = graph.settings;

			float maxy = GetFloatArg("maxY", (float)graphSettings.maxY.value);
			bool smooth = graphSettings.smooth.value && !GetBoolArg("nosmooth");
			double smoothKernelPercent = graphSettings.smoothKernelPercent.value;
			double smoothKernelSize = graphSettings.smoothKernelSize.value;
			double compression = graphSettings.compression.value;
			int width = graphSettings.width.value;
			int height = graphSettings.height.value;
			bool stacked = graphSettings.stacked.value;
			bool showAverages = graphSettings.showAverages.value;
			bool showTotals = graphSettings.showTotals.value;
			bool filterOutZeros = graphSettings.filterOutZeros.value;
			bool snapToPeaks = false;
			if (graphSettings.snapToPeaks.isSet)
			{
				snapToPeaks = graphSettings.snapToPeaks.value;
			}

			int lineDecimalPlaces = graphSettings.lineDecimalPlaces.isSet ? graphSettings.lineDecimalPlaces.value : 1;
			int maxHierarchyDepth = graphSettings.maxHierarchyDepth.value;
			string hideStatPrefix = graphSettings.hideStatPrefix.value;
			string showEvents = graphSettings.showEvents.value;
			double statMultiplier = graphSettings.statMultiplier.isSet ? graphSettings.statMultiplier.value : 1.0;
			bool hideEventNames = false;
			if (multipleCSVs)
			{
				showEvents = "CSV:*";
				hideEventNames = true;
			}
			bool interactive = true;

			string highlightEventRegions = "";
			if (!GetBoolArg("noStripEvents"))
			{
				List<CsvEventStripInfo> eventsToStrip = reportXML.GetCsvEventsToStrip();
				if (eventsToStrip != null)
				{
					highlightEventRegions += "\"";
					for (int i = 0; i < eventsToStrip.Count; i++)
					{
						if (i > 0)
						{
							highlightEventRegions += ";";
						}
						string endEvent = (eventsToStrip[i].endName == null) ? "{null}" : eventsToStrip[i].endName;
						string beginEvent = (eventsToStrip[i].beginName == null) ? "{null}" : eventsToStrip[i].beginName;
						highlightEventRegions += beginEvent + ";" + endEvent;
					}
					highlightEventRegions += "\"";
				}
			}

			Optional<double> minFilterStatValueSetting = graph.minFilterStatValue.isSet ? graph.minFilterStatValue : graphSettings.minFilterStatValue;

			string Quote(string s)
			{
				return "\"" + s + "\"";
			}

			Dictionary<string, string> args = new();

			void AddOptionalArg<T>(string name, Optional<T> value)
			{
				if (value.isSet)
				{
					args[name] = value.value.ToString();
				}
			}

			void AddConditionalArg<T>(string name, bool condition, T value)
			{
				if (condition)
				{
					args[name] = value.ToString();
				}
			}

			void AddConditionalFlag(string name, bool condition)
			{
				if (condition)
				{
					args[name] = "";
				}
			}

			args["title"] = Quote(title);
			AddConditionalArg("width", width > 0, (width * scaleby));
			AddConditionalArg("height", height > 0, (height * scaleby));
			AddOptionalArg("budget", graph.budget);
			AddConditionalArg("maxy", maxy > 0, maxy);
			args["uniqueID"] = "Graph_" + graphIndex.ToString();
			args["lineDecimalPlaces"] = lineDecimalPlaces.ToString();
			AddConditionalFlag("nocommandlineEmbed", GetBoolArg("embedGraphCommandline"));
			AddConditionalArg("statMultiplier", statMultiplier != 1.0, statMultiplier.ToString("0.0000000000000000000000"));
			AddConditionalArg("hideeventNames", hideEventNames, 1);
			AddConditionalArg("minx", minx > 0, minx);
			AddConditionalArg("maxx", maxx != Int32.MaxValue, maxx);
			AddOptionalArg("miny", graphSettings.minY);
			AddOptionalArg("maxAutoMaxY", graphSettings.maxAutoMaxY);
			AddOptionalArg("threshold", graphSettings.threshold);
			AddOptionalArg("averageThreshold", graphSettings.averageThreshold);
			AddOptionalArg("minFilterStatValue", minFilterStatValueSetting);
			AddConditionalArg("minFilterStatName", graphSettings.minFilterStatName.isSet, graphSettings.minFilterStatName.value);
			AddConditionalArg("compression", compression > 0.0, compression);
			AddConditionalArg("thickness", graphSettings.thickness.isSet, graphSettings.thickness.value * thicknessMultiplier);
			// Smoothing
			AddConditionalFlag("smooth", smooth);
			AddConditionalArg("smoothKernelPercent", smooth && smoothKernelPercent >= 0.0f, smoothKernelPercent);
			AddConditionalArg("smoothKernelSize", smooth && smoothKernelSize >= 0.0f, smoothKernelSize);

			AddConditionalFlag("interactive", interactive);
			AddConditionalFlag("stacked", stacked);
			AddConditionalFlag("forceLegendSort", stacked); // based on the stacked flag
			AddConditionalFlag("showAverages", showAverages);
			AddConditionalFlag("showTotals", showTotals);
			AddConditionalFlag("nosnap", !snapToPeaks);
			AddConditionalFlag("filterOutZeros", filterOutZeros);
			AddConditionalArg("maxHierarchyDepth", maxHierarchyDepth > 0, maxHierarchyDepth);
			AddConditionalArg("hideStatPrefix", hideStatPrefix.Length > 0, hideStatPrefix);
			AddConditionalArg("stacktotalstat", graphSettings.mainStat.isSet, graphSettings.mainStat.value);
			AddOptionalArg("legendAverageThreshold", graphSettings.legendAverageThreshold);
			AddConditionalArg("ignoreStats", graphSettings.ignoreStats.isSet, graphSettings.ignoreStats.value);

			AddConditionalArg("startEvent", graphSettings.startEvent.isSet, graphSettings.startEvent.value);
			AddConditionalArg("startEventOffset", graphSettings.startEventOffset.isSet, graphSettings.startEventOffset.value);
			AddConditionalArg("endEvent", graphSettings.endEvent.isSet, graphSettings.endEvent.value);
			AddConditionalArg("endEventOffset", graphSettings.endEventOffset.isSet, graphSettings.endEventOffset.value);

			List<string> statList = CsvStats.SplitStringListWithBracketGroups(graphSettings.statString.value, ',');

			string argString = string.Empty;
			if (argFormat == CsvToSvgArgFormat.Url)
			{
				string FormatStringList(string inStringList, string splitStr)
				{
					string[] tokens = inStringList
						.Split(splitStr)
						.Select(token => token.Trim())
						.ToArray();
					return String.Join(";", tokens);
				}
				// Exclude csvs from the args as that is a separate argument that the caller must setup.
				// Any multi-value args need to be delimited with a semi-colon.
				// Derived csv stat definitions can include equals, so encode it
				args["stats"] = String.Join(";", statList).Replace("=", "%3D");
				AddConditionalArg("showevents", showEvents.Length > 0, FormatStringList(showEvents, " "));
				AddConditionalArg("highlightEventRegions", highlightEventRegions.Length > 0, FormatStringList(highlightEventRegions, ","));

				List<string> argList = args.Select(a =>
					string.IsNullOrEmpty(a.Value) // Empty value means it's a flag
					? $"{a.Key}=true"
					: $"{a.Key}={a.Value.Replace("\"", "")}") // Strip quotes
					.ToList();
				argString = string.Join("&", argList);
			}
			else
			{
				args["csvs"] = Quote(csvFilename);
				args["o"] = Quote(svgFilename);
				AddConditionalArg("showevents", showEvents.Length > 0, showEvents);
				AddConditionalArg("highlightEventRegions", highlightEventRegions.Length > 0, highlightEventRegions);

				IEnumerable<string> quoteWrappedStatStrings = statList.Select(token => '"' + token + '"');
				args["stats"] = String.Join(" ", quoteWrappedStatStrings);

				List<string> argList = args.Select(a =>
					string.IsNullOrEmpty(a.Value)
					? $"-{a.Key}"
					: $"-{a.Key} {a.Value}")
					.ToList();
				argString = string.Join(" ", argList);
			}
			return argString;
		}


		GraphParams GetCsvToSvgGraphParams(ReportGraph graph, double thicknessMultiplier, int minx, int maxx, bool multipleCSVs, int graphIndex, float scaleby = 1.0f)
		{
			GraphParams graphParams = new GraphParams();
			graphParams.title = graph.title;

			GraphSettings graphSettings = graph.settings;
			graphParams.statNames.AddRange(CsvStats.SplitStringListWithBracketGroups(graphSettings.statString.value, ','));
			graphParams.lineThickness = (float)(graphSettings.thickness.value * thicknessMultiplier);
			graphParams.smooth = graphSettings.smooth.value && !GetBoolArg("nosmooth");
			if (graphParams.smooth)
			{
				if (graphSettings.smoothKernelPercent.isSet && graphSettings.smoothKernelPercent.value > 0)
				{
					graphParams.smoothKernelPercent = (float)graphSettings.smoothKernelPercent.value;
				}
				if (graphSettings.smoothKernelSize.isSet && graphSettings.smoothKernelSize.value > 0)
				{
					graphParams.smoothKernelSize = (int)(graphSettings.smoothKernelSize.value);
				}
			}

			if (graphSettings.compression.isSet)
			{
				graphParams.compression = (float)graphSettings.compression.value;
			}
			graphParams.width = (int)(graphSettings.width.value * scaleby);
			graphParams.height = (int)(graphSettings.height.value * scaleby);
			if (graphSettings.stacked.isSet)
			{
				graphParams.stacked = graphSettings.stacked.value;
				if (graphParams.stacked)
				{
					graphParams.forceLegendSort = true;
					if (graphSettings.mainStat.isSet)
					{
						graphParams.stackTotalStat = graphSettings.mainStat.value;
					}
				}
			}
			if (graphSettings.showAverages.isSet)
			{
				graphParams.showAverages = graphSettings.showAverages.value;
			}
			if (graphSettings.showTotals.isSet)
			{
				graphParams.showTotals = graphSettings.showTotals.value;
			}
			if (graphSettings.filterOutZeros.isSet)
			{
				graphParams.filterOutZeros = graphSettings.filterOutZeros.value;
			}
			graphParams.snapToPeaks = false;
			if (graphSettings.snapToPeaks.isSet)
			{
				graphParams.snapToPeaks = graphSettings.snapToPeaks.value;
			}

			graphParams.lineDecimalPlaces = graphSettings.lineDecimalPlaces.isSet ? graphSettings.lineDecimalPlaces.value : 1;
			if (graphSettings.maxHierarchyDepth.isSet)
			{
				graphParams.maxHierarchyDepth = graphSettings.maxHierarchyDepth.value;
			}
			if (graphSettings.hideStatPrefix.isSet && graphSettings.hideStatPrefix.value.Length > 0)
			{
				graphParams.hideStatPrefixes.AddRange(graphSettings.hideStatPrefix.value.Split(' ', ';'));
			}
			if (multipleCSVs)
			{
				graphParams.showEventNames.Add("CSV:*");
				graphParams.showEventNameTextMode = ShowEventTextMode.Hide;
			}
			else
			{
				if (graphSettings.showEvents.isSet && graphSettings.showEvents.value.Length > 0)
				{
					graphParams.showEventNames.AddRange(graphSettings.showEvents.value.Split(' ', ';'));
				}
			}
			if (graphSettings.statMultiplier.isSet)
			{
				graphParams.statMultiplier = (float)graphSettings.statMultiplier.value;
			}

			if (graphSettings.startEvent.isSet)
			{
				graphParams.startEvent = graphSettings.startEvent.value;
				if (graphSettings.startEventOffset.isSet)
				{
					graphParams.startEventOffset = graphSettings.startEventOffset.value;
				}
			}

			if (graphSettings.endEvent.isSet)
			{
				graphParams.endEvent = graphSettings.endEvent.value;
				if (graphSettings.endEventOffset.isSet)
				{
					graphParams.endEventOffset = graphSettings.endEventOffset.value;
				}
			}

			graphParams.interactive = true;

			if (!GetBoolArg("noStripEvents"))
			{
				List<CsvEventStripInfo> eventsToStrip = reportXML.GetCsvEventsToStrip();
				if (eventsToStrip != null)
				{
					for (int i = 0; i < eventsToStrip.Count; i++)
					{
						graphParams.highlightEventRegions.Add((eventsToStrip[i].beginName == null) ? "{null}" : eventsToStrip[i].beginName);
						graphParams.highlightEventRegions.Add((eventsToStrip[i].endName == null) ? "{null}" : eventsToStrip[i].endName);
					}
				}
			}

			if (graph.minFilterStatValue.isSet)
			{
				graphParams.minFilterStatValue = (float)graph.minFilterStatValue.value;
			}
			if (graphSettings.minFilterStatName.isSet)
			{
				graphParams.minFilterStatName = graphSettings.minFilterStatName.value;
			}
			if (graph.budget.isSet)
			{
				graphParams.budget = (float)graph.budget.value;
			}
			graphParams.uniqueId = "Graph_" + graphIndex.ToString();

			if (minx > 0)
			{
				graphParams.minX = minx;
			}
			if (maxx != Int32.MaxValue)
			{
				graphParams.maxX = maxx;
			}
			if (graphSettings.minY.isSet)
			{
				graphParams.minY = (float)graphSettings.minY.value;
			}
			graphParams.maxY = GetFloatArg("maxY", (float)graphSettings.maxY.value);

			if (graphSettings.maxAutoMaxY.isSet)
			{
				graphParams.maxAutoMaxY = (float)graphSettings.maxAutoMaxY.value;
			}
			if (graphSettings.threshold.isSet)
			{
				graphParams.threshold = (float)graphSettings.threshold.value;
			}
			if (graphSettings.averageThreshold.isSet)
			{
				graphParams.averageThreshold = (float)graphSettings.averageThreshold.value;
			}
			if (graphSettings.legendAverageThreshold.isSet)
			{
				graphParams.legendAverageThreshold = (float)graphSettings.legendAverageThreshold.value;
			}
			if (graphSettings.ignoreStats.isSet && graphSettings.ignoreStats.value.Length > 0)
			{
				graphParams.ignoreStats.AddRange(graphSettings.ignoreStats.value.Split(' ', ';'));
			}
			return graphParams;
		}


		Process LaunchCsvToSvgAsync(string args)
		{
			string csvToolPath = Path.Combine(GetBaseDirectory(), "CSVToSVG.exe");
			string binary = csvToolPath;

			if (Host == HostPlatform.Windows)
			{
				if (!File.Exists(binary) && binary.ToLower().Contains("\\perfreporttool\\"))
				{
					binary = binary.Replace("\\perfreporttool\\", "\\CsvToSVG\\", StringComparison.InvariantCultureIgnoreCase);
				}
			}
			else
			{
				throw new NotImplementedException("-CsvToSvgProcesses is deprecated and only supported on Windows");
			}

			// Generate the SVGs, multithreaded
			ProcessStartInfo startInfo = new ProcessStartInfo(binary);
			startInfo.Arguments = args;
			startInfo.CreateNoWindow = true;
			startInfo.UseShellExecute = false;
			Process process = Process.Start(startInfo);
			return process;
		}

		int CountCSVs(CsvStats csvStats)
		{
			// Count the CSVs
			int csvCount = 0;
			foreach (CsvEvent ev in csvStats.Events)
			{
				string eventName = ev.Name;
				if (eventName.Length > 0)
				{

					if (eventName.Contains("CSV:") && eventName.ToLower().Contains(".csv"))
					{
						csvCount++;
					}
				}
			}
			if (csvCount == 0)
			{
				csvCount = 1;
			}
			return csvCount;
		}

		static int Main(string[] args)
		{
			Program program = new Program();
			if (Debugger.IsAttached)
			{
				program.Run(args);
			}
			else
			{
				try
				{
					program.Run(args);
				}
				catch (System.Exception e)
				{
					Console.Error.WriteLine("[ERROR] " + e.Message);
					return 1;
				}
			}

			return 0;
		}

		bool matchesPattern(string str, string pattern)
		{
			string[] patternSections = pattern.ToLower().Split('*');
			// Check the substrings appear in order
			string remStr = str.ToLower();
			for (int i = 0; i < patternSections.Length; i++)
			{
				int idx = remStr.IndexOf(patternSections[i]);
				if (idx == -1)
				{
					return false;
				}
				remStr = remStr.Substring(idx + patternSections[i].Length);
			}
			return remStr.Length == 0;
		}

		System.IO.FileInfo[] GetFilesWithSearchPattern(string directory, string searchPatternStr, bool recurse, int maxFileAgeDays = -1)
		{
			List<System.IO.FileInfo> fileList = new List<FileInfo>();
			string[] searchPatterns = searchPatternStr.Split(';');
			DirectoryInfo di = new DirectoryInfo(directory);
			foreach (string searchPattern in searchPatterns)
			{
				System.IO.FileInfo[] files = di.GetFiles("*.*", recurse ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
				foreach (FileInfo file in files)
				{
					if (maxFileAgeDays >= 0)
					{
						DateTime fileModifiedTime = file.LastWriteTimeUtc;
						DateTime currentTime = DateTime.UtcNow;
						TimeSpan elapsed = currentTime.Subtract(fileModifiedTime);
						if (elapsed.TotalHours > (double)maxFileAgeDays * 24.0)
						{
							continue;
						}
					}

					if (matchesPattern(file.FullName, searchPattern))
					{
						fileList.Add(file);
					}
				}
			}
			return fileList.Distinct().ToArray();
		}

		void ConvertJsonToPrcs(string jsonFilename, string prcOutputDir)
		{
			Console.WriteLine("Converting " + jsonFilename + " to PRCs. Output folder: " + prcOutputDir);
			if (!Directory.Exists(prcOutputDir))
			{
				Directory.CreateDirectory(prcOutputDir);
			}
			Console.WriteLine("Reading " + jsonFilename);
			string jsonText = File.ReadAllText(jsonFilename);

			Console.WriteLine("Parsing json");
			Dictionary<string, dynamic> jsonDict = JsonToDynamicDict(jsonText);

			Console.WriteLine("Writing PRCs");
			foreach (string csvId in jsonDict.Keys)
			{
				Dictionary<string, dynamic> srcDict = jsonDict[csvId];
				SummaryTableRowData rowData = new SummaryTableRowData(srcDict);
				rowData.WriteToCache(prcOutputDir, csvId);
			}
		}

		Dictionary<string, dynamic> JsonToDynamicDict(string jsonStr)
		{
			JsonElement RootElement = JsonSerializer.Deserialize<JsonElement>((string)jsonStr);
			Dictionary<string, dynamic> RootElementValue = GetJsonValue(RootElement);
			return RootElementValue;
		}

		// .net Json support is poor, so we have to do stuff like this if we just want to read a json file to a dictionary
		dynamic GetJsonValue(JsonElement jsonElement)
		{
			string jsonStr = jsonElement.GetRawText();
			switch (jsonElement.ValueKind)
			{
				case JsonValueKind.Number:
					return jsonElement.GetDouble();

				case JsonValueKind.Null:
					return null;

				case JsonValueKind.True:
					return true;

				case JsonValueKind.False:
					return false;

				case JsonValueKind.String:
					return jsonElement.GetString();

				case JsonValueKind.Undefined:
					return null;

				case JsonValueKind.Array:
					List<dynamic> ArrayValue = new List<dynamic>();
					foreach (JsonElement element in jsonElement.EnumerateArray())
					{
						ArrayValue.Add(GetJsonValue(element));
					}
					return ArrayValue;

				case JsonValueKind.Object:
					Dictionary<string, dynamic> DictValue = new Dictionary<string, dynamic>();
					foreach (JsonProperty property in jsonElement.EnumerateObject())
					{
						DictValue[property.Name] = GetJsonValue(property.Value);
					}
					return DictValue;
			}
			return null;
		}
	}
}
