<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AssemblyTitle>PerfReportTool</AssemblyTitle>
    <Product>PerfReportTool</Product>
  </PropertyGroup>
  <ItemGroup>
    <None Remove="Properties\AssemblyInfo.cs~RF273305d.TMP" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Analytics.PerformanceTrends\EpicGames.Analytics.PerformanceTrends.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
    <ProjectReference Include="..\CsvToSvgLib\CsvToSvgLib.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Drawing.Common" Version="8.0.24" />
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
  <ItemGroup>
    <None Update="LLMReportGraphs.xml">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
    <None Update="LLMReportTypes.xml">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
    <None Update="ReportGraphs.xml">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
    <None Update="ReportTypes.xml">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
  </ItemGroup>
</Project>