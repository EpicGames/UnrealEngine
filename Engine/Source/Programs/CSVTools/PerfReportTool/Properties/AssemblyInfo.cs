// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("ae427314-972a-4178-8c36-fc3efcd164e3")]
[assembly: System.Runtime.CompilerServices.InternalsVisibleTo("PerfReportTool.Tests")]
