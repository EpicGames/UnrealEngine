// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using System;
using System.Collections.Generic;
using System.Xml.Linq;

namespace PerfReportTool
{

	class OptionalString
	{
		public OptionalString(string valueIn)
		{
			value = valueIn;
			isSet = true;
		}
		public OptionalString()
		{
			isSet = false;
		}
		public OptionalString(XElement element, string Name, bool IsElement = false, XmlVariableMappings vars = null )
		{
			isSet = false;
			if (IsElement)
			{
				XElement child = element.Element(Name);
				if (child != null)
				{
					value = child.GetValue(vars);
					isSet = true;
				}
			}
			else
			{
				XAttribute child = element.Attribute(Name);
				if (child != null)
				{
					value = element.GetRequiredAttribute<string>(vars, Name);
					isSet = true;
				}
			}
		}

		public void InheritFrom(OptionalString baseVersion) { if (!isSet) { isSet = baseVersion.isSet; value = baseVersion.value; } }
		public bool isSet;
		public string value;
	};

	class Optional<T>
	{
		public Optional(T valueIn)
		{
			value = valueIn;
			isSet = true;
		}
		public Optional()
		{
			isSet = false;
		}
		public Optional(XElement element, string AttributeName, XmlVariableMappings vars = null)
		{
			isSet = false;
			try
			{
				if (element.Attribute(AttributeName) != null)
				{
					value = element.GetRequiredAttribute<T>(vars, AttributeName);
					isSet = true;
				}
			}
			catch { }
		}
		public void InheritFrom(Optional<T> baseVersion) { if (!isSet) { isSet = baseVersion.isSet; value = baseVersion.value; } }

		public T value;
		public bool isSet;
	};


	static class OptionalHelper
	{
		public static string GetDoubleSetting(Optional<double> setting, string cmdline)
		{
			return (setting.isSet ? (cmdline + setting.value.ToString()) : "");
		}

		public static string GetStringSetting(OptionalString setting, string cmdline)
		{
			return (setting.isSet ? (cmdline + setting.value) : "");
		}
	};

	class ReportGraph
	{
		public ReportGraph(XElement element, XmlVariableMappings vars)
		{
			title = element.GetRequiredAttribute<string>(vars, "title");
			budget = new Optional<double>(element, "budget", vars);
			inSummary = element.GetSafeAttribute<bool>(vars, "inSummary", false);
			isInline = element.GetSafeAttribute<bool>(vars, "inline", false);
			parent = element.GetSafeAttribute<string>(vars, "parent");
			minFilterStatValue = new Optional<double>(element, "minFilterStatValue", vars);

			if (!isInline && parent != null)
			{
				throw new Exception("Parent can only be specified for inline graphs (inline='1'): " + element.ToString());
			}

			if (isInline)
			{
				// If this is an inline graph then just load the settings directly
				settings = new GraphSettings(element, vars);
			}
		}

		public bool ShouldShow(CsvStats stats)
		{
			bool bFoundStat = false;
			foreach (string statString in settings.statString.value.Split(','))
			{
				List<StatSamples> matchingStats = stats.GetStatsMatchingString(statString);
				if (matchingStats.Count > 0)
				{
					bFoundStat = true;
					break;
				}
			}
			if ( !bFoundStat ) 
			{
				return false;
			}

			if (settings.requiredEvents.isSet)
			{
				string[] requiredEvents = settings.requiredEvents.value.Split(',');
				foreach (string requiredEvent in requiredEvents)
				{
					bool bFoundEvent = false;
					foreach (CsvEvent ev in stats.Events)
					{
						if (CsvStats.DoesSearchStringMatch(requiredEvent, ev.Name))
						{
							bFoundEvent = true;
							break;
						}
					}
					if (!bFoundEvent)
					{
						return false;
					}
				}
			}
			return true;
		}
		public string title;
		public Optional<double> budget;
		public bool inSummary;
		public Optional<double> minFilterStatValue;
		public GraphSettings settings;

		public bool isInline;
		public string parent;
	};

	class GraphSettings
	{
		public GraphSettings(XElement element, XmlVariableMappings vars = null)
		{
			smooth = new Optional<bool>(element, "smooth", vars);
			thickness = new Optional<double>(element, "thickness", vars);
			minY = new Optional<double>(element, "minY", vars);
			maxY = new Optional<double>(element, "maxY", vars);

			// Back-compat support. Case was fixed in PRT 4.242.0
			if ( !minY.isSet )
			{
				minY = new Optional<double>(element, "miny", vars);
			}
			if (!maxY.isSet)
			{
				maxY = new Optional<double>(element, "maxy", vars);
			}

			maxAutoMaxY = new Optional<double>(element, "maxAutoMaxY", vars);
			threshold = new Optional<double>(element, "threshold", vars);
			averageThreshold = new Optional<double>(element, "averageThreshold", vars);
			minFilterStatValue = new Optional<double>(element, "minFilterStatValue", vars);
			minFilterStatName = new OptionalString(element, "minFilterStatName", false, vars);
			smoothKernelPercent = new Optional<double>(element, "smoothKernelPercent", vars);
			smoothKernelSize = new Optional<double>(element, "smoothKernelSize", vars);
			compression = new Optional<double>(element, "compression", vars);
			width = new Optional<int>(element, "width", vars);
			height = new Optional<int>(element, "height", vars);
			stacked = new Optional<bool>(element, "stacked", vars);
			showAverages = new Optional<bool>(element, "showAverages", vars);
			showTotals = new Optional<bool>(element, "showTotals", vars);
			filterOutZeros = new Optional<bool>(element, "filterOutZeros", vars);
			maxHierarchyDepth = new Optional<int>(element, "maxHierarchyDepth", vars);
			hideStatPrefix = new OptionalString(element, "hideStatPrefix", false, vars);
			mainStat = new OptionalString(element, "mainStat", false, vars);
			showEvents = new OptionalString(element, "showEvents", false, vars);
			requiresDetailedStats = new Optional<bool>(element, "requiresDetailedStats", vars);
			ignoreStats = new OptionalString(element, "ignoreStats", false, vars);

			statString = new OptionalString(element, "statString", true, vars);
			//additionalArgs = new OptionalString(element, "additionalArgs", true, vars);
			statMultiplier = new	(element, "statMultiplier", vars);
			legendAverageThreshold = new Optional<double>(element, "legendAverageThreshold", vars);
			snapToPeaks = new Optional<bool>(element, "snapToPeaks", vars);
			lineDecimalPlaces = new Optional<int>(element, "lineDecimalPlaces", vars);

			startEvent = new OptionalString(element, "startEvent", false, vars);
			startEventOffset = new Optional<int>(element, "startEventOffset", vars);
			endEvent = new OptionalString(element, "endEvent", false, vars);
			endEventOffset = new Optional<int>(element, "endEventOffset", vars);

			requiredEvents = new OptionalString(element, "requiredEvents", false, vars);
		}
		public void InheritFrom(GraphSettings baseSettings)
		{
			smooth.InheritFrom(baseSettings.smooth);
			statString.InheritFrom(baseSettings.statString);
			thickness.InheritFrom(baseSettings.thickness);
			minY.InheritFrom(baseSettings.minY);
			maxY.InheritFrom(baseSettings.maxY);
			maxAutoMaxY.InheritFrom(baseSettings.maxAutoMaxY);
			threshold.InheritFrom(baseSettings.threshold);
			averageThreshold.InheritFrom(baseSettings.averageThreshold);
			minFilterStatValue.InheritFrom(baseSettings.minFilterStatValue);
			minFilterStatName.InheritFrom(baseSettings.minFilterStatName);
			smoothKernelSize.InheritFrom(baseSettings.smoothKernelSize);
			smoothKernelPercent.InheritFrom(baseSettings.smoothKernelPercent);
			compression.InheritFrom(baseSettings.compression);
			width.InheritFrom(baseSettings.width);
			height.InheritFrom(baseSettings.height);
			//additionalArgs.InheritFrom(baseSettings.additionalArgs);
			stacked.InheritFrom(baseSettings.stacked);
			showAverages.InheritFrom(baseSettings.showAverages);
			showTotals.InheritFrom(baseSettings.showTotals);
			filterOutZeros.InheritFrom(baseSettings.filterOutZeros);
			maxHierarchyDepth.InheritFrom(baseSettings.maxHierarchyDepth);
			hideStatPrefix.InheritFrom(baseSettings.hideStatPrefix);
			mainStat.InheritFrom(baseSettings.mainStat);
			showEvents.InheritFrom(baseSettings.showEvents);
			requiresDetailedStats.InheritFrom(baseSettings.requiresDetailedStats);
			statMultiplier.InheritFrom(baseSettings.statMultiplier);
			ignoreStats.InheritFrom(baseSettings.ignoreStats);
			legendAverageThreshold.InheritFrom(baseSettings.legendAverageThreshold);
			snapToPeaks.InheritFrom(baseSettings.snapToPeaks);
			lineDecimalPlaces.InheritFrom(baseSettings.lineDecimalPlaces);
			startEvent.InheritFrom(baseSettings.startEvent);
			startEventOffset.InheritFrom(baseSettings.startEventOffset);
			endEvent.InheritFrom(baseSettings.endEvent);
			endEventOffset.InheritFrom(baseSettings.endEventOffset);
			requiredEvents.InheritFrom(baseSettings.requiredEvents);
		}
		public Optional<bool> smooth;
		public OptionalString statString;
		public Optional<double> thickness;
		public Optional<double> minY;
		public Optional<double> maxY;
		public Optional<double> maxAutoMaxY;
		public Optional<double> threshold;
		public Optional<double> averageThreshold;
		public Optional<double> minFilterStatValue;
		public OptionalString minFilterStatName;
		public Optional<double> smoothKernelSize;
		public Optional<double> smoothKernelPercent;
		public Optional<double> compression;
		public Optional<int> width;
		public Optional<int> height;
		//public OptionalString additionalArgs;
		public Optional<bool> stacked;
		public Optional<bool> showAverages;
		public Optional<bool> showTotals;
		public Optional<bool> filterOutZeros;
		public Optional<int> maxHierarchyDepth;
		public OptionalString hideStatPrefix;
		public OptionalString mainStat;
		public OptionalString showEvents;
		public OptionalString ignoreStats;
		public Optional<double> statMultiplier;
		public Optional<double> legendAverageThreshold;

		public Optional<bool> requiresDetailedStats;
		public Optional<bool> snapToPeaks;
		public Optional<int> lineDecimalPlaces;

		public OptionalString startEvent;
		public Optional<int> startEventOffset;
		public OptionalString endEvent;
		public Optional<int> endEventOffset;

		public OptionalString requiredEvents;

	};

}