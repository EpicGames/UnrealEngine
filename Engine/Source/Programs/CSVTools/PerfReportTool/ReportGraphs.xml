<?xml version="1.0" encoding="UTF-8"?>

<!-- Please use the XML schemas in "Engine/Source/Programs/CsvTools/PerfReportTool/XmlSchemas" to validate -->
<graphGroups xmlns="https://www.unrealengine.com/PerfReportTool" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance">
	<baseSettings thickness="2" smooth="1" smoothKernelSize="60" compression="0.5"
				  width="2000" height="600" stacked="0" maxHierarchyDepth="2" hideStatPrefix="" showEvents="*" maxY="40" showAverages="1"/>

	<graphGroup name="Core">
		<baseSettings smoothKernelSize="60" smoothKernelPercent="-1"/>

		<graph title="Stat Unit" showAverages="1" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime</statString>
		</graph>

		<graph title="Stat Unit Raw" showAverages="1" thickness="1" smooth="0" maxY="0">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime</statString>
		</graph>

		<graph title="Stat Unit 30hz" showAverages="1" maxY="60" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime</statString>
		</graph>

		<graph title="Stat Unit 20hz" showAverages="1" maxY="80" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime</statString>
		</graph>

		<graph title="Dynamic Resolution" maxY="100" thickness="4" showAverages="1">
			<statString>DynamicResolutionPercentage</statString>
		</graph>

		<graph title="Physical Memory" smooth="0" thickness="4" maxY="0">
			<statString>MemoryFreeMB,PhysicalUsedMB</statString>
		</graph>

		<graph title="RHI Drawcalls" maxY="4000" thickness="4">
			<statString>RHI/Drawcalls</statString>
		</graph>

		<graph title="Gamethread Breakdown" showAverages="1" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" legendAverageThreshold="0.1">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Breakdown Raw" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" smooth="0"  maxY="100" legendAverageThreshold="0.1">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Waits" showAverages="1" hideStatPrefix="exclusive/gamethread/eventwait/" stacked="1" mainStat="frametime" ignoreStats="exclusive/Gamethread/EventWait" maxHierarchyDepth="3">
			<statString>exclusive/gamethread/eventwait/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Physics" showAverages="1" stacked="1" maxY="0" maxHierarchyDepth="3" hideStatPrefix="exclusive/gamethread/">
		  <statString>exclusive/gamethread/eventwait/EndPhysics,exclusive/gamethread/Physics,exclusive/gamethread/vehicle*</statString>
		</graph>

		<graph title="Gamethread Breakdown Giant" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" smooth="0"  maxY="100" legendAverageThreshold="0.1" width="40000" height="1000">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>
    
		<graph title="Renderthread Breakdown" showAverages="1" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="renderthreadtime" ignoreStats="exclusive/Renderthread/FetchVisibilityForPrimitives;exclusive/renderthread/eventwait*">
			<statString>exclusive/renderthread/*,renderthreadtime</statString>
		</graph>

		<graph title="Renderthread Breakdown Raw" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="renderthreadtime" ignoreStats="exclusive/Renderthread/FetchVisibilityForPrimitives;exclusive/renderthread/eventwait*" smooth="0"  maxY="100">
			<statString>exclusive/renderthread/*,renderthreadtime</statString>
		</graph>

		<graph title="GPU Breakdown" showAverages="1" smoothKernelSize="120" hideStatPrefix="gpu/" stacked="1" mainStat="gpu/total">
			<statString>GPU/*</statString>
		</graph>

		<graph title="Niagara Breakdown" showAverages="1" thickness="2" >
			<statString>Ticks/NiagaraComponent,Exclusive/GameThread/Niagara,GPU/NiagaraGPUSimulation,GPU/NiagaraGPUSorting</statString>
		</graph>

		<graph title="Ticks Queued" showAverages="0" maxY="0" thickness="4" smooth="0">
			<statString>Basic/TicksQueued</statString>
		</graph>

		<graph title="Ticks Breakdown" showAverages="1" maxY="0" maxHierarchyDepth="4" hideStatPrefix="Ticks/" stacked="1" mainStat="Basic/TicksQueued" threshold="1">
			<statString>Ticks/*</statString>
		</graph>

		<graph title="Actor Counts" showAverages="0" maxY="0" thickness="4" hideStatPrefix="ActorCount/" smooth="0" stacked="1" mainStat="ActorCount/TotalActorCount">
			<statString>ActorCount/*</statString>
		</graph>

		<graph title="Demo Recording" hideStatPrefix="basic/" maxY="15" thickness="5" showAverages="1">
			<statString>Basic/GameThread/ConcurrentWithSlateTickT*,Basic/*/DemoRecordTime</statString>
		</graph>

		<graph title="Demo Recording (unsmoothed)" hideStatPrefix="basic/" maxY="0" thickness="5" showAverages="1" smooth="0">
			<statString>Basic/GameThread/ConcurrentWithSlateTickT*,Basic/*/DemoRecordTime</statString>
		</graph>

		<graph title="Animation Budget" showAverages="0" hideStatPrefix="Exclusive/GameThread/" mainStat="Exclusive/GameThread/Animation" thickness="4" smooth="1" filterOutZeros="0">
			<statString>Exclusive/GameThread/Animation,AnimationBudget/*</statString>
		</graph>

		<graph title="Log Counts" showAverages="1" hideStatPrefix="fmsglogf/" mainStat="fmsglogf/fmsglogfcount" thickness="4" smooth="0" filterOutZeros="0">
			<statString>fmsglogf/*</statString>
		</graph>

		<graph title="CSV Profiler" showAverages="1" hideStatPrefix="CSVProfiler/" stacked="1" mainStat="frametime" maxY="0">
			<statString>CSVProfiler/Num*</statString>
		</graph>
		
		<graph title="Android Memory" showAverages="1" hideStatPrefix="AndroidMemory/" stacked="1" mainStat="MemoryFreeMB" maxY="0">
			<statString>AndroidMemory/MemoryWarningState,AndroidMemory/TrimMemoryForegroundLevel,AndroidMemory/TrimMemoryBackgroundLevel</statString>
		</graph>		

		<graph title="Android Thermals" showAverages="1" hideStatPrefix="AndroidCPU/" maxY="100">
			<statString>AndroidCPU/CPUTemp,AndroidCPU/ThermalStatus</statString>
		</graph>
		
		<graph title="Android CPU" showAverages="1" hideStatPrefix="AndroidCPU/" maxY="0">
			<statString>AndroidCPU/CPUFreqMHzGroup0,AndroidCPU/CPUFreqMHzGroup1,AndroidCPU/CPUFreqMHzGroup2,AndroidCPU/CPUFreqMHzGroup3</statString>
		</graph>
		
		<graph title="Android CPU Percentage From Maximum" showAverages="1" hideStatPrefix="AndroidCPU/" maxY="0">
			<statString>AndroidCPU/CPUFreqPercentageGroup0,AndroidCPU/CPUFreqPercentageGroup1,AndroidCPU/CPUFreqPercentageGroup2,AndroidCPU/CPUFreqPercentageGroup3</statString>
		</graph>

		<graph title="GPU cycles / frame" showAverages="1" hideStatPrefix="AndroidGPU/" mainStat="frametime" maxY="0">
			<statString>AndroidGPU/GPUCyclesMln,AndroidGPU/VertexCyclesMln,AndroidGPU/FragmentCyclesMln</statString>
		</graph>
		
		<graph title="Pixels / frame" showAverages="1" hideStatPrefix="AndroidGPU/" maxY="0">
			<statString>AndroidGPU/PixelsMln</statString>
		</graph>
		
		<graph title="Shader cycles / frame" showAverages="1" hideStatPrefix="AndroidGPU/" maxY="0">
			<statString>AndroidGPU/ShaderCyclesMln,AndroidGPU/ShaderArithmeticCyclesMln,AndroidGPU/ShaderLoadStoreCyclesMln,AndroidGPU/ShaderTextureCyclesMln</statString>
		</graph>
		
		<graph title="GPU bandwidth / frame" showAverages="1" hideStatPrefix="AndroidGPU/" stacked="1" maxY="0">
			<statString>AndroidGPU/ExternalMemoryReadMB,AndroidGPU/ExternalMemoryWriteMB</statString>
		</graph>

	</graphGroup>

	<graphGroup name="AI5">		
		<baseSettings smooth="0" maxY="0"/>

		<graph title="All StateTree Costs" showAverages="1" hideStatPrefix="Exclusive/Gamethread/" stacked="1">
			<statString>Exclusive/Gamethread/StateTree*</statString>
		</graph>

		<graph title="StateTree Processor Costs" showAverages="1" hideStatPrefix="Exclusive/Gamethread/StateTreeProcessor" stacked="1">
			<statString>Exclusive/Gamethread/StateTreeProcessor*</statString>
		</graph>

		<graph title="StateTree Context Costs" showAverages="1" hideStatPrefix="Exclusive/Gamethread/StateTree_" stacked="1">
			<statString>Exclusive/Gamethread/StateTree_*</statString>
		</graph>

		<graph title="Ticked StateTree Count" showAverages="1" hideStatPrefix="StateTreeProcessor/">
			<statString>StateTreeProcessor/NumTickedStateTree</statString>
		</graph>

		<graph title="Mass Signals Count" showAverages="1" hideStatPrefix="MassSignalsCounters/">
			<statString>MassSignalsCounters/*</statString>
		</graph>
		
		<graph title="Deferred Commands Count" showAverages="1" hideStatPrefix="MassEntitiesCounters/">
			<statString>MassEntitiesCounters/*</statString>
		</graph>
		
		<graph title="Deferred Commands Costs" showAverages="1" hideStatPrefix="MassEntities/Gamethread/" stacked="1">
			<statString>MassEntities/*</statString>
		</graph>

		<graph title="Mass Actor Counts" showAverages="1" hideStatPrefix="MassActors/" stacked="1">
			<statString>MassActors/*</statString>
		</graph>

	</graphGroup>

</graphGroups>
