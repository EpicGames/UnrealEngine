// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Text;
using System.Xml.Linq;
using CSVStats;
using PerfSummaries;

namespace PerfReportTool
{
	class ReportTypeParams
	{
		public string reportTypeOverride;
		public bool forceReportType;
	}

	class ReportTypeInfo
	{
		public ReportTypeInfo(XElement element, Dictionary<string, XElement> sharedSummaries, string baseXmlDirectory, XmlVariableMappings inVariableMappings, CsvMetadata csvMetadata )
		{
			graphs = new List<ReportGraph>();
			summaries = new List<Summary>();

			vars = inVariableMappings;

			// Apply local variable sets
			foreach (XElement variableSetEl in element.Elements("variableSet"))
			{
				vars.ApplyVariableSet(variableSetEl, csvMetadata);
			}

			name = element.GetRequiredAttribute<string>(vars, "name");
			title = element.GetRequiredAttribute<string>(vars, "title");
			bStripEvents = element.GetSafeAttribute<bool>(vars, "stripEvents", true);

			foreach (XElement child in element.Elements())
			{
				if (child.Name == "graph")
				{
					ReportGraph graph = new ReportGraph(child, vars);
					graphs.Add(graph);
				}
				else if (child.Name == "summary" || child.Name == "summaryRef")
				{
					XElement summaryElement = null;
					if (child.Name == "summaryRef")
					{
						summaryElement = sharedSummaries[child.GetRequiredAttribute<string>(vars, "name")];
					}
					else
					{
						summaryElement = child;
					}
					string summaryType = summaryElement.GetRequiredAttribute<string>(vars, "type");
					summaries.Add(SummaryFactory.Create(summaryType, summaryElement, vars, baseXmlDirectory));
				}
				else if (child.Name == "metadataToShow")
				{
					metadataToShowList = child.Value.Split(',');
				}

			}
			ComputeSummaryTableCacheID();
		}

		public string GetSummaryTableCacheID()
		{
			return summaryTableCacheID;
		}

		private void ComputeSummaryTableCacheID()
		{
			StringBuilder sb = new StringBuilder();
			sb.Append("NAME={" + name + "}\n");
			sb.Append("TITLE={" + title + "}\n");
			foreach (Summary summary in summaries)
			{
				sb.Append("SUMMARY={");
				sb.Append("TYPE={" + summary.GetType().ToString() + "}");
				sb.Append("UNSTRIPPED={" + summary.useUnstrippedCsvStats + "}");
				sb.Append("STATS={" + string.Join(",", summary.stats) + "}");
				sb.Append("}");
			}
			summaryTableCacheID = HashHelper.StringToHashStr(sb.ToString(), 16);
		}

		public List<ReportGraph> graphs;
		public List<Summary> summaries;
		public string name;
		public string title;
		public string[] metadataToShowList;
		public string summaryTableCacheID;
		public bool bStripEvents;

		public XmlVariableMappings vars;
	};
}