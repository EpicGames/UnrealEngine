<?xml version="1.0" encoding="UTF-8"?>

<!-- Please use the XML schemas in "Engine/Source/Programs/CsvTools/PerfReportTool/XmlSchemas" to validate -->
<root xmlns="https://www.unrealengine.com/PerfReportTool" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance">
	<!-- Friendly names for display purposes -->
	<statDisplayNameMappings>
		<mapping statName="RHI/Drawcalls" displayName="DrawCalls"/>
		<mapping statName="DynamicResolutionPercentage" displayName="DynRes %"/>
		<mapping statName="Basic/TicksQueued" displayName="TicksQueued"/>
		<mapping statName="MemoryFreeMB" displayName="MemFreeMB"/>
		<mapping statName="PhysicalUsedMB" displayName="MemUsedMB"/> 
		<mapping statName="buildversion" displayName="Build Version"/>
		<mapping statName="config" displayName="Config"/>
		<mapping statName="commandline" displayName="Commandline"/>
		<mapping statName="syncLoads" displayName="Sync Loads"/>
		<mapping statName="scalabilitylevel" displayName="Scaling"/>
		<mapping statName="deviceprofile" displayName="DeviceProfile"/>
		<mapping statName="TargetFPS" displayName="FPS"/>
		<mapping statName="HitchTimePercent" displayName="HitchTime %"/>
	</statDisplayNameMappings>

	<!-- Strip Events - these delimit frames which will be excluded from stats computation -->
	<!-- Note: order is important, since these are stripped in the order listed (with no handling for overlapping events) -->
	<csvEventsToStrip>
		<eventPair begin="App_WillDeactivate" end="App_HasReactivated"/>
	</csvEventsToStrip>

	<!-- Preset summary tables -->
	<!-- In bulk mode (-csvdir), summary table row data is produced for each CSV. This includes data from all active summaries -->
	<!-- The preset Summary Tables below can be used to provide a filtered and sorted view of that data (see -summaryTable). -->
	<summaryTables>
		<summaryTable name="all" rowSort="buildversion,Platform,deviceprofile">
			<filter>buildversion,Platform,DeviceProfile,scalabilitylevel,*</filter>
		</summaryTable>

		<summaryTable name="default" rowSort="buildversion,playlistid,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,scalabilitylevel,Report,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,config,playlistid,framecount*,syncloads</filter>
		</summaryTable>

		<summaryTable name="memory" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel">
			<filter>MemoryFreeMB*,PhysicalUsedMB*,Total Time*</filter>
		</summaryTable>

		<summaryTable name="condensed" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>DeviceProfile,scalabilitylevel,TargetFPS,MVP*,Hitches/Min,MemoryFreeMB Min,PhysicalUsedMB Max,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,DynamicResolution*,SyncLoads*</filter>
		</summaryTable>
	</summaryTables>

	<reporttypes reportGraphsFile="ReportGraphs.xml">

		<!-- 60fps report -->
		<reporttype name="Default60fps" title="60FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="60" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>
			
			<metadataToShow>buildversion,platform,deviceprofile</metadataToShow>

			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP60"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GPUtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds>33, 20, 16.6, 16.2</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>

			<graph title="Stat Unit" budget="16.66" />
			<graph title="Stat Unit Raw" budget="16.66" />
			<graph title="Gamethread Breakdown" budget="16.66"/>
			<graph title="Gamethread Breakdown Raw" budget="16.66"/>
			<graph title="Gamethread Waits" budget="2"/>
			<graph title="Gamethread Physics" budget="2"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="16.66" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="RHI Drawcalls" budget="3000" />
			<graph title="Renderthread Breakdown" budget="16.66"/>
			<graph title="Renderthread Breakdown Raw" budget="16.66"/>
			<graph title="Demo Recording" budget="16.66" />
			<graph title="Demo Recording (unsmoothed)" budget="16.66" />
			<graph title="Animation Budget" budget="1.5" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
			<graph title="Android Memory" budget="0" />
			<graph title="Android Thermals" budget="0" />
			<graph title="Android CPU" budget="0" />
			<graph title="Android CPU Percentage From Maximum" budget="0" />
			<graph title="GPU cycles / frame" budget="0" />
			<graph title="Pixels / frame" budget="0" />
			<graph title="Shader cycles / frame" budget="0" />
			<graph title="GPU bandwidth / frame" budget="0" />
		</reporttype>


		<!-- 30 Fps report -->
		<reporttype name="Default30fps" title="30FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="30" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>

			<metadataToShow>buildversion,platform,deviceprofile</metadataToShow>

			<summary type="fpschart" fps="30" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued</stats>
				<colourThresholds stat="Hitches/Min"> 0, 2.5, 3.75, 5</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP30"> 0, 2, 3, 4</colourThresholds>
				<colourThresholds stat="MemoryFreeMB">400,300,200,100</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="GPUtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds> 20, 16.6, 16.2</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>

			<graph title="Stat Unit 30hz" budget="33.33" />
			<graph title="Stat Unit Raw" budget="33.33" />
			<graph title="Gamethread Breakdown" budget="33.33"/>
			<graph title="Gamethread Breakdown Raw" budget="33.33"/>
			<graph title="Gamethread Waits" budget="4"/>
			<graph title="Gamethread Physics" budget="4"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="33.33" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="RHI Drawcalls" budget="2000" />
			<graph title="Renderthread Breakdown" budget="33.33"/>
			<graph title="Renderthread Breakdown Raw" budget="33.33"/>
			<graph title="Animation Budget" budget="3.0" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
			<graph title="Android Memory" budget="0" />
			<graph title="Android Thermals" budget="0" />
			<graph title="Android CPU" budget="0" />
			<graph title="Android CPU Percentage From Maximum" budget="0" />
			<graph title="GPU cycles / frame" budget="0" />
			<graph title="Pixels / frame" budget="0" />
			<graph title="Shader cycles / frame" budget="0" />
			<graph title="GPU bandwidth / frame" budget="0" />
		</reporttype>

		<!-- AI5 Fps report -->
		<reporttype name="AI5" title="AI Performance Report" vsync="1">
			<metadataToShow>buildversion,platform,deviceprofile</metadataToShow>

			<graph title="All StateTree Costs" budget="0"/>
			<graph title="StateTree Processor Costs" budget="0"/>
			<graph title="StateTree Context Costs" budget="0"/>
			<graph title="Ticked StateTree Count" budget="0"/>
			<graph title="Mass Signals Count" budget="0"/>
			<graph title="Deferred Commands Count" budget="0"/>
			<graph title="Deferred Commands Costs" budget="0"/>
			<graph title="Mass Actor Counts" budget="0" />
		</reporttype>

		<reporttype name="Memory" title="Physical Memory Report" vsync="1">
			<autodetection requiredstats="memoryfreemb,physicalusedmb"/>
			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>MemoryFreeMB(min),PhysicalUsedMB(max),MemoryFreeMB,PhysicalUsedMB,PhysicalUsedMB(min)</stats>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
			</summary>
			<graph title="Physical Memory" budget="300"/>
		</reporttype>
	</reporttypes>
</root>