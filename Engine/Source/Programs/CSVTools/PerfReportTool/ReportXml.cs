// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using CSVStats;
using PerfSummaries;
using System.Globalization;
using System.Text.Json;

namespace PerfReportTool
{
	class CsvEventStripInfo
	{
		public string beginName;
		public string endName;
	};

	class DerivedMetadataEntry
	{
		public DerivedMetadataEntry(XElement derivedMetadataEntry)
		{
			metadataQuery = derivedMetadataEntry.GetSafeAttribute<string>("metadataQuery");
			if (metadataQuery == null)
			{
				// Back-compat: support sourceName/destName if metadataQuery isn't provided
				string sourceName = derivedMetadataEntry.GetSafeAttribute<string>("sourceName");
				if (sourceName != null)
				{
					string sourceValue = derivedMetadataEntry.GetRequiredAttribute<string>("sourceValue");
					if (sourceValue != null)
					{
						metadataQuery = sourceName + "=" + sourceValue;
					}
				}
			}
			destName = derivedMetadataEntry.GetRequiredAttribute<string>("destName");
			destValue = derivedMetadataEntry.GetRequiredAttribute<string>("destValue");
			bHasVariables = (destName.Contains("${") || destValue.Contains("${") || (metadataQuery != null && metadataQuery.Contains("${")));
		}
		private DerivedMetadataEntry()
		{
		}
		public DerivedMetadataEntry ApplyVariableMappings(XmlVariableMappings vars)
		{
			if (vars == null || !bHasVariables)
			{
				return this;
			}
			DerivedMetadataEntry newEntry = new DerivedMetadataEntry();
			newEntry.metadataQuery = metadataQuery != null ? vars.ResolveVariables(metadataQuery) : null;
			newEntry.destName = vars.ResolveVariables(destName);
			newEntry.destValue = vars.ResolveVariables(destValue);
			return newEntry;
		}
		public string metadataQuery;
		public string destName;
		public string destValue;
		public bool bHasVariables;
	};

	class DerivedCsvStatDefinition
	{
		public DerivedCsvStatDefinition(XElement element, XmlVariableMappings vars)
		{
			name = element.GetSafeAttribute<string>(vars, "name");
			expression = element.GetSafeAttribute<string>(vars, "expression");
		}
		public string getExpressionStr()
		{
			return name + "=" + expression;
		}
		public string name;
		string expression;
	}

	class DerivedMetadataMappings
	{
		public DerivedMetadataMappings()
		{
			entries = new List<DerivedMetadataEntry>();
		}
		public void ApplyMapping(CsvMetadata csvMetadata, XmlVariableMappings vars)
		{
			if (csvMetadata != null)
			{
				List<KeyValuePair<string, string>> valuesToAdd = new List<KeyValuePair<string, string>>();
				foreach (DerivedMetadataEntry rawEntry in entries)
				{
					DerivedMetadataEntry entry = rawEntry.ApplyVariableMappings(vars);

					// Only override if the key is not already in the CSV metadata
					if (!csvMetadata.Values.ContainsKey(entry.destName.ToLowerInvariant()))
					{
						if (entry.metadataQuery == null || CsvStats.DoesMetadataMatchFilter(csvMetadata, entry.metadataQuery))
						{
							string key = entry.destName.ToLowerInvariant();
							valuesToAdd.Add(new KeyValuePair<string, string>(key, entry.destValue));

							// Add the derived metadata value to variables
							vars.SetVariable("meta." + key, entry.destValue);
						}
					}
				}
				foreach (KeyValuePair<string, string> pair in valuesToAdd)
				{
					csvMetadata.Values[pair.Key] = pair.Value;
				}
			}
		}
		public List<DerivedMetadataEntry> entries;
	}

	class ReportXML
	{
		bool IsAbsolutePath(string path)
		{
			if (path.Length > 3 && path[1] == ':' && (path[2] == '\\' || path[2] == '/'))
			{
				return true;
			}
			return false;
		}

		static XDocument LoadXmlDocument(string filename)
		{
			XDocument xDoc = XDocument.Load(filename);
			// Strip the namespace so we don't have to qualify every name
			StripNamespaceRecursive(xDoc.Root, "https://www.unrealengine.com/PerfReportTool");
			return xDoc;
		}

		static void StripNamespaceRecursive(XElement element, XNamespace namespaceToStrip)
		{
			foreach (XElement el in element.DescendantsAndSelf())
			{
				el.Name = el.Name.LocalName;
				List<XAttribute> atList = el.Attributes().ToList();
				el.Attributes().Remove();
				foreach (XAttribute at in atList)
				{
					el.Add(new XAttribute(at.Name.LocalName, at.Value));
				}
			}
		}


		public ReportXML(
			string graphXMLFilenameIn, 
			string reportXMLFilenameIn, 
			string baseXMLDirectoryOverride, 
			string additionalSummaryTableXmlFilename, 
			string summaryTableXmlSubstStr, 
			string summaryTableXmlAppendStr,
			string summaryTableXmlRowSortAppendStr )
		{
			string location = System.Reflection.Assembly.GetEntryAssembly().Location.ToLower();
			string baseDirectory = Path.GetDirectoryName(location);

			// Check if this is a debug build, and redirect base dir to binaries if so
			if (baseDirectory.Contains("\\engine\\source\\programs\\") && baseDirectory.Contains("\\csvtools\\") && baseDirectory.Contains("\\bin\\debug"))
			{
				baseDirectory = baseDirectory.Replace("\\engine\\source\\programs\\", "\\engine\\binaries\\dotnet\\");
				int csvToolsIndex = baseDirectory.LastIndexOf("\\csvtools\\");
				baseDirectory = baseDirectory.Substring(0, csvToolsIndex + "\\csvtools\\".Length);
			}

			// Check if the base directory is being overridden
			if (baseXMLDirectoryOverride.Length > 0)
			{
				baseDirectory = IsAbsolutePath(baseXMLDirectoryOverride) ? baseXMLDirectoryOverride : Path.Combine(baseDirectory, baseXMLDirectoryOverride);
			}
			Console.Out.WriteLine("BaseDir:   " + baseDirectory);

			baseXmlDirectory = baseDirectory;

			// Read the report type XML
			reportTypeXmlFilename = Path.Combine(baseDirectory, "ReportTypes.xml");
			if (reportXMLFilenameIn.Length > 0)
			{
				reportTypeXmlFilename = IsAbsolutePath(reportXMLFilenameIn) ? reportXMLFilenameIn : Path.Combine(baseDirectory, reportXMLFilenameIn);
			}
			Console.Out.WriteLine("ReportXML: " + reportTypeXmlFilename);
			XDocument reportTypesDoc = LoadXmlDocument(reportTypeXmlFilename);
			rootElement = reportTypesDoc.Element("root");
			if (rootElement == null)
			{
				throw new Exception("No root element found in report XML " + reportTypeXmlFilename);
			}

			// Read the additional summary table XML (if supplied)
			if (additionalSummaryTableXmlFilename.Length > 0)
			{
				if (!IsAbsolutePath(additionalSummaryTableXmlFilename))
				{
					additionalSummaryTableXmlFilename = Path.Combine(baseDirectory, additionalSummaryTableXmlFilename);
				}
				XDocument summaryXmlDoc = LoadXmlDocument(additionalSummaryTableXmlFilename);
				XElement summaryTablesEl = summaryXmlDoc.Element("summaryTables");
				if (summaryTablesEl == null)
				{
					throw new Exception("No summaryTables element found in summaryTableXML file: " + additionalSummaryTableXmlFilename);
				}

				XElement destSummaryTablesEl = rootElement.Element("summaryTables");
				if (destSummaryTablesEl == null)
				{
					rootElement.Add(summaryTablesEl);
				}
				else
				{
					foreach (XElement child in summaryTablesEl.Elements())
					{
						destSummaryTablesEl.Add(child);
					}
				}
			}
			Console.Out.WriteLine("ReportXML: " + reportTypeXmlFilename);

			reportTypesElement = rootElement.Element("reporttypes");
			if (reportTypesElement == null)
			{
				throw new Exception("No reporttypes element found in report XML " + reportTypeXmlFilename);
			}

			// Read the global element set
			globalVariableSetElement = rootElement.Element("globalVariableSet");
			if (globalVariableSetElement != null)
			{
				// Read static variable mappings. That is all the variables which don't depend on CSV metadata (anything with a metadata query will be stripped)
                // These variables are independent of CSVs and can be used outside of individual reports (e.g in summary tables)
				staticVariableMappings = new XmlVariableMappings();
				staticVariableMappings.ApplyVariableSet(globalVariableSetElement, null);
			}

			// Read the graph XML
			string graphsXMLFilename;
			if (graphXMLFilenameIn.Length > 0)
			{
				graphsXMLFilename = IsAbsolutePath(graphXMLFilenameIn) ? graphXMLFilenameIn : Path.Combine(baseDirectory, graphXMLFilenameIn);
			}
			else
			{
				graphsXMLFilename = reportTypesElement.GetSafeAttribute<string>("reportGraphsFile");
				if (graphsXMLFilename != null)
				{
					graphsXMLFilename = Path.Combine(Path.GetDirectoryName(reportTypeXmlFilename), graphsXMLFilename);
				}
				else
				{
					graphsXMLFilename = Path.Combine(baseDirectory, "ReportGraphs.xml");
				}

			}
			defaultReportTypeName = reportTypesElement.GetSafeAttribute<string>(staticVariableMappings, "default");

			Console.Out.WriteLine("GraphXML:  " + graphsXMLFilename+"\n");
			XDocument reportGraphsDoc = LoadXmlDocument(graphsXMLFilename);
			graphGroupsElement = reportGraphsDoc.Element("graphGroups");

			// Read the base settings - all other settings will inherit from this
			GraphSettings baseSettings = new GraphSettings(graphGroupsElement.Element("baseSettings"));
			if (reportTypesElement == null)
			{
				throw new Exception("No baseSettings element found in graph XML " + graphsXMLFilename);
			}

			graphs = new Dictionary<string, GraphSettings>();
			foreach (XElement graphGroupElement in graphGroupsElement.Elements("graphGroup"))
			{
				// Create the base settings
				XElement settingsElement = graphGroupElement.Element("baseSettings");
				GraphSettings groupSettings = new GraphSettings(settingsElement);
				groupSettings.InheritFrom(baseSettings);
				foreach (XElement graphElement in graphGroupElement.Elements("graph"))
				{
					string title = graphElement.GetRequiredAttribute<string>("title").ToLower();
					GraphSettings graphSettings = new GraphSettings(graphElement);
					graphSettings.InheritFrom(groupSettings);
					graphs.Add(title, graphSettings);
				}
			}



			// Read the display name mapping
			statDisplayNameMapping = new Dictionary<string, StatDisplayInfo>();
			XElement displayNameElement = rootElement.Element("statDisplayNameMappings");
			if (displayNameElement != null)
			{
				foreach (XElement mapping in displayNameElement.Elements("mapping"))
				{
					string statName = mapping.GetSafeAttribute<string>("statName");
					string displayName = mapping.GetSafeAttribute<string>("displayName");
					string toolTip = mapping.GetSafeAttribute<string>("toolTip");

					if (statName != null && (displayName != null || toolTip != null))
					{
						statDisplayNameMapping.Add(statName.ToLower(), new StatDisplayInfo(displayName, toolTip));
					}
				}
			}

			XElement summaryTableColumnInfoListEl = rootElement.Element("summaryTableColumnFormatInfo");
			if (summaryTableColumnInfoListEl != null)
			{
				columnFormatInfoList = new SummaryTableColumnFormatInfoCollection(summaryTableColumnInfoListEl);
			}

			// Read any metadata proxies
			metadataProxyInfo = new Dictionary<string, string>();
			XElement metadataProxiesMappingsElement = rootElement.Element("csvMetadataProxies");
			if (metadataProxiesMappingsElement != null)
			{
				foreach (XElement proxy in metadataProxiesMappingsElement.Elements("csvMetadata"))
				{
					string key = proxy.FirstAttribute.Value.ToString().ToLower();
					string value = proxy.Value.ToString().ToLower();
					metadataProxyInfo[key] = value;
				}
			}

			// Read the derived metadata mappings
			derivedMetadataMappings = new DerivedMetadataMappings();
			XElement derivedMetadataMappingsElement = rootElement.Element("derivedMetadataMappings");
			if (derivedMetadataMappingsElement != null)
			{
				foreach (XElement mapping in derivedMetadataMappingsElement.Elements("mapping"))
				{
					derivedMetadataMappings.entries.Add(new DerivedMetadataEntry(mapping));
				}
			}

			// Read the derived CSV stat definitions mappings
			derivedCsvStatDefinitions = new Dictionary<string, DerivedCsvStatDefinition>();
			XElement derivedCsvStatsElement = rootElement.Element("derivedCsvStats");
			if (derivedCsvStatsElement != null)
			{
				foreach (XElement el in derivedCsvStatsElement.Elements("derivedCsvStat"))
				{
					DerivedCsvStatDefinition derivedCsvStatDefinition = new DerivedCsvStatDefinition(el, staticVariableMappings);
					derivedCsvStatDefinitions[derivedCsvStatDefinition.name.ToLowerInvariant()] = derivedCsvStatDefinition;
				}
			}


			// Read events to strip
			XElement eventsToStripEl = rootElement.Element("csvEventsToStrip");
			if (eventsToStripEl != null)
			{
				csvEventsToStrip = new List<CsvEventStripInfo>();
				foreach (XElement eventPair in eventsToStripEl.Elements("eventPair"))
				{
					CsvEventStripInfo eventInfo = new CsvEventStripInfo();
					eventInfo.beginName = eventPair.GetSafeAttribute<string>("begin");
					eventInfo.endName = eventPair.GetSafeAttribute<string>("end");

					if (eventInfo.beginName == null && eventInfo.endName == null)
					{
						throw new Exception("eventPair with no begin or end attribute found! Need to have one or the other.");
					}
					csvEventsToStrip.Add(eventInfo);
				}
			}

			summaryTablesElement = rootElement.Element("summaryTables");
			if (summaryTablesElement != null)
			{
				// Read the substitutions
				Dictionary<string, List<string>> substitutionsDict = null;
				string[] substitutions = summaryTableXmlSubstStr.Split(';');
				if (substitutions.Length>0)
				{
					substitutionsDict = new Dictionary<string, List<string>>();
					foreach (string substStr in substitutions)
					{
						string [] pair = substStr.Split('=');
						if (pair.Length == 2)
						{
							substitutionsDict[pair[0].ToLowerInvariant()] = pair[1].Split(",").ToList();
						}
					}
				}

				string[] appendList = null;
				if ( summaryTableXmlAppendStr != null )
				{
					appendList = summaryTableXmlAppendStr.Split(',');
				}

				string[] rowSortAppendList = null;
				if (summaryTableXmlRowSortAppendStr != null)
				{
					rowSortAppendList = summaryTableXmlRowSortAppendStr.Split(',');
				}


				summaryTables = new Dictionary<string, SummaryTableInfo>();
				foreach (XElement summaryElement in summaryTablesElement.Elements("summaryTable"))
				{
					SummaryTableInfo table = new SummaryTableInfo(summaryElement, substitutionsDict, appendList, rowSortAppendList, staticVariableMappings);
					summaryTables.Add(summaryElement.GetRequiredAttribute<string>("name").ToLower(), table);
				}
			}

			// Add any shared summaries
			XElement sharedSummariesElement = rootElement.Element("sharedSummaries");
			sharedSummaries = new Dictionary<string, XElement>();
			if (sharedSummariesElement != null)
			{
				foreach (XElement summaryElement in sharedSummariesElement.Elements("summary"))
				{
					sharedSummaries.Add(summaryElement.GetRequiredAttribute<string>("refName"), summaryElement);
				}
			}

		}

		public ReportTypeInfo GetReportTypeInfo(string reportType, CachedCsvFile csvFile, bool bBulkMode, bool forceReportType)
		{
			XmlVariableMappings vars = csvFile.xmlVariableMappings;
			// Apply the global variable set
			if (globalVariableSetElement != null)
			{
				vars.ApplyVariableSet(globalVariableSetElement, csvFile.metadata);
			}
			ReportTypeInfo reportTypeInfo = null;
			if (reportType == "")
			{
				XElement defaultReportTypeElement = null;
				// Attempt to determine the report type automatically based on the stats
				foreach (XElement element in reportTypesElement.Elements("reporttype"))
				{
					bool bReportTypeSupportsAutodetect = element.GetSafeAttribute<bool>(vars, "allowAutoDetect", true);

					if (bReportTypeSupportsAutodetect && IsReportTypeXMLCompatibleWithStats(element, csvFile.dummyCsvStats, vars))
					{
						reportTypeInfo = new ReportTypeInfo(element, sharedSummaries, baseXmlDirectory, vars, csvFile.metadata);
						break;
					}

					if (defaultReportTypeName != null && element.GetSafeAttribute<string>(vars, "name") == defaultReportTypeName )
					{
						defaultReportTypeElement = element;
					}
				}
				// Attempt to fall back to a default ReportType if we didn't find one
				if (reportTypeInfo == null && defaultReportTypeName != null)
				{
					if ( defaultReportTypeElement == null )
					{
						throw new Exception("Default report type " + defaultReportTypeName + " was not found in " + reportTypeXmlFilename);
					}
					if (!IsReportTypeXMLCompatibleWithStats(defaultReportTypeElement, csvFile.dummyCsvStats, vars, true))
					{
						throw new Exception("Default report type " + defaultReportTypeName + " was not compatible with CSV " + csvFile.filename);
					}
					Console.Out.WriteLine("Falling back to default report type: " + defaultReportTypeName);
					reportTypeInfo = new ReportTypeInfo(defaultReportTypeElement, sharedSummaries, baseXmlDirectory, vars, csvFile.metadata);
				}
				else if (reportTypeInfo == null)
				{
					throw new Exception("Compatible report type for CSV " + csvFile.filename + " could not be found in " + reportTypeXmlFilename);
				}
			}
			else
			{
				XElement foundReportTypeElement = null;
				foreach (XElement element in reportTypesElement.Elements("reporttype"))
				{
					if (element.GetSafeAttribute<string>(vars, "name").ToLower() == reportType)
					{
						foundReportTypeElement = element;
					}
				}
				if (foundReportTypeElement == null)
				{
					throw new Exception("Report type " + reportType + " not found in " + reportTypeXmlFilename);
				}

				if (!IsReportTypeXMLCompatibleWithStats(foundReportTypeElement, csvFile.dummyCsvStats, vars))
				{
					if (forceReportType)
					{
						Console.Out.WriteLine("Report type " + reportType + " is not compatible with CSV " + csvFile.filename + ", but using it anyway");
					}
					else
					{
						throw new Exception("Report type " + reportType + " is not compatible with CSV " + csvFile.filename);
					}
				}
				reportTypeInfo = new ReportTypeInfo(foundReportTypeElement, sharedSummaries, baseXmlDirectory, vars, csvFile.metadata);
			}

			// Load the graphs
			List<ReportGraph> invalidGraphs = new List<ReportGraph>();
			foreach (ReportGraph graph in reportTypeInfo.graphs)
			{
				if (graph.isInline)
				{
					if (graph.parent != null)
					{
						GraphSettings parentSettings = null;
						if (!graphs.TryGetValue(graph.parent.ToLower(), out parentSettings))
						{
							if (bBulkMode)
							{
								Console.Error.WriteLine("Parent graph with title \"" + graph.parent + "\" was not found in graphs XML. Skipping.");
								invalidGraphs.Add(graph);
								continue;
							}
							else
							{
								// Fatal in non-bulk mode
								throw new Exception("Parent graph with title \"" + graph.parent + "\" was not found in graphs XML");
							}
						}
						graph.settings.InheritFrom(parentSettings);
					}
				}
				else
				{
					if (!graphs.TryGetValue(graph.title.ToLower(), out graph.settings))
					{
						if (bBulkMode)
						{
							Console.Error.WriteLine("Graph with title \"" + graph.title + "\" was not found in graphs XML. Skipping");
							invalidGraphs.Add(graph);
						}
						else
						{
							// Fatal in non-bulk mode
							throw new Exception("Graph with title \"" + graph.title + "\" was not found in graphs XML");
						}
					}
				}
			}

			// Strip any invalid graphs
			reportTypeInfo.graphs.RemoveAll((graph) => invalidGraphs.Contains(graph));

			foreach (Summary summary in reportTypeInfo.summaries)
			{
				summary.PostInit(reportTypeInfo, csvFile.dummyCsvStats);
			}
			return reportTypeInfo;
		}

		bool IsReportTypeXMLCompatibleWithStats(XElement reportTypeElement, CsvStats csvStats, XmlVariableMappings vars, bool bIsDefaultFallback=false)
		{
			string name = reportTypeElement.GetSafeAttribute<string>(vars, "name");
			if (name == null)
			{
				return false;
			}
			string reportTypeName = name;

			XElement autoDetectionEl = reportTypeElement.Element("autodetection");
			if (autoDetectionEl == null)
			{
				return false;
			}
			string requiredStatsStr = autoDetectionEl.GetSafeAttribute<string>(vars, "requiredstats");
			if (requiredStatsStr != null)
			{
				string[] requiredStats = requiredStatsStr.Split(',');
				foreach (string stat in requiredStats)
				{
					if (csvStats.GetStatsMatchingString(stat).Count == 0)
					{
						return false;
					}
				}
			}

			foreach (XElement requiredMetadataEl in autoDetectionEl.Elements("requiredmetadata"))
			{
				string key = requiredMetadataEl.GetSafeAttribute<string>(vars, "key");
				if (key == null)
				{
					throw new Exception("Report type " + reportTypeName + " has no 'key' attribute!");
				}
				string allowedValuesAt = requiredMetadataEl.GetSafeAttribute<string>(vars, "allowedValues");
				if (allowedValuesAt == null)
				{
					throw new Exception("Report type " + reportTypeName + " has no 'allowedValues' attribute!");
				}

				if (csvStats.metaData == null)
				{
					// There was required metadata, but the CSV doesn't have any
					return false;
				}

				// Some metadata may be safe to skip for the default fallback case
				if (bIsDefaultFallback && requiredMetadataEl.GetSafeAttribute(vars, "ignoreForDefaultFallback", false))
				{
					continue;
				}

				bool ignoreIfKeyNotFound = requiredMetadataEl.GetSafeAttribute(vars, "ignoreIfKeyNotFound", true);
				bool stopIfKeyFound = requiredMetadataEl.GetSafeAttribute(vars, "stopIfKeyFound", false);

				key = key.ToLower();
				if (csvStats.metaData.Values.ContainsKey(key))
				{
					string value = csvStats.metaData.Values[key].ToLower();
					string[] allowedValues = allowedValuesAt.ToString().ToLower().Split(',');
					if (!allowedValues.Contains(value))
					{
						return false;
					}
					if (stopIfKeyFound)
					{
						break;
					}
				}
				else if (ignoreIfKeyNotFound == false)
				{
					return false;
				}
			}

			Console.Out.WriteLine("Autodetected report type: " + reportTypeName);

			return true;
		}


		public Dictionary<string, StatDisplayInfo> GetDisplayNameMapping() { return statDisplayNameMapping; }

		public SummaryTableInfo GetSummaryTable(string name)
		{
			if (summaryTables.ContainsKey(name.ToLower()))
			{
				return summaryTables[name.ToLower()];
			}
			else
			{
				throw new Exception("Requested summary table type '" + name + "' was not found in <summaryTables>");
			}
		}

		public List<CsvEventStripInfo> GetCsvEventsToStrip()
		{
			return csvEventsToStrip;
		}

		public List<string> GetSummaryTableNames()
		{
			return summaryTables.Keys.ToList();
		}

		public string GetMetadataProxyInfo(string key)
		{
			string lowerKey = key.ToLower();
			if (metadataProxyInfo.ContainsKey(lowerKey))
			{
				return metadataProxyInfo[lowerKey];
			}
			return "";
		}

		public DerivedCsvStatDefinition GetDerivedCsvStatDefinition(string name)
		{
			if (derivedCsvStatDefinitions.TryGetValue(name, out DerivedCsvStatDefinition def))
			{
				return def;
			}
			return null; 
		}

		Dictionary<string, SummaryTableInfo> summaryTables;

		XElement reportTypesElement;
		XElement rootElement;
		XElement graphGroupsElement;
		XElement summaryTablesElement;
		XElement globalVariableSetElement;
		XmlVariableMappings staticVariableMappings;
		string defaultReportTypeName;
		Dictionary<string, XElement> sharedSummaries;
		Dictionary<string, GraphSettings> graphs;
		Dictionary<string, StatDisplayInfo> statDisplayNameMapping;
		Dictionary<string, string> metadataProxyInfo;
		public SummaryTableColumnFormatInfoCollection columnFormatInfoList;
		string baseXmlDirectory;

		List<CsvEventStripInfo> csvEventsToStrip;
		string reportTypeXmlFilename;
		public DerivedMetadataMappings derivedMetadataMappings;
		public Dictionary<string, DerivedCsvStatDefinition> derivedCsvStatDefinitions;
	}

	class XmlVariableMappings
	{
		public void SetVariable(string Name, string Value, bool bValidate = true)
		{
			if (bValidate)
			{
				// Check for legal characters in the name
				if (!Name.All(x => char.IsLetterOrDigit(x) || x == '.' || x=='_'))
				{
					throw new Exception("Invalid variable name: " + Name);
				}
			}

			vars[Name] = Value;
		}

		public void SetMetadataVariables(CsvMetadata csvMetadata)
		{
			Dictionary<string, string> metadataDict = csvMetadata.Values;
			foreach (string key in metadataDict.Keys)
			{
				SetVariable("meta." + key, metadataDict[key], false);
			}
		}

		public void DumpToLog(bool bIncludeMetadata = false)
		{
			string[] keys = vars.Keys.ToArray();
			Array.Sort(keys);
			foreach (string key in keys)
			{
				if (!key.StartsWith("meta.") || bIncludeMetadata)
				{
					string value = vars[key].Replace(", ",",").Replace(",",", "); // Ensure padding for arrays
					Console.WriteLine(key.PadRight(50) + value);
				}
			}
		}
		public void SerializeToJson(string Filename, string toMatch, string toIgnore)
		{
			string[] keys = vars.Keys.ToArray();
			List<string> MatchedKeys = new List<string>();
			foreach (string key in keys)
			{
				if (key.Contains(toMatch) && (toIgnore.Length==0 || !key.Contains(toIgnore)))
				{
					MatchedKeys.Add(key);
				}
			}

			if (MatchedKeys.Any())
			{
				MatchedKeys.Sort();

				Dictionary<string, string> MatchedVars = new Dictionary<string, string>();
				foreach (string key in MatchedKeys)
				{
					if (vars.ContainsKey(key) && vars[key].Any())
					{
						MatchedVars[key] = vars[key].Replace(", ", ",").Replace(",", ", ");
					}
				}

				JsonSerializerOptions options = new JsonSerializerOptions { WriteIndented = true };
				FileStream createStream = File.Create(Filename);
				JsonSerializer.Serialize(createStream, MatchedVars, options);
				createStream.Dispose();
			}
		}

		public string ResolveVariables(string attributeValue)
		{
			if (!attributeValue.Contains('$'))
			{
				return attributeValue;
			}

			// Remap all variables found in the attribute value
			int StringPos = 0;
			while (StringPos < attributeValue.Length)
			{
				int VarStartIndex = attributeValue.IndexOf("${", StringPos);
				if (VarStartIndex == -1)
				{
					break;
				}
				int VarEndIndex = attributeValue.IndexOf('}', VarStartIndex+1);
				if (VarEndIndex == -1)
				{
					break;
				}

				// Advance StringPos
				StringPos = VarEndIndex;

				string FullVariableName = attributeValue.Substring(VarStartIndex+2, VarEndIndex - VarStartIndex-2);
				string VariableName = FullVariableName;

				string VariableValue = "";

				// Check for an array index
				int OpenBracketIndex = VariableName.IndexOf('[');
				if (OpenBracketIndex != -1)
				{
					int ArrayIndex = -1;
					if (FullVariableName.EndsWith("]"))
					{
						string ArrayIndexStr = VariableName.Substring(OpenBracketIndex + 1, VariableName.Length - 2 - OpenBracketIndex);
						if (!int.TryParse(ArrayIndexStr, out ArrayIndex))
						{
							ArrayIndex = -1;
						}
					}
					if (ArrayIndex < 0)
					{
						Console.WriteLine("[Warning] Failed to resolve variable ${" + FullVariableName + "}. Can't read array index");
					}
					else
					{
						VariableName = FullVariableName.Substring(0, OpenBracketIndex);
						if (vars.TryGetValue(VariableName, out VariableValue))
						{
							string[] elements = VariableValue.Split(",");
							if (ArrayIndex < elements.Length)
							{
								VariableValue = elements[ArrayIndex];
							}
							else
							{
								Console.WriteLine("[Warning] Failed to resolve variable ${" + FullVariableName + "}. Array index out of range!");
							}
						}
						else
						{
							Console.WriteLine("[Warning] Failed to resolve array variable ${" + FullVariableName + "}");
							VariableValue = "";
						}
					}
				}
				else 
				{
					// Read the variable value. Default to empty string and replace anyway if not found
					if (!vars.TryGetValue(VariableName, out VariableValue))
					{
						Console.WriteLine("[Warning] Failed to resolve variable ${" + VariableName + "}");
						VariableValue = "";
					}
				}
				
				// Replace the variable name with its value and update StringPos to take into account the replace
				attributeValue = attributeValue.Substring(0, VarStartIndex) + VariableValue + attributeValue.Substring(VarEndIndex + 1);
				StringPos = VarStartIndex + VariableValue.Length;
			}
			return attributeValue;
		}

		public void ApplyVariableSet(XElement variableSetElement, CsvMetadata csvMetadata, double parentMultiplier=1.0)
		{
			string metadataQuery = variableSetElement.GetSafeAttribute<string>(this, "metadataQuery");
			double multiplier = variableSetElement.GetSafeAttribute<double>(this, "multiplier", 1.0) * parentMultiplier;

			if ( metadataQuery == null || ( csvMetadata != null && CsvStats.DoesMetadataMatchFilter(csvMetadata, metadataQuery) ) )
			{
				// We match, so apply all variables and recursive variablesets in order
				foreach (XElement child in variableSetElement.Elements())
				{ 
					if (child.Name == "var")
					{
						string name = child.GetSafeAttribute<string>("name");
						string value;
						if (name == null)
						{
							// Back-compat version for legacy format <var name="value"> names
							Console.WriteLine("Warning: Variable " + name + " was in deprecated format. Please convert to new format: <var name=\"name\">value</var>");
							name = child.FirstAttribute.Name.ToString();
							value = ResolveVariables(child.FirstAttribute.Value);
						}
						else
						{
							// New format names, ie <var name="name">value</var>
							value = ResolveVariables(child.Value);
						}
						double variableMultiplier = child.GetSafeAttribute<double>(this, "multiplier", 1.0) * multiplier;

						// Apply the multplier if there is one before setting the variable
						if (variableMultiplier != 1.0)
						{
							string[] arrayValues = value.Split(',');
							List<string> finalValues = new List<string>();
							foreach (string elementValue in arrayValues)
							{
								if (!double.TryParse(elementValue, out double doubleVal))
								{
									break;
								}
								finalValues.Add((variableMultiplier * doubleVal).ToString());
							}
							if (finalValues.Count == arrayValues.Length)
							{
								value = string.Join(',', finalValues);
							}
						}
						SetVariable(name, value);
					}
					else if (child.Name == "variableSet")
					{
						ApplyVariableSet(child, csvMetadata);
					}
				}
			}
		}

		Dictionary<string, string> vars = new Dictionary<string, string>();
	}
	

	static class Extensions
	{
		public static string GetValue(this XElement element, XmlVariableMappings xmlVariableMappings = null)
		{
			string value = element.Value;
			if (xmlVariableMappings != null)
			{
				value = xmlVariableMappings.ResolveVariables(value);
			}
			return value;
		}
		public static T GetRequiredAttribute<T>(this XElement element, string attributeName)
		{
			return GetAttributeInternal<T>(element, null, attributeName, default, true);
		}

		public static T GetRequiredAttribute<T>(this XElement element, XmlVariableMappings xmlVariableMappings, string attributeName)
		{
			return GetAttributeInternal<T>(element, xmlVariableMappings, attributeName, default, true );
		}

		public static T GetSafeAttribute<T>(this XElement element, string attributeName, T defaultValue = default)
		{
			return GetAttributeInternal<T>(element, null,  attributeName, defaultValue, false);
		}

		public static T GetSafeAttribute<T>(this XElement element, XmlVariableMappings xmlVariableMappings, string attributeName, T defaultValue = default)
		{
			return GetAttributeInternal<T>(element, xmlVariableMappings, attributeName, defaultValue, false);
		}

		private static T GetAttributeInternal<T>(this XElement element, XmlVariableMappings xmlVariableMappings, string attributeName, T defaultValue, bool throwIfNotFound)
		{
			XAttribute attribute = element.Attribute(attributeName);
			if (attribute == null)
			{
				if (throwIfNotFound)
				{
					throw new Exception("Attribute "+attributeName+" not found in element "+element.Name);
				}
				return defaultValue;
			}

			// Resolve variables if a mapping is provided
			string attributeValue = attribute.Value;
			if (xmlVariableMappings != null)
			{
				attributeValue = xmlVariableMappings.ResolveVariables(attributeValue);
			}

			if (typeof(T).IsEnum)
			{
#nullable enable
				if (attributeValue != null && Enum.TryParse(typeof(T), attributeValue, true, out object? result))
#nullable disable
				{
					return (T)result;
				}
			}
			try
			{
				switch (Type.GetTypeCode(typeof(T)))
				{
					case TypeCode.Boolean:
						try
						{
							// Support int/bool conversion
							return (T)Convert.ChangeType(Convert.ChangeType(attributeValue, typeof(int)), typeof(bool));
						}
						catch (FormatException)
						{
							// fall back to reading it as an actual bool
							return (T)Convert.ChangeType(attributeValue, typeof(T));
						}
					case TypeCode.Single:
					case TypeCode.Double:
					case TypeCode.Decimal:
						return (T)Convert.ChangeType(attributeValue, typeof(T), CultureInfo.InvariantCulture.NumberFormat);
					default:
						return (T)Convert.ChangeType(attributeValue, typeof(T));
				}
			}
			catch (FormatException e)
			{	
				// If the attribute value is empty (likely due to failed variable mapping), display the original version
				string attributeValueToDisplay = attributeValue.Length > 0 ? attributeValue : attribute.Value;
				Console.WriteLine(string.Format("[Warning] Failed to convert XML attribute '{0}' '{1}' ({2})", attributeName, attributeValueToDisplay, e.Message));
				return defaultValue;
			}
		}
	};
}
