// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using System.Xml.Linq;

namespace PerfSummaries
{
	class BoundedStatValuesSummary : Summary
	{
		class Column
		{
			public string name;
			public string formula;
			public double value;
			public string summaryStatName;
			public string statName;
			public string otherStatName;
			public bool perSecond;
			public bool filterOutZeros;
			public bool applyEndOffset;
			public double multiplier;
			public double threshold;
			public double frameExponent; // Exponent for relative frame time (0-1) in streamingstressmetric formula
			public double statExponent; // Exponent for stat value in streamingstressmetric formula
			public ColourThresholdList colourThresholdList;

			public Column() { }

			public Column(Column inCol)
			{
				name = inCol.name;
				formula = inCol.formula;
				value = inCol.value;
				summaryStatName = inCol.summaryStatName;
				statName = inCol.statName;
				otherStatName = inCol.otherStatName;
				perSecond = inCol.perSecond;
				filterOutZeros = inCol.filterOutZeros;
				applyEndOffset = inCol.applyEndOffset;
				multiplier = inCol.multiplier;
				threshold = inCol.threshold;
				frameExponent = inCol.frameExponent;
				statExponent = inCol.statExponent;
				colourThresholdList = inCol.colourThresholdList;
			}
		};
		public BoundedStatValuesSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			if (stats.Count != 0)
			{
				throw new Exception("<stats> element is not supported");
			}

			title = element.GetSafeAttribute(vars, "title", "Events");
			beginEvent = element.GetSafeAttribute<string>(vars, "beginevent");
			endEvent = element.GetSafeAttribute<string>(vars, "endevent");

			endOffsetPercentage = element.GetSafeAttribute<double>(vars, "endoffsetpercent", 0.0);
			columns = new List<Column>();

			foreach (XElement columnEl in element.Elements("column"))
			{
				Column column = new Column();
				double[] colourThresholds = ReadColourThresholdsXML(columnEl.Element("colourThresholds"), vars);
				if (colourThresholds != null)
				{
					column.colourThresholdList = new ColourThresholdList();
					for (int i = 0; i < colourThresholds.Length; i++)
					{
						column.colourThresholdList.Add(new ThresholdInfo(colourThresholds[i], null));
					}
				}

				column.summaryStatName = columnEl.GetSafeAttribute<string>(vars, "summaryStatName");
				column.statName = columnEl.GetRequiredAttribute<string>(vars, "stat").ToLower();
				if (!stats.Contains(column.statName))
				{
					stats.Add(column.statName);
				}
				column.otherStatName = columnEl.GetSafeAttribute<string>("otherStat", "").ToLower();

				column.name = columnEl.GetRequiredAttribute<string>(vars, "name");
				column.formula = columnEl.GetRequiredAttribute<string>(vars, "formula").ToLower();
				// These attributes have back compat case support. This was fixed in PRT 4.242.0
				column.filterOutZeros = columnEl.GetSafeAttribute<bool>(vars, "filterOutZeros", columnEl.GetSafeAttribute<bool>(vars, "filteroutzeros", false));
				column.perSecond = columnEl.GetSafeAttribute<bool>(vars, "perSecond", columnEl.GetSafeAttribute<bool>(vars, "persecond", false));
				column.multiplier = columnEl.GetSafeAttribute<double>(vars, "multiplier", 1.0);
				column.threshold = columnEl.GetSafeAttribute<double>(vars, "threshold", 0.0);
				column.applyEndOffset = columnEl.GetSafeAttribute<bool>(vars, "applyEndOffset", true);
				column.frameExponent = columnEl.GetSafeAttribute<double>(vars, "frameExponent", 4.0);
				column.statExponent = columnEl.GetSafeAttribute<double>(vars, "statExponent", 0.25);
				columns.Add(column);
			}
		}

		public BoundedStatValuesSummary() { }

		public override string GetName() { return "boundedstatvalues"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;
			int startFrame = -1;
			int endFrame = int.MaxValue;

			// Find the start and end frames based on the events
			if (beginEvent != null)
			{
				foreach (CsvEvent ev in csvStats.Events)
				{
					if (CsvStats.DoesSearchStringMatch(ev.Name, beginEvent))
					{
						startFrame = ev.Frame;
						break;
					}
				}
				if (startFrame == -1)
				{
					Console.WriteLine("BoundedStatValuesSummary: Begin event " + beginEvent + " was not found");
					return htmlSection;
				}
			}
			if (endEvent != null)
			{
				foreach (CsvEvent ev in csvStats.Events)
				{
					if (CsvStats.DoesSearchStringMatch(ev.Name, endEvent))
					{
						endFrame = ev.Frame;
						if (endFrame > startFrame)
						{
							break;
						}
					}
				}
				if (endFrame == int.MaxValue)
				{
					Console.WriteLine("BoundedStatValuesSummary: End event " + endEvent + " was not found");
					return htmlSection;
				}
			}
			if (startFrame >= endFrame)
			{
				Console.WriteLine("Warning: BoundedStatValuesSummary: end event "+ endEvent + " appeared before the start event "+beginEvent);
				return htmlSection;
			}
			endFrame = Math.Min(endFrame, csvStats.SampleCount - 1);
			startFrame = Math.Max(startFrame, 0);

			// Adjust the end frame based on the specified offset percentage, but cache the old value (some columns may need the unmodified one)
			int endEventFrame = Math.Min(csvStats.SampleCount, endFrame + 1);
			if (endOffsetPercentage > 0.0)
			{
				double multiplier = endOffsetPercentage / 100.0;
				endFrame += (int)((double)(endFrame - startFrame) * multiplier);
			}
			endFrame = Math.Min(csvStats.SampleCount, endFrame + 1);
			StatSamples frameTimeStat = csvStats.GetStat("frametime");
			List<float> frameTimes = frameTimeStat.samples;

			List<Column> filteredColumns = new List<Column>();

			//Filtering and wildcard logic
			//This adds new columns for each stat which matches a wildcard column, and also filters the columns to ones
			//that have a valid stat associated with them. The original wildcard column will be removed as well
			foreach (Column col in columns)
			{
				bool nameIsWildcard = col.name != null && col.name.Contains('*');
				bool summaryStatNameIsWildcard = col.summaryStatName != null && col.summaryStatName.Contains('*');
				bool statNameIsWildcard = col.statName != null && col.statName.Contains('*');
				bool otherStatNameIsSet = col.otherStatName != null && col.otherStatName.Length > 0;
				bool otherStatNameIsWildcard = col.otherStatName != null && col.otherStatName.Contains('*');
				bool anyNameWildcard = nameIsWildcard || summaryStatNameIsWildcard || statNameIsWildcard || otherStatNameIsWildcard;
				bool allNameWildcard = nameIsWildcard && summaryStatNameIsWildcard && statNameIsWildcard && (otherStatNameIsSet ? otherStatNameIsWildcard : true);

				//Check all wildcard fields are set, or none of them should be set
				if (anyNameWildcard && !allNameWildcard)
				{
					string errorString = "Warning: BoundedStatValuesSummary: Skipping column because wildcard * was found in some column parameters but not all. name: " + col.name + " summaryStatName: " + col.summaryStatName + " statName: " + col.statName;
					if(otherStatNameIsSet)
					{
						errorString += " otherStatName: " + col.otherStatName;
					}
					Console.WriteLine(errorString);

					continue;
				}

				//Generate new columns based on the wildcard definition
				if (anyNameWildcard && allNameWildcard)
				{
					string[] splitStr = col.statName.Split('*', 2);
					if(splitStr.Length == 0)
					{
						Console.WriteLine("Warning: Skipping stat due to error: StatName is a wildcard and should contain a * symbol: " + col.statName);
						continue;
					}

					//Find each stat that matches the wildcard in statName
					string statPrefix = splitStr[0];
					foreach (KeyValuePair<string, StatSamples> foundStat in csvStats.Stats)
					{
						string foundStatName = foundStat.Key;
						if (foundStatName.ToLower().Contains(statPrefix.ToLower()) && !foundStatName.Equals(col.statName))
						{
							//Create a new column as a copy of the Wildcard Definition Column
							Column newCol = new Column(col);

							string[] splitString = foundStatName.Split(statPrefix, 2);
							if(splitString.Length <= 1)
							{
								Console.WriteLine("Warning: Skipping stat due to error: FoundStatName(" + foundStatName + ") does not contain expected prefix: " + statPrefix);
								continue;
							}
							
							string statPostfix = splitString[1];
							newCol.name = col.name.Substring(0, col.name.IndexOf('*')) + statPostfix;
							newCol.summaryStatName = col.summaryStatName.Substring(0, col.summaryStatName.IndexOf('*')) + statPostfix;
							newCol.statName = col.statName.Substring(0, col.statName.IndexOf('*')) + statPostfix;
							if(csvStats.GetStat(newCol.statName) == null)
							{
								Console.WriteLine("Warning: Skipping stat due to error: StatName not found in csvstats: " + newCol.statName);
								continue;
							}
							
							if (otherStatNameIsSet)
							{
								newCol.otherStatName = col.otherStatName.Substring(0, col.otherStatName.IndexOf('*')) + statPostfix;
								if (csvStats.GetStat(newCol.otherStatName) == null)
								{
									Console.WriteLine("Warning: Skipping stat due to error: StatName not found in csvstats: " + newCol.otherStatName);
									continue;
								}
							}

							//Add newly formed column to the filteredColumns list
							filteredColumns.Add(newCol);							
						}
					}
				}
				else
				{
					// This is a non-wildcard column, so add it as usual
					// Filter only columns with stats that exist in the CSV
					if (csvStats.GetStat(col.statName) != null
						&& (String.IsNullOrWhiteSpace(col.otherStatName) || csvStats.GetStat(col.otherStatName) != null))
					{
						filteredColumns.Add(col); 
					}
				}
			}

			// Nothing to report, so bail out!
			if (filteredColumns.Count == 0)
			{
				return htmlSection;
			}

			// Process the column values
			foreach (Column col in filteredColumns)
			{
				List<float> statValues = csvStats.GetStat(col.statName).samples;
				List<float> otherStatValues = String.IsNullOrWhiteSpace(col.otherStatName) ? null : csvStats.GetStat(col.otherStatName).samples;
				double value = 0.0;
				double totalFrameWeight = 0.0;
				int colEndFrame = col.applyEndOffset ? endFrame : endEventFrame;

				if (col.formula == "average")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (col.filterOutZeros == false || statValues[i] > 0)
						{
							value += statValues[i] * frameTimes[i];
							totalFrameWeight += frameTimes[i];
						}
					}
				}
				else if (col.formula == "unweighted_average")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (col.filterOutZeros == false || statValues[i] > 0)
						{
							value += statValues[i];
							++totalFrameWeight;
						}
					}
				}
				else if (col.formula == "maximum")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (col.filterOutZeros == false || statValues[i] > 0)
						{
							value = statValues[i] > value ? statValues[i] : value;
						}
					}

					totalFrameWeight = 1.0;
				}
				else if (col.formula == "minimum")
				{
					value = double.MaxValue;
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (col.filterOutZeros == false || statValues[i] > 0)
						{
							value = statValues[i] < value ? statValues[i] : value;
						}
					}

					totalFrameWeight = 1.0;
				}
				else if (col.formula == "percentoverthreshold")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (statValues[i] > col.threshold)
						{
							value += frameTimes[i];
						}
						totalFrameWeight += frameTimes[i];
					}
					value *= 100.0;
				}
				else if (col.formula == "percentunderthreshold")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (statValues[i] < col.threshold)
						{
							value += frameTimes[i];
						}
						totalFrameWeight += frameTimes[i];
					}
					value *= 100.0;
				}
				else if (col.formula == "sum")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						value += statValues[i];
					}

					if (col.perSecond)
					{
						double totalTimeMS = 0.0;
						for (int i = startFrame; i < colEndFrame; i++)
						{
							if (col.filterOutZeros == false || statValues[i] > 0)
							{
								totalTimeMS += frameTimes[i];
							}
						}
						value /= (totalTimeMS / 1000.0);
					}
					totalFrameWeight = 1.0;
				}
				else if (col.formula == "sumwhenotheroverthreshold")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (otherStatValues[i] > col.threshold)
						{
							value += statValues[i];
						}
					}

					if (col.perSecond)
					{
						double totalTimeMS = 0.0;
						for (int i = startFrame; i < colEndFrame; i++)
						{
							if ((col.filterOutZeros == false || statValues[i] > 0) && otherStatValues[i] > col.threshold)
							{
								totalTimeMS += frameTimes[i];
							}
						}
						value /= (totalTimeMS / 1000.0);
					}
					totalFrameWeight = 1.0;
				}
				else if (col.formula == "sumwhenotherunderthreshold")
				{
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (otherStatValues[i] < col.threshold)
						{
							value += statValues[i];
						}
					}

					if (col.perSecond)
					{
						double totalTimeMS = 0.0;
						for (int i = startFrame; i < colEndFrame; i++)
						{
							if ((col.filterOutZeros == false || statValues[i] > 0) && otherStatValues[i] < col.threshold)
							{
								totalTimeMS += frameTimes[i];
							}
						}
						value /= (totalTimeMS / 1000.0);
					}
					totalFrameWeight = 1.0;
				}
				else if (col.formula == "streamingstressmetric")
				{
					// Note: tInc is scaled such that it hits 1.0 on the event frame, regardless of the offset
					double tInc = 1.0 / (double)(endEventFrame - startFrame);
					double t = tInc * 0.5;
					for (int i = startFrame; i < colEndFrame; i++)
					{
						if (col.filterOutZeros == false || statValues[i] > 0)
						{
							// Frame weighting is scaled to heavily favor final frames. Note that t can exceed 1 after the event frame if an offset percentage is specified, so we clamp it
							double frameWeight = Math.Pow(Math.Min(t, 1.0), col.frameExponent) * frameTimes[i];

							// If we're past the end event frame, apply a linear falloff to the weight
							if (i >= endEventFrame)
							{
								double falloff = 1.0 - (double)(i - endEventFrame) / (colEndFrame - endEventFrame);
								frameWeight *= falloff;
							}

							// The frame score takes into account the queue depth, but it's not massively significant
							double frameScore = Math.Pow(statValues[i], col.statExponent);
							value += frameScore * frameWeight;
							totalFrameWeight += frameWeight;
						}
						t += tInc;
					}
				}
				else if (col.formula == "ratio")
				{
					double numerator = 0.0;
					for (int i = startFrame; i < colEndFrame; i++)
					{
						numerator += statValues[i];
					}
					double denominator = 0.0;
					for (int i = startFrame; i < colEndFrame; i++)
					{
						denominator += otherStatValues[i];
					}
					// Guard against divide by 0 as json serialization cannot handle Inf.
					value = denominator != 0.0 ? numerator / denominator : 0.0;
					totalFrameWeight = 1.0;
				}
				else
				{
					throw new Exception("BoundedStatValuesSummary: unexpected formula " + col.formula);
				}
				value *= col.multiplier;
				col.value = totalFrameWeight != 0.0 ? value / totalFrameWeight : value;
			}

			// Output HTML
			if (bWriteHtml)
			{
				htmlSection = new HtmlSection(title, bStartCollapsed, MakeSectionId(title));

				htmlSection.WriteLine("  <table border='0' style='width:1400'>");
				htmlSection.WriteLine("  <tr>");
				foreach (Column col in filteredColumns)
				{
					htmlSection.WriteLine("<th>" + col.name + "</th>");
				}
				htmlSection.WriteLine("  </tr>");
				htmlSection.WriteLine("  <tr>");
				foreach (Column col in filteredColumns)
				{
					string bgcolor = "'#ffffff'";
					if (col.colourThresholdList != null)
					{
						bgcolor = col.colourThresholdList.GetColourForValue(col.value);
					}
					htmlSection.WriteLine("<td bgcolor=" + bgcolor + ">" + col.value.ToString("0.00") + "</td>");
				}
				htmlSection.WriteLine("  </tr>");
				htmlSection.WriteLine("  </table>");
			}

			// Output summary table row data
			if (rowData != null)
			{
				foreach (Column col in filteredColumns)
				{
					if (col.summaryStatName != null)
					{
						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, col.summaryStatName, col.value, col.colourThresholdList);
					}
				}
			}
			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}
		string title;
		string beginEvent;
		string endEvent;
		double endOffsetPercentage;
		List<Column> columns;
	};

}