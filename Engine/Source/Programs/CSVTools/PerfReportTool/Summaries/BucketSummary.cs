// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;
using System.Collections;

namespace PerfSummaries
{
	class BucketSummary : Summary
	{
		private enum ESortMode
		{
			None,
			Alpha,
			BucketIndex
		}

		private enum ESortDirection
		{ 
			Ascending,
			Descending,
		}

		public BucketSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			Title = element.GetSafeAttribute(vars, "title", "Untitled Bucket Summary");

			XElement BucketElement = element.Element("buckets");
			bool ReportOutOfRangeDefault = BucketElement.GetSafeAttribute<bool>(vars, "reportOutOfRange", false);
			// eg. <buckets reportAboveRange="1" reportBelowRange="1" precision="0.0">
			ReportBelowRange = BucketElement.GetSafeAttribute<bool>(vars, "reportBelowRange", ReportOutOfRangeDefault);
			ReportAboveRange = BucketElement.GetSafeAttribute<bool>(vars, "reportAboveRange", ReportOutOfRangeDefault);
			ColumnHeaderPrecision = BucketElement.GetSafeAttribute<string>(vars, "precision", "0");

			string[] XmlBuckets = BucketElement.Value.Split(',');
			Buckets = new double[XmlBuckets.Length];
			for (int i = 0; i < XmlBuckets.Length; i++)
			{
				Buckets[i] = Convert.ToDouble(XmlBuckets[i], System.Globalization.CultureInfo.InvariantCulture);
			}

			LowEndColor  = new Colour(1.0f, 1.0f, 1.0f, 1.0f);
			HighEndColor = new Colour(1.0f, 1.0f, 1.0f, 1.0f);
			XElement ColorElement = element.Element("colorDisplay");
			if (ColorElement != null)
			{
				LowEndColor = new Colour(ColorElement.GetSafeAttribute<string>(vars, "lowEndColor"));
				HighEndColor = new Colour(ColorElement.GetSafeAttribute<string>(vars, "highEndColor"));
			}

			// eg. <sort mode="bucketIndex" bucketIndex="0" sortDirection="descending"></sort>
			XElement SortElement = element.Element("sort");
			if (SortElement != null)
			{
				string SortModeStr = SortElement.GetSafeAttribute<string>(vars, "mode", null);
				SortModeStr = SortModeStr != null ? SortModeStr.ToLower() : null;
				if (SortModeStr == "alpha")
				{
					SortMode = ESortMode.Alpha;
				}
				else if (SortModeStr == "bucketindex")
				{
					SortMode = ESortMode.BucketIndex;
					SortByBucketIndex = SortElement.GetRequiredAttribute<int>(vars, "bucketIndex");
				}

				string SortDirectionStr = SortElement.GetSafeAttribute<string>(vars, "sortDirection", "ascending").ToLower();
				SortDirection = SortDirectionStr == "ascending" ? ESortDirection.Ascending : ESortDirection.Descending;
			}
		}
		public BucketSummary() { }

		public override string GetName() { return "bucketsummary"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			// Only HTML reporting is supported (does not summary table row data)
			if (!bWriteHtml)
			{
				return null;
			}

			if (Buckets.Length == 0)
			{
				return null;
			}

			HtmlSection htmlSection = new HtmlSection(Title, bStartCollapsed, MakeSectionId(Title));

			htmlSection.WriteLine("  <table border='0' style='width:1000'>");
			htmlSection.WriteLine("  <tr><td></td>");

			List<string> Hitches = new List<string>();

			if (ReportBelowRange)
			{
				htmlSection.WriteLine("  <th> <" + Buckets[0].ToString(ColumnHeaderPrecision) + "</b></td>");
			}

			for (int i = 1; i < Buckets.Length; ++i)
			{
				double Begin = Buckets[i - 1];
				double End = Buckets[i];

				htmlSection.WriteLine("  <th> [" + Begin.ToString(ColumnHeaderPrecision) + ", " + End.ToString(ColumnHeaderPrecision) + ")"+ "</b></td>");
			}

			if (ReportAboveRange)
			{
				htmlSection.WriteLine("  <th> >=" + Buckets.Last().ToString(ColumnHeaderPrecision) + "</b></td>");
			}

			htmlSection.WriteLine("  </tr>");

			// Create a copy of the stats if we need to sort them.
			List<string> sortedStats = SortMode == ESortMode.None ? stats : new List<string>(stats);
			if (SortMode == ESortMode.Alpha)
			{
				sortedStats.Sort((string a, string b) =>
				{
					return SortDirection == ESortDirection.Ascending ? a.CompareTo(b) : b.CompareTo(a);
				});
			}
			else if (SortMode == ESortMode.BucketIndex)
			{
				if (SortByBucketIndex >= 0 && SortByBucketIndex < Buckets.Length)
				{
					float BucketValue = (float)Buckets[SortByBucketIndex];

					sortedStats.Sort((string a, string b) =>
					{
						StatSamples StatsA = csvStats.GetStat(a.ToLower());
						StatSamples StatsB = csvStats.GetStat(b.ToLower());
						int CountA = StatsA.GetCountOfFramesAtOrOverBudget(BucketValue);
						int CountB = StatsB.GetCountOfFramesAtOrOverBudget(BucketValue);
						return SortDirection == ESortDirection.Ascending ? CountA.CompareTo(CountB) : CountB.CompareTo(CountA);
					});
				}
			}

			foreach (string unitStat in sortedStats)
			{
				string StatToCheck = unitStat.Split('(')[0];

				StatSamples Stats = csvStats.GetStat(StatToCheck.ToLower());
				if (Stats == null)
				{
					continue;
				}

				Hitches.Clear();
				htmlSection.WriteLine("  <tr><td><b>" + StatToCheck + "</b></td>");
				Hitches.Add(StatToCheck);

				int[] BucketCounts = new int[Buckets.Length + 1];
				int TotalSamples = Stats.GetNumSamples();
				TotalSamples -= 2; // First and last frames are not counted in GetCountOfFramesAtOrOverBudget.

				BucketCounts[0] = TotalSamples - Stats.GetCountOfFramesAtOrOverBudget((float)Buckets[0]);
				int SamplesAccountedFor = BucketCounts[0];

				for (int i = 0; i < Buckets.Length - 1; ++i)
				{
					double Begin = Buckets[i];
					double End = Buckets[i + 1];
					int BeginCount = (int)Stats.GetCountOfFramesAtOrOverBudget((float)Buckets[i]);
					int EndCount = (int)Stats.GetCountOfFramesAtOrOverBudget((float)Buckets[i + 1]);
					int BucketCount = BeginCount - EndCount;
					BucketCounts[i + 1] = BucketCount;
					SamplesAccountedFor += BucketCount;
				}
				BucketCounts[BucketCounts.Length - 1] = TotalSamples - SamplesAccountedFor;

				int FirstIndex = ReportBelowRange ? 0 : 1;
				int IndexCount = ReportAboveRange ? BucketCounts.Length : BucketCounts.Length - 1;
				int HighestBucketCount = 0;
				for (int i = FirstIndex; i < IndexCount; ++i)
				{
					HighestBucketCount = Math.Max(HighestBucketCount, BucketCounts[i]);
				}

				Colour White = new Colour(1.0f, 1.0f, 1.0f, 1.0f);
				for (int i = FirstIndex; i < IndexCount; ++i)
				{
					int Count = BucketCounts[i];
					float T = (float)i / (float)(IndexCount - 1);
					Colour Color = Colour.LerpUsingHSV(LowEndColor, HighEndColor, T);
					float Intensity = Count / (float)HighestBucketCount;
					Color = Colour.Lerp(White, Color, Intensity);
					string ColorString = Color.ToHTMLString();

					htmlSection.WriteLine("  <td bgcolor=" + ColorString + ">" + Count.ToString("0") + "</td>");
					Hitches.Add(Count.ToString("0"));
				}

				htmlSection.WriteLine("  </tr>");
			}
			htmlSection.WriteLine("  </table>");
			return htmlSection;
		}

		public double[] Buckets;
		public string Title;
		public bool ReportBelowRange;
		public bool ReportAboveRange;
		public Colour LowEndColor;
		public Colour HighEndColor;
		private string ColumnHeaderPrecision = "0";
		// Sorting
		ESortMode SortMode = ESortMode.None;
		ESortDirection SortDirection;
		private int SortByBucketIndex = -1;
	};

}