// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;
using System.Xml.Serialization;
using System.Data.Common;
using System.Drawing;

namespace PerfSummaries
{
	class CheckpointSummary : Summary
	{
		class CaptureEventPoint
		{
			public string friendlyName;
			public string eventString;
			public string metricSuffix;

			public CaptureEventPoint() { }

			public CaptureEventPoint(string inFriendlyName, string inEventString, string inMetricSuffix)
			{
				friendlyName = inFriendlyName;
				eventString = inEventString;
				metricSuffix = inMetricSuffix;
			}
		}

		public CheckpointSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);

			if (element == null)
			{
				return;
			}

			metricPrefix = element.GetSafeAttribute<string>(vars, "metricPrefix");
			summaryTitle = element.GetSafeAttribute<string>(vars, "title");

			captureEventPoints = new List<CaptureEventPoint>();

			foreach (XElement child in element.Elements())
			{
				if (child.Name == "eventCapture")
				{
					string friendlyName = child.GetRequiredAttribute<string>(vars, "friendlyName");
					string eventString = child.GetRequiredAttribute<string>(vars, "eventString");
					string metricSuffix = child.GetSafeAttribute<string>(vars, "metricSuffix");

					captureEventPoints.Add(new CaptureEventPoint(friendlyName, eventString, metricSuffix));
				}
			}
		}

		public CheckpointSummary()
		{
		}

		public override string GetName() { return "checkpoint"; }

		public List<int> GetFramesMatchingEventString(string inEventString, CsvStats csvStats)
		{
			List<int> frames = new List<int>();

			for (int i = 0; i < csvStats.Events.Count; i++)
			{
				if (CsvStats.DoesSearchStringMatch(csvStats.Events[i].Name, inEventString))
				{
					frames.Add(csvStats.Events[i].Frame);
				}
			}

			return frames;
		}

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;
			CsvStats statsToUse = useUnstrippedCsvStats ? csvStatsUnstripped : csvStats;

			// No events means we shouldn't do any work
			if (statsToUse.Events.Count == 0)
			{
				return null;
			}

			if (bWriteHtml)
			{
				// write out all the headers to the HTML
				htmlSection = new HtmlSection(summaryTitle, bStartCollapsed, MakeSectionId(summaryTitle));
				string HeaderRow = "";

				HeaderRow += "<th>Checkpoint Name</th>";

				// Write the stats as headers 
				foreach (string statName in stats)
				{
					string baseStatName = statName;

					int bracketIndex = statName.IndexOf('(');
					if (bracketIndex != -1)
					{
						baseStatName = statName.Substring(0, bracketIndex);
					}

					if (!csvStats.Stats.ContainsKey(baseStatName.ToLower()))
					{
						continue;
					}

					HeaderRow += "<th>" + TableUtil.FormatStatName(baseStatName) + "</th>";
				}

				htmlSection.WriteLine("<table border='0' style='width:400'>");
				htmlSection.WriteLine("  <tr>" + HeaderRow + "</tr>");
			}

			// Per-event breakdown
			foreach (CaptureEventPoint capturePoint in captureEventPoints)
			{
				// Get the list of frames that match our event string
				List<int> framesToSample = GetFramesMatchingEventString(capturePoint.eventString, statsToUse);

				if (framesToSample == null || framesToSample.Count == 0)
				{
					continue;
				}

				// only care about first frame occurrence of the event
				int frameToSample = framesToSample[0];
				string ValueRow = "";

				// Update the CSV stat values
				foreach (string statName in stats)
				{
					string baseStatName = statName;

					int bracketIndex = statName.IndexOf('(');
					if (bracketIndex != -1)
					{
						baseStatName = statName.Substring(0, bracketIndex);
					}

					if (!csvStats.Stats.ContainsKey(baseStatName.ToLower()))
					{
						continue;
					}

					StatSamples stat = statsToUse.Stats[baseStatName.ToLower()];

					if (stat.GetNumSamples() == 0)
					{
						continue;
					}

					float value = stat.samples[frameToSample];

					if (htmlSection != null)
					{
						ValueRow += "<td>" + value.ToString("0.00") + "</td>";
					}

					// Requires capture event metricSuffix and summary metricPrefix to be something, otherwise no metric data will be written
					if (rowData != null && metricPrefix.Length > 0 && capturePoint.metricSuffix.Length > 0)
					{
						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, metricPrefix + "_" + baseStatName + "_" + capturePoint.metricSuffix, (double)value);
					}
				}

				if (ValueRow.Length > 0 && htmlSection != null)
				{
					// Prepends the friendly name only if we have a row to write out.
					ValueRow = "<td>" + capturePoint.friendlyName + "</td>" + ValueRow;
					htmlSection.WriteLine("  <tr>" + ValueRow + "</tr>");
				}
			}

			if (htmlSection != null)
			{
				htmlSection.WriteLine("</table>");
			}

			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}

		List<CaptureEventPoint> captureEventPoints;
		string metricPrefix;
		string summaryTitle;
	};

}