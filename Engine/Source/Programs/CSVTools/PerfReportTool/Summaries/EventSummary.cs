// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;

namespace PerfSummaries
{
	class EventSummary : Summary
	{
		public EventSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			title = element.GetSafeAttribute(vars, "title", "Events");
			summaryStatName = element.GetRequiredAttribute<string>(vars, "summaryStatName");
			events = element.Element("events").Value.Split(',');
			colourThresholds = ReadColourThresholdsXML(element.Element("colourThresholds"), vars);
			bSortByName = element.GetSafeAttribute<bool>("sortByName", false);
		}

		public EventSummary() { }

		public override string GetName() { return "event"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;
			Dictionary<string, int> eventCountsDict = new Dictionary<string, int>();
			int eventCount = 0;
			foreach (CsvEvent ev in csvStats.Events)
			{
				foreach (string eventName in events)
				{
					if (CsvStats.DoesSearchStringMatch(ev.Name, eventName))
					{
						int len = eventName.Length;
						if (eventName.EndsWith("*"))
						{
							len--;
						}
						string eventContent = ev.Name.Substring(len).Trim();
						if (eventCountsDict.ContainsKey(eventContent))
						{
							eventCountsDict[eventContent]++;
						}
						else
						{
							eventCountsDict.Add(eventContent, 1);
						}
						eventCount++;
					}
				}
			}

			// Sort descending
			if (bSortByName)
			{
				eventCountsDict = eventCountsDict
					.OrderBy(pair => pair.Key)
					.ToDictionary(x => x.Key, x => x.Value);
			}
			else
			{
				eventCountsDict = eventCountsDict
					.OrderBy(pair => -pair.Value)
					.ToDictionary(x => x.Key, x => x.Value);
			}

			// Output HTML
			if (bWriteHtml && eventCountsDict.Count > 0)
			{
				htmlSection = new HtmlSection(title, bStartCollapsed, MakeSectionId(title));
				htmlSection.WriteLine("  <table border='0' style='width:1200'>");
				htmlSection.WriteLine("  <tr><th>Name</th><th><b>Count</th></tr>");
				foreach (KeyValuePair<string, int> pair in eventCountsDict.ToList())
				{
					htmlSection.WriteLine("  <tr><td>" + pair.Key + "</td><td>" + pair.Value + "</td></tr>");
				}
				htmlSection.WriteLine("  </table>");
			}

			// Output summary table row data
			if (rowData != null)
			{
				ColourThresholdList thresholdList = null;

				if (colourThresholds != null)
				{
					thresholdList = new ColourThresholdList();
					for (int i = 0; i < colourThresholds.Length; i++)
					{
						thresholdList.Add(new ThresholdInfo(colourThresholds[i]));
					}
				}
				rowData.Add(SummaryTableElement.Type.SummaryTableMetric, summaryStatName, (double)eventCount, thresholdList);
			}
			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}
		string[] events;
		double[] colourThresholds;
		string title;
		string summaryStatName;
		bool bSortByName;
	};

}