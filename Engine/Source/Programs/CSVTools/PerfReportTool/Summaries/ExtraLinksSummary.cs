// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using System.IO;
using CSVStats;

namespace PerfSummaries
{
	class ExtraLinksSummary : Summary
	{
		class ExtraLink
		{
			public ExtraLink(string fileLine, string inLinkTemplateCsvId)
			{
				string[] Sections = fileLine.Split(',');
				if (Sections.Length < 3)
                {
					throw new Exception("Bad links line format: " + fileLine);
				}
				ReportText = Sections[0];
                SummaryTableText = Sections[1];
				LinkURL = Sections[2];
				bool bNeedsTemplate = true;
				string linkTemplateMode = (Sections.Length >= 4) ? Sections[3] : null;
				if (linkTemplateMode == null || linkTemplateMode == "auto")
				{
					// Auto detect if template is needed. If this is a web URL then we assume not
					if (LinkURL.StartsWith("http://") || LinkURL.StartsWith("https://") )
					{
						bNeedsTemplate = false;
					}
				}
				else if (linkTemplateMode == "noLinkTemplate")
				{
					bNeedsTemplate = false;
				}
				else if (linkTemplateMode != "linkTemplate")
				{
					throw new Exception("Bad link type in line: " + fileLine);
				}

				// If this is a file link then we'll need to template so it can be fixed up externally
				if (bNeedsTemplate)
				{
					LinkTemplateCsvId = inLinkTemplateCsvId;
				}
			}
            public string GetLinkString(bool bIsSummaryTable)
            {
                if (bIsSummaryTable && LinkTemplateCsvId != null)
                {
                    // Output a link template if link templates are enabled and we have a CSV
                    return "{LinkTemplate:" + SummaryTableText.Replace(" ", "") + ":" + LinkTemplateCsvId + "}";
                }
                string Text = bIsSummaryTable ? SummaryTableText : ReportText;
                return "<a href='" + LinkURL + "'>" + Text + "</a>";
            }
            public string ReportText;
			public string SummaryTableText;
			public string LinkURL;
            public string LinkTemplateCsvId;
        };
        public ExtraLinksSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory, bool bInLinkTemplates=false)
        {
			ReadStatsFromXML(element, vars);
			title = "Links";
            if (element != null)
            {
                title = element.GetSafeAttribute(vars, "title", title);
            }
            bLinkTemplates = bInLinkTemplates;
        }

		public ExtraLinksSummary() { }

		public override string GetName() { return "extralinks"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;
			List<ExtraLink> links = new List<ExtraLink>();

            string csvId = null;
            if (bLinkTemplates && csvStats.metaData != null)
            {
                csvId = csvStats.metaData.GetValue("csvid", null);
            }


            string csvFilename = csvStats.metaData.GetValue("csvfilename", null);
			if (csvFilename == null)
			{
				Console.WriteLine("Can't find CSV filename for ExtraLinks summary. Skipping");
				return null;
			}

			string linksFilename = csvFilename + ".links";
			if (!File.Exists(linksFilename))
			{
				Console.WriteLine("Can't find file " + linksFilename + " for ExtraLinks summary. Skipping");
				return null;
			}
			string[] lines = File.ReadAllLines(linksFilename);
			foreach (string line in lines)
			{
				links.Add(new ExtraLink(line, csvId));
			}
			if (links.Count == 0)
			{
				return null;
			}

			// Output HTML
			if (bWriteHtml)
			{
				htmlSection = new HtmlSection(title, bStartCollapsed, MakeSectionId(title));
				htmlSection.WriteLine("  <ul>");
				foreach (ExtraLink link in links)
				{
					htmlSection.WriteLine("  <li>" + link.GetLinkString(false) + "</li>");
				}
				htmlSection.WriteLine("  </ul>");
			}

			// Output summary row data
			if (rowData != null)
			{
				foreach (ExtraLink link in links)
				{
					rowData.Add(SummaryTableElement.Type.SummaryTableMetric, link.ReportText, link.GetLinkString(true), null);
				}
			}

			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
        }
		string title;
        bool bLinkTemplates;
	};

}