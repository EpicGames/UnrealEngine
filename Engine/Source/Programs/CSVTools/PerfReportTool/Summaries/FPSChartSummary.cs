// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;

namespace PerfSummaries
{
	class FPSChartSummary : Summary
	{
		public FPSChartSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			fps = element.GetRequiredAttribute<int>(vars, "fps");
			recommendedFps = element.GetSafeAttribute<int>(vars, "recommendedFps", 0);
			slowFrameTimeBuffer = element.GetSafeAttribute<float>(vars, "slowFrameTimeBuffer", 0);
			hitchThreshold = (float)element.GetRequiredAttribute<double>(vars, "hitchThreshold");
			bUseEngineHitchMetric = element.GetSafeAttribute<bool>(vars, "useEngineHitchMetric", false);
			if (bUseEngineHitchMetric)
			{
				engineHitchToNonHitchRatio = element.GetSafeAttribute<float>(vars, "engineHitchToNonHitchRatio", 1.5f);
				engineMinTimeBetweenHitchesMs = element.GetSafeAttribute<float>(vars, "engineMinTimeBetweenHitchesMs", 200.0f);
			}

			bIgnoreHitchTimePercent = element.GetSafeAttribute<bool>(vars, "ignoreHitchTimePercent", false);
			bIgnoreMVP = element.GetSafeAttribute<bool>(vars, "ignoreMVP", false);
			bIgnoreSlowFrameTimePercent = element.GetSafeAttribute<bool>(vars, "ignoreSlowFrameTimePercent", false);
		}

		public FPSChartSummary() { }

		public override string GetName() { return "fpschart"; }

		float GetEngineHitchToNonHitchRatio()
		{
			float MinimumRatio = 1.0f;
			float targetFrameTime = 1000.0f / fps;
			float MaximumRatio = hitchThreshold / targetFrameTime;

			return Math.Min(Math.Max(engineHitchToNonHitchRatio, MinimumRatio), MaximumRatio);
		}

		struct FpsChartData
		{
			public float MVP;
			public float HitchesPerMinute;
			public float HitchTimePercent;
			public int HitchCount;
			public float TotalTimeSeconds;
			public float? SlowFrameTimePerecent;
		};

		FpsChartData ComputeFPSChartDataForFrames(List<float> frameTimes, bool skiplastFrame)
		{
			double totalFrametime = 0.0;
			int hitchCount = 0;
			double totalHitchTime = 0.0;

			int frameCount = skiplastFrame ? frameTimes.Count - 1 : frameTimes.Count;

			// Count hitches
			if (bUseEngineHitchMetric)
			{
				// Minimum time passed before we'll record a new hitch
				double CurrentTime = 0.0;
				double LastHitchTime = float.MinValue;
				double LastFrameTime = float.MinValue;
				float HitchMultiplierAmount = GetEngineHitchToNonHitchRatio();

				for (int i = 0; i < frameCount; i++)
				{
					float frametime = frameTimes[i];
					// How long has it been since the last hitch we detected?
					if (frametime >= hitchThreshold)
					{
						double TimeSinceLastHitch = (CurrentTime - LastHitchTime);
						if (TimeSinceLastHitch >= engineMinTimeBetweenHitchesMs)
						{
							// For the current frame to be considered a hitch, it must have run at least this many times slower than
							// the previous frame

							// If our frame time is much larger than our last frame time, we'll count this as a hitch!
							if (frametime > (LastFrameTime * HitchMultiplierAmount))
							{
								LastHitchTime = CurrentTime;
								hitchCount++;
							}
						}
						totalHitchTime += frametime;
					}
					LastFrameTime = frametime;
					CurrentTime += (double)frametime;
				}
				totalFrametime = CurrentTime;
			}
			else
			{
				for (int i = 0; i < frameCount; i++)
				{
					float frametime = frameTimes[i];
					totalFrametime += frametime;
					if (frametime >= hitchThreshold)
					{
						hitchCount++;
					}
				}
			}
			float TotalSeconds = (float)totalFrametime / 1000.0f;
			float TotalMinutes = TotalSeconds / 60.0f;

			FpsChartData outData = new FpsChartData();
			outData.HitchCount = hitchCount;
			outData.TotalTimeSeconds = TotalSeconds;
			outData.HitchesPerMinute = (float)hitchCount / TotalMinutes;

			// subtract hitch threshold to weight larger hitches
			totalHitchTime -= (hitchCount * hitchThreshold);
			outData.HitchTimePercent = (float)(totalHitchTime / totalFrametime) * 100.0f;

			if (!bIgnoreSlowFrameTimePercent && recommendedFps > 0 && fps > 0 && TotalSeconds > 0.0f)
			{
				// If the user is targeting a lower fps than we recommend then use that as the baseline instead.
				// Otherwise we'll incorrectly compute a high SFP due to the frame rate being limited to a lower value.
				float minTargetFps = (float)Math.Min(recommendedFps, fps);
				float frameTimeTargetMs = (1.0f / minTargetFps) * 1000.0f;
				// We allow for some tolerance in determining what is a slow frame to account for natural noise in timings.
				float frameTimeThreshold = frameTimeTargetMs + slowFrameTimeBuffer;

				// Accumulate the total time spent over the frame time target each frame.
				float totalOverBudget = 0.0f;
				for (int i = 0; i < frameCount; i++)
				{
					if (frameTimes[i] > frameTimeThreshold)
					{
						totalOverBudget += frameTimes[i] - frameTimeTargetMs;
					}
				}

				// The final value is the % of time spent over the frame time target.
				float totalOverBudgetSeconds = totalOverBudget / 1000.0f;
				outData.SlowFrameTimePerecent = (totalOverBudgetSeconds / TotalSeconds) * 100.0f; // Scale 0-100
			}
			else
			{
				// Avoid outputting 0 if the required values weren't available as it may be misleading.
				outData.SlowFrameTimePerecent = null;
			}

			// If frame count is low enough this can get rounded down to 0
			int TotalTargetFrames = (int)((double)fps * (TotalSeconds));
			if (TotalTargetFrames > 0)
			{
				int MissedFrames = Math.Max(TotalTargetFrames - frameTimes.Count, 0);
				outData.MVP = (((float)MissedFrames * 100.0f) / (float)TotalTargetFrames);
			}
			else
			{
				outData.MVP = 0;
			}
			return outData;
		}

		class ColumnInfo
		{
			public ColumnInfo(string inName, double inValue, ColourThresholdList inColorThresholds, string inDetails=null, bool inIsAvgValue=false)
			{
				Name = inName;
				Value = inValue;
				ColorThresholds = inColorThresholds; 
				isAverageValue = inIsAvgValue;
				Details = inDetails;

				UpdateColor();
			}

			public void UpdateColor()
			{
				Color = ColourThresholdList.GetSafeColourForValue(ColorThresholds, Value);
			}

			public string Name;
			public double Value;
			public string Color;
			public ColourThresholdList ColorThresholds;
			public string Details;
			public bool isAverageValue;
		};

		private ColourThresholdList ComputeFrameTimeColorThresholdsFromMVP( int fps )
		{
			ColourThresholdList mvpColorThresholdList = GetStatColourThresholdList("MVP"+ fps.ToString());
			ColourThresholdList colorThresholdListOut = null;
			if (mvpColorThresholdList != null && mvpColorThresholdList.Count == 4)
			{
				colorThresholdListOut = new ColourThresholdList();
				double idealFrameTime = 1000.0 / (double)fps;
				for (int i = 0; i < 4; i++)
				{
					// MVP = (1/idealFrameTime - 1/avgFrameTime) * idealFrameTime
					// avgFrameTime = idealFrameTime / (1 - MVP)
					ThresholdInfo mvpThreshold = mvpColorThresholdList.Thresholds[i];
					double frameTimeThreshold = idealFrameTime / (1.0 - mvpThreshold.value / 100.0);
					colorThresholdListOut.Thresholds.Add(new ThresholdInfo(frameTimeThreshold, mvpThreshold.colour));
				}
			}
			return colorThresholdListOut;
		}

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;

			System.IO.StreamWriter statsCsvFile = null;
			if (bWriteSummaryCsv)
			{
				string csvPath = Path.Combine(Path.GetDirectoryName(htmlFileName), "FrameStats_colored.csv");
				statsCsvFile = new System.IO.StreamWriter(csvPath, false);
			}
			// Compute MVP30 and MVP60. Note: we ignore the last frame because fpscharts can hitch
			List<float> frameTimes = csvStats.Stats["frametime"].samples;
			FpsChartData fpsChartData = ComputeFPSChartDataForFrames(frameTimes, true);

			// Write the averages
			List<ColumnInfo> Columns = new List<ColumnInfo>();

			Columns.Add( new ColumnInfo("Total Time (s)", fpsChartData.TotalTimeSeconds, new ColourThresholdList() ) );
			Columns.Add(new ColumnInfo("Hitches/Min", fpsChartData.HitchesPerMinute, GetStatColourThresholdList("Hitches/Min") ) );
			
			if (!bIgnoreHitchTimePercent)
			{
				Columns.Add(new ColumnInfo("HitchTimePercent", fpsChartData.HitchTimePercent, GetStatColourThresholdList("HitchTimePercent")));
			}

			if (!bIgnoreSlowFrameTimePercent && fpsChartData.SlowFrameTimePerecent.HasValue)
			{
				Columns.Add(new ColumnInfo("SlowFrameTimePercent", fpsChartData.SlowFrameTimePerecent.Value, GetStatColourThresholdList("SlowFrameTimePercent")));
			}

			string MvpStatName = "MVP" + fps.ToString();
			if (!bIgnoreMVP)
			{
				Columns.Add(new ColumnInfo(MvpStatName, fpsChartData.MVP, GetStatColourThresholdList(MvpStatName)));
			}

			// Output CSV stats
			int csvStatColumnStartIndex = Columns.Count;
			foreach (string statName in stats)
			{
				string baseStatName = statName;

				string [] statAttributes=new string[0];
				int bracketIndex = statName.IndexOf('(');
				if (bracketIndex != -1)
				{
					baseStatName = statName.Substring(0,bracketIndex);
					int rBracketIndex = statName.LastIndexOf(')');
					if (rBracketIndex > bracketIndex)
					{
						string attributesStr = statName.Substring(bracketIndex + 1, rBracketIndex - (bracketIndex + 1)).ToLower();
						statAttributes = attributesStr.Split(' ');
					}
				}

				string ValueType = " Avg";
				bool bIsAvg = false;
				if (!csvStats.Stats.ContainsKey(baseStatName.ToLower()))
				{
					continue;
				}

				bool bUnstripped = statAttributes.Contains("unstripped");
				StatSamples stat = bUnstripped ? csvStatsUnstripped.Stats[baseStatName.ToLower()] : csvStats.Stats[baseStatName.ToLower()];
				float value;
				if (statAttributes.Contains("min"))
				{
					value = stat.ComputeMinValue();
					ValueType = " Min";
				}
				else if (statAttributes.Contains("max"))
				{
					value = stat.ComputeMaxValue();
					ValueType = " Max";
				}
				else
				{
					value = stat.average;
					bIsAvg = true;
				}

				string detailStr = null;
				if (bUnstripped)
				{
					detailStr = "all frames";
				}

				ColourThresholdList colorThresholdList = GetStatColourThresholdList(statName);

				// If the frametime color thresholds are not specified then compute them based on MVP
				if (!bIgnoreMVP && colorThresholdList == null && baseStatName.ToLower() == "frametime" && fps > 0)
				{
					colorThresholdList = ComputeFrameTimeColorThresholdsFromMVP(fps);
				}
				Columns.Add(new ColumnInfo(baseStatName + ValueType, value, colorThresholdList, detailStr, bIsAvg));
			}

			// Output summary table row data
			if (rowData != null)
			{
				foreach (ColumnInfo column in Columns)
				{
					string columnName = column.Name;

					// Output simply MVP to rowData instead of MVP30 etc
					if (columnName.StartsWith("MVP"))
					{
						columnName = "MVP";
					}
					// Hide pre-existing stats with the same name
					if (column.isAverageValue && columnName.EndsWith(" Avg"))
					{
						string originalStatName = columnName.Substring(0, columnName.Length - 4).ToLower();
						SummaryTableElement smv;
						if (rowData.dict.TryGetValue(originalStatName, out smv))
						{
							if (smv.type == SummaryTableElement.Type.CsvStatAverage)
							{
								smv.SetFlag(SummaryTableElement.Flags.Hidden, true);
							}
						}
					}
					rowData.Add(SummaryTableElement.Type.SummaryTableMetric, columnName, column.Value, column.ColorThresholds);
				}
				rowData.Add(SummaryTableElement.Type.SummaryTableMetric, "TargetFPS", (double)fps);
			}

			// Output HTML
			if (bWriteHtml)
			{
				htmlSection = new HtmlSection("FPSChart", bStartCollapsed, MakeSectionId("FPSChart"));
				string HeaderRow = "";
				string ValueRow = "";
				HeaderRow += "<th>Section Name</th>";
				ValueRow += "<td>Entire Run</td>";
				foreach (ColumnInfo column in Columns)
				{ 
					string columnName = column.Name;
					if (columnName.ToLower().EndsWith("time"))
					{
						columnName += " (ms)";
					}

					if (column.Details != null)
					{
						columnName += " ("+ column.Details + ")";
					}

					HeaderRow += "<th>" + TableUtil.FormatStatName(columnName) + "</th>";
					ValueRow += "<td bgcolor=" + column.Color + ">" + column.Value.ToString("0.00") + "</td>";
				}
				htmlSection.WriteLine("<table border='0' style='width:400'>");
				htmlSection.WriteLine("  <tr>" + HeaderRow + "</tr>");
				htmlSection.WriteLine("  <tr>" + ValueRow + "</tr>");
			}

			// Output CSV
			if (statsCsvFile != null)
			{
				List<string> ColumnNames = new List<string>();
				List<double> ColumnValues = new List<double>();
				List<string> ColumnColors = new List<string>();
				foreach (ColumnInfo column in Columns)
				{
					ColumnNames.Add(column.Name);
					ColumnValues.Add(column.Value);
					ColumnColors.Add(column.Color);
				}
				statsCsvFile.Write("Section Name,");
				statsCsvFile.WriteLine(string.Join(",", ColumnNames));

				statsCsvFile.Write("Entire Run,");
				statsCsvFile.WriteLine(string.Join(",", ColumnValues));

				// Pass through color data as part of database-friendly stuff.
				statsCsvFile.Write("Entire Run BGColors,");
				statsCsvFile.WriteLine(string.Join(",", ColumnColors));
			}



			if (csvStats.Events.Count > 0)
			{
				Dictionary<string, ColumnInfo> ColumnDict = new Dictionary<string, ColumnInfo>();
				foreach (ColumnInfo column in Columns)
				{
					ColumnDict[column.Name] = column;
				}

				// Per-event breakdown
				foreach (CaptureRange CapRange in captures)
				{
					CaptureData CaptureFrameTimes = GetFramesForCapture(CapRange, frameTimes, csvStats.Events);

					if (CaptureFrameTimes == null)
					{
						continue;
					}
					FpsChartData captureFpsChartData = ComputeFPSChartDataForFrames(CaptureFrameTimes.Frames, true);

					if (captureFpsChartData.TotalTimeSeconds == 0.0f)
					{
						continue;
					}

					ColumnDict["Total Time (s)"].Value = captureFpsChartData.TotalTimeSeconds;
					ColumnDict["Hitches/Min"].Value = captureFpsChartData.HitchesPerMinute;

					if (!bIgnoreHitchTimePercent)
					{
						ColumnDict["HitchTimePercent"].Value = captureFpsChartData.HitchTimePercent;
					}

					if (!bIgnoreMVP)
					{
						ColumnDict[MvpStatName].Value = captureFpsChartData.MVP;
					}

					if (!bIgnoreSlowFrameTimePercent && captureFpsChartData.SlowFrameTimePerecent.HasValue)
					{
						ColumnDict["SlowFrameTimePercent"].Value = captureFpsChartData.SlowFrameTimePerecent.Value;
					}

					// Update the CSV stat values
					int columnIndex = csvStatColumnStartIndex;
					foreach (string statName in stats)
					{
						string StatToCheck = statName.Split('(')[0];
						if (!csvStats.Stats.ContainsKey(StatToCheck.ToLower()))
						{
							continue;
						}

						string[] StatTokens = statName.Split('(');

						float value = 0;
						if (StatTokens.Length > 1 && StatTokens[1].ToLower().Contains("min"))
						{
							value = csvStats.Stats[StatTokens[0].ToLower()].ComputeMinValue(CaptureFrameTimes.startIndex, CaptureFrameTimes.endIndex);
						}
						else if (StatTokens.Length > 1 && StatTokens[1].ToLower().Contains("max"))
						{
							value = csvStats.Stats[StatTokens[0].ToLower()].ComputeMaxValue(CaptureFrameTimes.startIndex, CaptureFrameTimes.endIndex);
						}
						else
						{
							value = csvStats.Stats[StatTokens[0].ToLower()].ComputeAverage(CaptureFrameTimes.startIndex, CaptureFrameTimes.endIndex);
						}
						Columns[columnIndex].Value = value;
						columnIndex++;
					}

					// Recompute colors
					foreach (ColumnInfo column in Columns)
					{
						column.UpdateColor();
					}

					// Write out data per capture range to summary table row data
					if (rowData != null)
                    {
						if (!bIgnoreHitchTimePercent)
						{
							rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_HitchTimePercent", captureFpsChartData.HitchTimePercent, GetStatColourThresholdList("HitchTimePercent"));
						}

						if (!bIgnoreMVP)
                        {
							rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_MVP", captureFpsChartData.MVP, GetStatColourThresholdList(MvpStatName));
						}

						if (!bIgnoreSlowFrameTimePercent && captureFpsChartData.SlowFrameTimePerecent.HasValue)
						{
							rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_SlowFrameTimePercent", captureFpsChartData.SlowFrameTimePerecent.Value, GetStatColourThresholdList("SlowFrameTimePerecent"));
						}

						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_HPM", captureFpsChartData.HitchesPerMinute, GetStatColourThresholdList("Hitches/Min"));
						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_HitchCount", captureFpsChartData.HitchCount);
						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, CapRange.name + "_TotalTimeSeconds", captureFpsChartData.TotalTimeSeconds);
					}

					// Output HTML
					if (htmlSection != null)
					{
						string ValueRow = "";
						ValueRow += "<td>" + CapRange.name + "</td>";
						foreach (ColumnInfo column in Columns)
						{
							ValueRow += "<td bgcolor=" + column.Color + ">" + column.Value.ToString("0.00") + "</td>";
						}
						htmlSection.WriteLine("  <tr>" + ValueRow + "</tr>");
					}

					// Output CSV
					if (statsCsvFile != null)
					{
						List<double> ColumnValues = new List<double>();
						List<string> ColumnColors = new List<string>();
						foreach (ColumnInfo column in Columns)
						{
							ColumnValues.Add(column.Value);
							ColumnColors.Add(column.Color);
						}

						statsCsvFile.Write(CapRange.name + ",");
						statsCsvFile.WriteLine(string.Join(",", ColumnValues));

						// Pass through color data as part of database-friendly stuff.
						statsCsvFile.Write(CapRange.name + " colors,");
						statsCsvFile.WriteLine(string.Join(",", ColumnColors));
					}
				}
			}

			if (htmlSection != null)
			{
				htmlSection.WriteLine("</table>");
				htmlSection.WriteLine("<p style='font-size:8'>Engine hitch metric: " + (bUseEngineHitchMetric ? "enabled" : "disabled") + "</p>");
			}

			if (statsCsvFile != null)
			{
				statsCsvFile.Close();
			}
			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}

		int fps;
		int recommendedFps;
		float slowFrameTimeBuffer;
		float hitchThreshold;
		bool bUseEngineHitchMetric;
		bool bIgnoreHitchTimePercent;
		bool bIgnoreSlowFrameTimePercent;
		bool bIgnoreMVP;
		float engineHitchToNonHitchRatio;
		float engineMinTimeBetweenHitchesMs;
	};

}