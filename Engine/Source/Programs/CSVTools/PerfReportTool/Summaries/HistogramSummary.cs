// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;

namespace PerfSummaries
{
	class HistogramSummary : Summary
	{
		public HistogramSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);

			ColourThresholds = ReadColourThresholdsXML(element.Element("colourThresholds"), vars);

			string[] histogramStrings = element.Element("histogramThresholds").GetValue(vars).Split(',');
			HistogramThresholds = new double[histogramStrings.Length];
			for (int i = 0; i < histogramStrings.Length; i++)
			{
				if (histogramStrings[i].Length > 0) // Fail gracefully if a variable fails to resolve
				{
					HistogramThresholds[i] = Convert.ToDouble(histogramStrings[i], System.Globalization.CultureInfo.InvariantCulture);
				}
			}

			foreach (XElement child in element.Elements())
			{
				if (child.Name == "budgetOverride")
				{
					BudgetOverrideStatName = child.GetRequiredAttribute<string>(vars, "stat");
					BudgetOverrideStatBudget = child.GetRequiredAttribute<double>(vars, "budget");
				}
			}

			bSuppressAveragesTable = element.GetSafeAttribute<bool>(vars, "suppressAveragesTable", false);
		}

		public HistogramSummary() { }

		public override string GetName() { return "histogram"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			// Only HTML reporting is supported (does not output summary table row data)
			if (!bWriteHtml)
			{
				return null;
			}

			HtmlSection htmlSection = new HtmlSection("Histogram", bStartCollapsed, MakeSectionId("Histogram"));

			// Histogram
			double[] thresholds = HistogramThresholds;
			htmlSection.WriteLine("  <table border='0' style='width:800'>");

			htmlSection.WriteLine("  <tr><td></td>");

			// Display the override stat budget first
			bool HasBudgetOverrideStat = false;
			if (BudgetOverrideStatName != null)
			{
				htmlSection.WriteLine("  <td><b><=" + BudgetOverrideStatBudget.ToString() + "ms</b></td>");
				HasBudgetOverrideStat = true;
			}

			foreach (float thresh in thresholds)
			{
				htmlSection.WriteLine("  <td><b>>" + thresh.ToString("0.0") + "ms</b></td>");
			}
			htmlSection.WriteLine("  </tr>");

			foreach (string unitStat in stats)
			{
				string StatToCheck = unitStat.Split('(')[0];
				if (!csvStats.Stats.ContainsKey(StatToCheck.ToLower()))
				{
					continue;
				}

				htmlSection.WriteLine("  <tr><td><b>" + StatToCheck + "</b></td>");
				int thresholdIndex = 0;

				// Display the render thread budget column (don't display the other stats)
				if (HasBudgetOverrideStat)
				{
					if (StatToCheck.ToLower() == BudgetOverrideStatName)
					{
						float RatioOfFramesOverBudget = 1.0f - csvStats.GetStat(StatToCheck.ToLower()).GetRatioOfFramesInBudget((float)BudgetOverrideStatBudget);
						float pc = RatioOfFramesOverBudget * 100.0f;
						string colour = ColourThresholdList.GetThresholdColour(pc, 50.0f, 35.0f, 20.0f, 0.0f);
						htmlSection.WriteLine("  <td bgcolor=" + colour + ">" + pc.ToString("0.00") + "%</td>");
					}
					else
					{
						htmlSection.WriteLine("  <td></td>");
					}
				}

				foreach (float thresh in thresholds)
				{
					float threshold = (float)thresholds[thresholdIndex];
					float RatioOfFramesOverBudget = 1.0f - csvStats.GetStat(StatToCheck.ToLower()).GetRatioOfFramesInBudget((float)threshold);
					float pc = RatioOfFramesOverBudget * 100.0f;
					string colour = ColourThresholdList.GetThresholdColour(pc, 50.0f, 35.0f, 20.0f, 0.0f);
					htmlSection.WriteLine("  <td bgcolor=" + colour + ">" + pc.ToString("0.00") + "%</td>");
					thresholdIndex++;
				}
				htmlSection.WriteLine("  </tr>");
			}
			htmlSection.WriteLine("  </table>");

			if (!bSuppressAveragesTable)
			{
				// Write the averages
				htmlSection.WriteLine("  <h3>Stat unit averages</h3>");
				htmlSection.WriteLine("  <table border='0' style='width:400'>");
				htmlSection.WriteLine("  <tr><td></td><th>ms</b></th>");
				foreach (string stat in stats)
				{
					string StatToCheck = stat.Split('(')[0];
					if (!csvStats.Stats.ContainsKey(StatToCheck.ToLower()))
					{
						continue;
					}
					float val = csvStats.Stats[StatToCheck.ToLower()].average;
					string colour = ColourThresholdList.GetThresholdColour(val, ColourThresholds[0], ColourThresholds[1], ColourThresholds[2], ColourThresholds[3]);
					htmlSection.WriteLine("  <tr><td><b>" + StatToCheck + "</b></td><td bgcolor=" + colour + ">" + val.ToString("0.00") + "</td></tr>");
				}
				htmlSection.WriteLine("  </table>");
			}

			return htmlSection;
		}

		public double[] ColourThresholds;
		public double[] HistogramThresholds;
		public string BudgetOverrideStatName;
		public double BudgetOverrideStatBudget;
		bool bSuppressAveragesTable;
	};

}