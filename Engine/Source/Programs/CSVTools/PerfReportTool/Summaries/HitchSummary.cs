// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;

namespace PerfSummaries
{
	class HitchSummary : Summary
	{
		public HitchSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);

			string[] hitchThresholds = element.Element("hitchThresholds").GetValue(vars).Split(',');
			HitchThresholds = new double[hitchThresholds.Length];
			for (int i = 0; i < hitchThresholds.Length; i++)
			{
				if (hitchThresholds[i].Length > 0) // Fail gracefully if a variable fails to resolve
				{
					HitchThresholds[i] = Convert.ToDouble(hitchThresholds[i], System.Globalization.CultureInfo.InvariantCulture);
				}
			}
		}
		public HitchSummary() { }

		public override string GetName() { return "hitches"; }

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;

			// if we have 20k frames in a typical flythrough then 20 frames would be red
			float redThresholdFor50ms = (float)csvStats.SampleCount / 500.0f;

			if (bWriteHtml)
			{
				htmlSection = new HtmlSection("Hitches", bStartCollapsed, MakeSectionId("Hitches"));
				htmlSection.WriteLine("  <table border='0' style='width:800'>");
				htmlSection.WriteLine("  <tr><td></td>");

				StreamWriter statsCsvFile = null;
				if (bWriteSummaryCsv)
				{
					string csvPath = Path.Combine(Path.GetDirectoryName(htmlFileName), "HitchStats.csv");
					statsCsvFile = new System.IO.StreamWriter(csvPath, false);
				}

				List<string> Thresholds = new List<string>();
				List<string> Hitches = new List<string>();
				Thresholds.Add("Hitch Size");
				foreach (float thresh in HitchThresholds)
				{
					htmlSection.WriteLine("  <th> >" + thresh.ToString("0") + "ms</b></td>");
					Thresholds.Add(thresh.ToString("0"));
				}
				if (statsCsvFile != null)
				{
					statsCsvFile.WriteLine(string.Join(",", Thresholds));
				}
				htmlSection.WriteLine("  </tr>");

				foreach (string unitStat in stats)
				{
					string StatToCheck = unitStat.Split('(')[0];
					StatSamples statSample = csvStats.GetStat(StatToCheck.ToLower());
					if (statSample == null)
					{
						continue;
					}

					Hitches.Clear();
					htmlSection.WriteLine("  <tr><td><b>" + StatToCheck + "</b></td>");
					Hitches.Add(StatToCheck);
					int thresholdIndex = 0;

					foreach (float threshold in HitchThresholds)
					{
						float count = (float)statSample.GetCountOfFramesOverBudget(threshold);
						float redThreshold = (redThresholdFor50ms * 50.0f) / threshold; // Adjust the colour threshold based on the current threshold
						string colour = ColourThresholdList.GetThresholdColour(count, redThreshold, redThreshold * 0.66, redThreshold * 0.33, 0.0f);
						htmlSection.WriteLine("  <td bgcolor=" + colour + ">" + count.ToString("0") + "</td>");
						Hitches.Add(count.ToString("0"));
						thresholdIndex++;
					}
					if (statsCsvFile != null)
					{
						statsCsvFile.WriteLine(string.Join(",", Hitches));
					}

					htmlSection.WriteLine("  </tr>");
				}
				if (statsCsvFile != null)
				{
					statsCsvFile.Close();
				}
				htmlSection.WriteLine("  </table>");
				htmlSection.WriteLine("<p style='font-size:8'>Note: Simplified hitch metric. All frames over threshold are counted" + "</p>");
			}

			// Write out hitches to the summary for 100ms intervals up to 1000
			if (rowData != null)
			{
				StatSamples frametimeStat = csvStats.GetStat("frametime");
				if (frametimeStat != null)
				{
					for (int i=1; i<=10;i++)
					{
						int threshold = 100 * i;
						int count = frametimeStat.GetCountOfFramesOverBudget((float)threshold);

						float redThreshold = (redThresholdFor50ms * 50.0f) / (float)threshold; // Adjust the colour threshold based on the current threshold
						ColourThresholdList colorThresholdList=new ColourThresholdList(redThreshold,redThreshold*0.66,redThreshold*0.33,0.0);

						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, "Hitches>" + threshold + "ms", (double)count, colorThresholdList);
					}
				}
			}
			return htmlSection;
		}
		public double[] HitchThresholds;
	};

}