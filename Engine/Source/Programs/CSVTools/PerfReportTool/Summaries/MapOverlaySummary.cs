// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Drawing;
using System.Drawing.Imaging;
using System.IO;
using System.Linq;
using System.Numerics;
using System.Runtime.Versioning;
using System.Xml.Linq;

namespace PerfSummaries
{
	class MapOverlaySummary : Summary
	{
		class MapOverlayEvent
		{
			public MapOverlayEvent(string inName)
			{
				name = inName;
			}
			public MapOverlayEvent(XElement element)
			{
			}
			public string name;
			public string summaryStatName;
			public string shortName;
			public string lineColor;
		};

		struct SVGPosition
		{
			public int X;
			public int Y;
		};

		class MapOverlay
		{
			public MapOverlay(XElement element, XmlVariableMappings vars)
			{
				positionStatNames[0] = element.GetSafeAttribute<string>(vars, "xStat");
				positionStatNames[1] = element.GetSafeAttribute<string>(vars, "yStat");
				positionStatNames[2] = element.GetSafeAttribute<string>(vars, "zStat");
				summaryStatNamePrefix = element.GetSafeAttribute<string>(vars, "summaryStatNamePrefix"); // unused!
				lineColor = element.GetSafeAttribute<string>(vars, "lineColor", "#ffffff");
				foreach (XElement eventEl in element.Elements("event"))
				{
					MapOverlayEvent ev = new MapOverlayEvent(eventEl.GetRequiredAttribute<string>(vars, "name"));
					ev.shortName = eventEl.GetSafeAttribute<string>(vars, "shortName");
					ev.summaryStatName = eventEl.GetSafeAttribute<string>(vars, "summaryStatName"); // unused!
					ev.lineColor = eventEl.GetSafeAttribute<string>(vars, "lineColor");
					if (eventEl.GetSafeAttribute<bool>(vars, "isStartEvent", false))
					{
						startEvents.Add(ev);
					}
					events.Add(ev);
				}
			}
			public string[] positionStatNames = new string[3];
			public string summaryStatNamePrefix;
			public List<MapOverlayEvent> startEvents = new List<MapOverlayEvent>();
			public string lineColor;
			public List<MapOverlayEvent> events = new List<MapOverlayEvent>();
		}

		public MapOverlaySummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			if (stats.Count != 0)
			{
				throw new Exception("<stats> element is not supported");
			}

			sourceImagePath = element.GetSafeAttribute<string>(vars, "sourceImage");
			if (baseXmlDirectory == null)
			{
				throw new Exception("BaseXmlDirectory not specified");
			}
			if (!System.IO.Path.IsPathRooted(sourceImagePath))
			{
				sourceImagePath = System.IO.Path.GetFullPath(System.IO.Path.Combine(baseXmlDirectory, sourceImagePath));
			}

			string transformtext = element.GetSafeAttribute(vars, "transform", "");
			if (transformtext.Length == 0)
			{
				// support for previous metadata
				float offsetX = element.GetSafeAttribute<float>(vars, "offsetX", 0.0f);
				float offsetY = element.GetSafeAttribute<float>(vars, "offsetY", 0.0f);
				float scale = element.GetSafeAttribute<float>(vars, "scale", 1.0f);

				transform[0] = scale * 0.5f;
				transform[1] = 0.0f;
				transform[2] = 0.0f;
				transform[3] = -scale * 0.5f;
				transform[4] = offsetX * 0.5f + 0.5f;
				transform[5] = -offsetY * 0.5f + 0.5f;
			}
			else
			{
				transform = Array.ConvertAll(transformtext.Split(' ', StringSplitOptions.RemoveEmptyEntries), float.Parse);
			}

			string fulltransformtext = element.GetSafeAttribute(vars, "transformFull", "");
			if (fulltransformtext.Length != 0)
			{
				fullTransform = Array.ConvertAll(fulltransformtext.Split(' ', StringSplitOptions.RemoveEmptyEntries), float.Parse);
				if (fullTransform.Length != 16)
				{
					throw new Exception("Malformed transform detected");
				}

				transformMatrix = new Matrix4x4(fullTransform[0], fullTransform[1], fullTransform[2], fullTransform[3],
												fullTransform[4], fullTransform[5], fullTransform[6], fullTransform[7],
												fullTransform[8], fullTransform[9], fullTransform[10], fullTransform[11],
												fullTransform[12], fullTransform[13], fullTransform[14], fullTransform[15]);
			}

			title = element.GetSafeAttribute(vars, "title", "Events");
			destImageFilename = element.GetRequiredAttribute<string>(vars, "destImage");
			imageWidth = element.GetSafeAttribute<float>(vars, "width", 250.0f);
			imageHeight = element.GetSafeAttribute<float>(vars, "height", 250.0f);
			framesPerLineSegment = element.GetSafeAttribute<int>(vars, "framesPerLineSegment", 5);
			lineSplitDistanceThreshold = element.GetSafeAttribute<float>(vars, "lineSplitDistanceThreshold", float.MaxValue);
			createCroppedImage = element.GetSafeAttribute<bool>(vars, "createCroppedImage", false);

			foreach (XElement overlayEl in element.Elements("overlay"))
			{
				MapOverlay overlay = new MapOverlay(overlayEl, vars);
				overlays.Add(overlay);
				stats.Add(overlay.positionStatNames[0]);
				stats.Add(overlay.positionStatNames[1]);
				stats.Add(overlay.positionStatNames[2]);
			}
		}
		public MapOverlaySummary() { }

		public override string GetName() { return "mapoverlay"; }

		int toSvgX(float worldX, float worldY)
		{
			float svgX = worldX * transform[0] + worldY * transform[1] + transform[4];
			svgX *= imageWidth;
			return (int)(svgX + 0.5f);
		}

		int toSvgY(float worldX, float worldY)
		{
			float svgY = worldX * transform[2] + worldY * transform[3] + transform[5];
			svgY *= imageHeight;
			return (int)(svgY + 0.5f);
		}

		SVGPosition toSVGLocation(float worldX, float worldY, Rectangle imageRegion)
		{
			Vector3 svgPos;

			// TODO: Remove when we fully support full transforms
			if (fullTransform != null)
			{
				// These seem backwards?
				Vector3 worldPos3D = new Vector3(worldY, worldX, 0.0f);
				svgPos = Vector3.Transform(worldPos3D, transformMatrix);

				svgPos.Y = -svgPos.Y;
				// From -1<>1 to 0<>1
				svgPos.X += 1.0f;
				svgPos.Y += 1.0f;
				svgPos.X *= 0.5f;
				svgPos.Y *= 0.5f;

				// To Image Size
				svgPos.X *= imageWidth;
				svgPos.Y *= imageHeight;
			}
			else
			{
				svgPos.X = toSvgX(worldX, worldY);
				svgPos.Y = toSvgY(worldX, worldY);
			}

			// Scale to SubImage
			if (imageRegion.Width != imageWidth || imageRegion.Height != imageHeight)
			{
				svgPos.X = (svgPos.X - imageRegion.X) / imageRegion.Width * imageWidth;
				svgPos.Y = (svgPos.Y - imageRegion.Y) / imageRegion.Height * imageHeight;
			}

			return new SVGPosition()
			{
				X = (int)(svgPos.X + 0.5f),
				Y =	(int)(svgPos.Y + 0.5f),
			};
		}

		SVGPosition toSVGLocationClamped(float worldX, float worldY, Rectangle imageRegion, int imagePadding)
		{
			SVGPosition svgPos = toSVGLocation(worldX, worldY, imageRegion);
			svgPos.X += imagePadding;
			svgPos.Y += imagePadding;
			svgPos.X = Math.Min(Math.Max(0, svgPos.X), (int)imageWidth);
			svgPos.Y = Math.Min(Math.Max(0, svgPos.Y), (int)imageHeight);
			return svgPos;
		}

		private void CopyAndResizeImage(string sourceImagePath, string destImagePath, int destWidth, int destHeight)
		{
			if (!OperatingSystem.IsWindowsVersionAtLeast(6, 1))
			{
				Console.WriteLine("CopyAndResizeImage is not supported on this platform!");
				return;
			}
			Console.WriteLine("Downsampling map image.\n  Source: " + sourceImagePath + "\n  Dest  : " + destImagePath);
			using (FileStream fileStream = new FileStream(sourceImagePath, FileMode.Open, FileAccess.Read))
			{
				Console.WriteLine("Reading source image");
				using (var image = System.Drawing.Image.FromStream(fileStream))
				{
					Console.WriteLine("Generating downsampled image");
					var thumbnail = image.GetThumbnailImage(destWidth, destHeight, null, IntPtr.Zero);
					using (var destImageStream = new FileStream(destImagePath, FileMode.OpenOrCreate, FileAccess.Write))
					{
						Console.WriteLine("Saving downsampled map image: " + destImagePath);
						thumbnail.Save(destImageStream, ImageFormat.Jpeg);
					}
				}
			}
		}

		private void CopyAndTrimImage(string sourceImagePath, string destImagePath, int destWidth, int destHeight, Rectangle subRegion)
		{
			if (!OperatingSystem.IsWindowsVersionAtLeast(6, 1))
			{
				Console.WriteLine("CopyAndTrimImage is not supported on this platform!");
				return;
			}

			Console.WriteLine("Generating sub map image.\n  Source: " + sourceImagePath + "\n  Dest  : " + destImagePath);

			using (FileStream fileStream = new FileStream(sourceImagePath, FileMode.Open, FileAccess.Read))
			{
				Console.WriteLine("Reading source image");
				using (var image = Image.FromStream(fileStream))
				{
					Rectangle scaledSubRegion = subRegion;
					float subRegionScaleW = image.Width / imageWidth;
					float subRegionScaleH = image.Height / imageHeight;
					scaledSubRegion.X = (int)((float)scaledSubRegion.X * subRegionScaleW);
					scaledSubRegion.Width = (int)((float)scaledSubRegion.Width * subRegionScaleW);
					scaledSubRegion.Y = (int)((float)scaledSubRegion.Y * subRegionScaleH);
					scaledSubRegion.Height = (int)((float)scaledSubRegion.Height * subRegionScaleH);

					if (scaledSubRegion.Width > 0 && scaledSubRegion.Height > 0)
					{
						// Trimming
						Console.WriteLine("Generating Cropped Image");
						using (Bitmap croppedBitmap = new Bitmap(scaledSubRegion.Width, scaledSubRegion.Height))
						{
							using (Graphics graphics = Graphics.FromImage(croppedBitmap))
							{
								graphics.DrawImage(image, new Rectangle(0, 0, scaledSubRegion.Width, scaledSubRegion.Height),
												   scaledSubRegion, GraphicsUnit.Pixel);

								using (var thumbnail = croppedBitmap.GetThumbnailImage(destWidth, destHeight, null, IntPtr.Zero))
								{
									using (var destImageStream = new FileStream(destImagePath, FileMode.OpenOrCreate, FileAccess.Write))
									{
										Console.WriteLine("Saving trimmed map image: " + destImagePath);
										thumbnail.Save(destImageStream, ImageFormat.Jpeg);
									}
								}
							}
						}
					}
					else 
					{
						Console.WriteLine("Failed to scale image correctly!");
					}
				}
			}
		}

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;

			// Output HTML
			if (bWriteHtml)
			{
				htmlSection = new HtmlSection(title, bStartCollapsed, MakeSectionId(title));
				string outputDirectory = System.IO.Path.GetDirectoryName(System.IO.Path.GetFullPath(htmlFileName));
				string outputMapFilename = System.IO.Path.Combine(outputDirectory, destImageFilename);

				// Skip the copy if the output file already exists
				if (!File.Exists(outputMapFilename))
				{
					if (File.Exists(sourceImagePath))
					{
						// Copy the file to the reports folder and reset attributes to ensure it's not readonly if the source is
						//					File.Copy(sourceImagePath, outputMapFilename);
						//					File.SetAttributes(outputMapFilename, FileAttributes.Normal);
						CopyAndResizeImage(sourceImagePath, outputMapFilename, (int)imageWidth, (int)imageHeight);
					}
					else
					{
						Console.WriteLine("[Warning] Can't find source map image: " + sourceImagePath);
					}
				}

				Rectangle imageRegion = new Rectangle(0, 0, (int)imageWidth, (int)imageHeight);

				GenerateImageSection(csvStats, htmlSection, imageRegion, destImageFilename);

				if (createCroppedImage)
				{
					// Generate Sub Image Location, Padded to give a small border.
					int extraImagePadding = 3;
					SVGPosition minPos = toSVGLocationClamped(eventDimensions.minX, eventDimensions.minY, imageRegion, -extraImagePadding);
					SVGPosition maxPos = toSVGLocationClamped(eventDimensions.maxX, eventDimensions.maxY, imageRegion, extraImagePadding);

					if (minPos.X != maxPos.X && minPos.Y != maxPos.Y)
					{
						// Center the map.
						int SubRegionWidth = (int)(maxPos.X - minPos.X);
						int SubRegionHeight = (int)(maxPos.Y - minPos.Y);
						int CenterX = minPos.X + (SubRegionWidth / 2);
						int CenterY = minPos.Y + (SubRegionHeight / 2);
						int UniformDim = Math.Max(SubRegionWidth, SubRegionHeight);
						Rectangle subRegion = new Rectangle(CenterX - (UniformDim / 2), CenterY - (UniformDim / 2), UniformDim, UniformDim);
						// Clamps us to the original image bounds in case we overflow it.
						subRegion = Rectangle.Intersect(subRegion, imageRegion);

						string croppedImageName = "crop_" + destImageFilename;
						string outputTrimMapFilename = System.IO.Path.Combine(outputDirectory, croppedImageName);

						// Skip the copy if the output file already exists
						if (!File.Exists(outputTrimMapFilename))
						{
							if (File.Exists(sourceImagePath))
							{
								CopyAndTrimImage(sourceImagePath, outputTrimMapFilename, (int)imageWidth, (int)imageHeight, subRegion);
							}
							else
							{
								Console.WriteLine("[Warning] Can't find source map image: " + sourceImagePath);
							}
						}

						GenerateImageSection(csvStats, htmlSection, subRegion, croppedImageName);
					}
				}
			}

			// Output row data
			if (rowData != null)
			{
			}
			return htmlSection;
		}

		private void GenerateImageSection(CsvStats csvStats, HtmlSection htmlSection, Rectangle imageRegion, string imageName)
		{
			// Check if the file exists in the output directory
			htmlSection.WriteLine("<svg version='1.1' xmlns='http://www.w3.org/2000/svg' xmlns:xlink='http://www.w3.org/1999/xlink' width='" + imageWidth + "' height='" + imageHeight + "'>");
			htmlSection.WriteLine("<image href='" + imageName + "' width='" + imageWidth + "' height='" + imageHeight + "' />");

			// Draw the overlays
			foreach (MapOverlay overlay in overlays)
			{
				StatSamples xStat = csvStats.GetStat(overlay.positionStatNames[0]);
				StatSamples yStat = csvStats.GetStat(overlay.positionStatNames[1]);

				if (xStat == null || yStat == null)
				{
					continue;
				}

				// If a startevent is specified, update the start frame
				int startFrame = 0;
				if (overlay.startEvents.Count != 0)
				{
					foreach (CsvEvent ev in csvStats.Events)
					{
						foreach (MapOverlayEvent startEv in overlay.startEvents)
						{
							if (CsvStats.DoesSearchStringMatch(ev.Name, startEv.name))
							{
								startFrame = ev.Frame;
								break;
							}
						}

						if (startFrame != 0)
						{
							break;
						}
					}
				}

				// Make a mapping from frame to map indices
				List<KeyValuePair<int, MapOverlayEvent>> frameEvents = new List<KeyValuePair<int, MapOverlayEvent>>();
				foreach (MapOverlayEvent mapEvent in overlay.events)
				{
					foreach (CsvEvent ev in csvStats.Events)
					{
						if (ev.Frame >= startFrame && CsvStats.DoesSearchStringMatch(ev.Name, mapEvent.name))
						{
							frameEvents.Add(new KeyValuePair<int, MapOverlayEvent>(ev.Frame, mapEvent));
						}
					}
				}
				frameEvents.Sort((pair0, pair1) => pair0.Key.CompareTo(pair1.Key));
				int eventIndex = 0;

				// Draw the lines
				string currentLineColor = overlay.lineColor;
				string lineStartTemplate = "<polyline style='fill:none;stroke-width:1.3;stroke:{LINECOLOUR}' points='";
				htmlSection.Write(lineStartTemplate.Replace("{LINECOLOUR}", currentLineColor));
				float adjustedLineSplitDistanceThreshold = lineSplitDistanceThreshold * framesPerLineSegment;
				float oldx = 0;
				float oldy = 0;
				int lastFrameIndex = -1;
				for (int i = startFrame; i < xStat.samples.Count; i += framesPerLineSegment)
				{
					float x = xStat.samples[i];
					float y = yStat.samples[i];
					SVGPosition svgPos = toSVGLocation(x, y, imageRegion);

					string lineCoordsStr = svgPos.X + "," + svgPos.Y + " ";

					eventDimensions.minX = Math.Min(eventDimensions.minX, x);
					eventDimensions.maxX = Math.Max(eventDimensions.maxX, x);
					eventDimensions.minY = Math.Min(eventDimensions.minY, y);
					eventDimensions.maxY = Math.Max(eventDimensions.maxY, y);

					// Figure out which event we're up to so we can do color changes
					bool restartLineStrip = false;
					while (eventIndex < frameEvents.Count && lastFrameIndex < frameEvents[eventIndex].Key && i >= frameEvents[eventIndex].Key)
					{
						MapOverlayEvent mapEvent = frameEvents[eventIndex].Value;
						string newLineColor = mapEvent.lineColor != null ? mapEvent.lineColor : overlay.lineColor;
						// If we changed color, restart the line strip
						if (newLineColor != currentLineColor)
						{
							currentLineColor = newLineColor;
							restartLineStrip = true;
						}
						eventIndex++;
					}

					// If the distance between this point and the last is over the threshold, restart the line strip
					float maxManhattanDist = Math.Max(Math.Abs(x - oldx), Math.Abs(y - oldy));
					if (maxManhattanDist > adjustedLineSplitDistanceThreshold)
					{
						restartLineStrip = true;
					}
					else
					{
						htmlSection.Write(lineCoordsStr);
					}

					if (restartLineStrip)
					{
						htmlSection.WriteLine("'/>");
						htmlSection.Write(lineStartTemplate.Replace("{LINECOLOUR}", currentLineColor));
						htmlSection.Write(lineCoordsStr);
					}
					oldx = x;
					oldy = y;
					lastFrameIndex = i;
				}
				htmlSection.WriteLine("'/>");

				// Plot the events 
				float circleRadius = 3;
				string eventColourString = "#ffffff";
				foreach (MapOverlayEvent mapEvent in overlay.events)
				{
					foreach (CsvEvent ev in csvStats.Events)
					{
						if (ev.Frame >= startFrame && CsvStats.DoesSearchStringMatch(ev.Name, mapEvent.name))
						{
							string eventText = mapEvent.shortName != null ? mapEvent.shortName : ev.Name;
							float x = xStat.samples[ev.Frame];
							float y = yStat.samples[ev.Frame];
							SVGPosition svgPos = toSVGLocation(x, y, imageRegion);
							htmlSection.Write("<circle cx='" + svgPos.X + "' cy='" + svgPos.Y + "' r='" + circleRadius + "' fill='" + eventColourString + "' fill-opacity='1.0'/>");
							htmlSection.WriteLine("<text x='" + (svgPos.X + 5) + "' y='" + svgPos.Y + "' text-anchor='left' style='font-family: Verdana;fill: #ffffff; font-size: " + 9 + "px;'>" + eventText + "</text>");
						}
					}
				}
			}

			//htmlSection.WriteLine("<text x='50%' y='" + (imageHeight * 0.05) + "' text-anchor='middle' style='font-family: Verdana;fill: #FFFFFF; stroke: #C0C0C0;  font-size: " + 20 + "px;'>" + title + "</text>");
			htmlSection.WriteLine("</svg>");
		}

		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}

		class MapEventDimensions
		{
			public float minX = float.MaxValue;
			public float maxX = float.MinValue;
			public float minY = float.MaxValue;
			public float maxY = float.MinValue;
		}

		string title;
		string sourceImagePath;
		float[] transform = new float[6];
		float[] fullTransform = null; // Default to null while we transition
		Matrix4x4 transformMatrix = Matrix4x4.Identity;
		string destImageFilename;
		float imageWidth;
		float imageHeight;
		float lineSplitDistanceThreshold;
		int framesPerLineSegment;
		bool createCroppedImage = false;

		List<MapOverlay> overlays = new List<MapOverlay>();
		MapEventDimensions eventDimensions = new MapEventDimensions();
	};

}