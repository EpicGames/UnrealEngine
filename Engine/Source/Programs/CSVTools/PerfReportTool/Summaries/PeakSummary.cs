// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using CSVStats;

namespace PerfSummaries
{
	class PeakSummary : Summary
	{
		/*
		  A peak summary displays a list of stats by their peak values. 
		  The stat list comes from the graphs, where inSummary='1' specifies that a graph's stats should be
		  included. Specifying a budget for the graph also assigns budgets to its stats. 
		  A list of summarySection elements can be specified. This groups stats into separate sections, displayed
		  at the top of the report. 
		  Note: a stat will only appear in in one section. If there are multiple compatble sections, it will appear 
		  in the first.

			<summarySection title="Audio">
				<statFilter>LLM/Audio/*</statFilter>
			</summarySection>
		*/
		class PeakSummarySection
		{
			public PeakSummarySection(string inTitle, string inStatFilterStr)
			{
				title = inTitle;
				statNamesFilter = inStatFilterStr.Split(',');
			}
			public PeakSummarySection(XElement element, XmlVariableMappings vars)
			{
				XElement statFilterElement = element.Element("statFilter");
				title = element.GetRequiredAttribute<string>(vars, "title");

				statNamesFilter = statFilterElement.Value.Split(',');
			}
			public bool StatMatchesSection(CsvStats csvStats, string statName)
			{
				if (statNameFilterSet == null)
				{
					statNameFilterSet = csvStats.GetStatNamesMatchingStringList_Set(statNamesFilter);
				}
				return statNameFilterSet.Contains(statName);
			}
			public void AddStat(PeakStatInfo statInfo)
			{
				stats.Add(statInfo);
			}
			string[] statNamesFilter;
			HashSet<string> statNameFilterSet;

			public List<PeakStatInfo> stats = new List<PeakStatInfo>();
			public string title;
		};

		public PeakSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			//read the child elements (mostly for colourThresholds)
			ReadStatsFromXML(element, vars);
			hideStatPrefix = element.GetSafeAttribute<string>(vars, "hideStatPrefix", "").ToLower();

			foreach (XElement child in element.Elements())
			{
				if (child.Name == "summarySection")
				{
					peakSummarySections.Add(new PeakSummarySection(child, vars));
				}
			}

			// If we don't have any sections then add a default one
			if (peakSummarySections.Count == 0)
			{
				PeakSummarySection section = new PeakSummarySection("Peaks", "*");
				peakSummarySections.Add(section);
			}
		}

		public PeakSummary() { }

		public override string GetName() { return "peak"; }

		void WriteStatSection(HtmlSection htmlSection, CsvStats csvStats, PeakSummarySection section, StreamWriter LLMCsvData, SummaryTableRowData summaryTableRowData)
		{
			if (htmlSection != null)
			{
				// Here we are deciding which title we have and write it to the file.
				htmlSection.WriteLine("<h3>" + section.title + "</h3>");
				htmlSection.WriteLine("  <table border='0' style='width:400'>");

				//Hard-coded start of the table.
				htmlSection.WriteLine("    <tr><td style='width:200'></td><td style='width:75'><b>Average</b></td><td style='width:75'><b>Peak</b></td><td style='width:75'><b>Budget</b></td></tr>");
			}

			foreach (PeakStatInfo statInfo in section.stats)
			{
				// Do the calculations for the averages and peak, and then write it to the table along with the budget.
				string statName = statInfo.name;
				StatSamples csvStat = csvStats.Stats[statName.ToLower()];
				double peak = (double)csvStat.ComputeMaxValue();
				double average = (double)csvStat.average;

				string peakColour = "#ffffff";
				string averageColour = "#ffffff";
				string budgetString = "";
				ColourThresholdList colorThresholdList = new ColourThresholdList();

				if (statInfo.budget.isSet)
				{
					double budget = statInfo.budget.value;
					float redValue = (float)budget * 1.5f;
					float orangeValue = (float)budget * 1.25f;
					float yellowValue = (float)budget * 1.0f;
					float greenValue = (float)budget * 0.9f;

					colorThresholdList.Add(new ThresholdInfo(greenValue));
					colorThresholdList.Add(new ThresholdInfo(yellowValue));
					colorThresholdList.Add(new ThresholdInfo(orangeValue));
					colorThresholdList.Add(new ThresholdInfo(redValue));

					peakColour = colorThresholdList.GetColourForValue(peak);
					averageColour = colorThresholdList.GetColourForValue(average);
					budgetString = budget.ToString("0");
				}
				if (htmlSection != null)
				{
					htmlSection.WriteLine("    <tr><td>" + statInfo.shortName + "</td><td bgcolor=" + averageColour + ">" + average.ToString("0") + "</td><td bgcolor=" + peakColour + ">" + peak.ToString("0") + "</td><td>" + budgetString + "</td></tr>");
				}

				// Pass through color data as part of database-friendly stuff.
				if (LLMCsvData != null)
				{
					string csvStatName = statName.Replace('/', ' ').Replace("$32$", " ");
					LLMCsvData.WriteLine(string.Format("{0},{1},{2},{3}", csvStatName, average.ToString("0"), peak.ToString("0"), budgetString, averageColour, peakColour));
					LLMCsvData.WriteLine(string.Format("{0}_Colors,{1},{2},'#aaaaaa'", csvStatName, averageColour, peakColour));
				}

				if (summaryTableRowData != null)
				{
					SummaryTableElement smv;
					// Hide duplicate CsvStatAverage stats
					if (summaryTableRowData.dict.TryGetValue(statName.ToLower(), out smv))
					{
						if (smv.type == SummaryTableElement.Type.CsvStatAverage)
						{
							smv.SetFlag(SummaryTableElement.Flags.Hidden, true);
						}
					}
					summaryTableRowData.Add(SummaryTableElement.Type.SummaryTableMetric, statInfo.shortName + " Avg", average, colorThresholdList);
					summaryTableRowData.Add(SummaryTableElement.Type.SummaryTableMetric, statInfo.shortName + " Max", peak, colorThresholdList);
				}

			}
			if (htmlSection != null)
			{
				htmlSection.WriteLine("  </table>");
			}
		}

		PeakSummarySection FindStatSection(CsvStats csvStats, string statName)
		{
			for (int i = 0; i < peakSummarySections.Count; i++)
			{
				if (peakSummarySections[i].StatMatchesSection(csvStats, statName))
				{
					return peakSummarySections[i];
				}
			}
			return null;
		}

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;
			// Only HTML reporting is supported (does not output summary table row data)
			StreamWriter LLMCsvData = null;
			if (bWriteSummaryCsv)
			{
				// FIXME: This summary type is not specific to LLM. Pass filename in!
				string LLMCsvPath = Path.Combine(Path.GetDirectoryName(htmlFileName), "LLMStats_colored.csv");
				LLMCsvData = new StreamWriter(LLMCsvPath);
				LLMCsvData.WriteLine("Stat,Average,Peak,Budget");
			}

			// Add all stats to the appropriate sections
			foreach (string stat in stats)
			{
				PeakSummarySection section = FindStatSection(csvStats, stat);
				if (section != null)
				{
					PeakStatInfo statInfo = getOrAddStatInfo(stat);
					section.AddStat(statInfo);
				}
			}

			if (bWriteHtml)
			{
				htmlSection = new HtmlSection("Peaks Summary", bStartCollapsed, MakeSectionId("Peaks"));
			}

			foreach (PeakSummarySection section in peakSummarySections)
			{
				WriteStatSection(htmlSection, csvStats, section, LLMCsvData, rowData);
			}

			if (LLMCsvData != null)
			{
				LLMCsvData.Close();
			}

			return htmlSection;
		}



		void AddStat(string statName, Optional<double> budget)
		{
			stats.Add(statName);

			PeakStatInfo info = getOrAddStatInfo(statName);
			if (budget.isSet)
			{
				info.budget = budget;
			}
		}

		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
			// Find the stats by spinning through the graphs in this reporttype
			foreach (ReportGraph graph in reportTypeInfo.graphs)
			{
				if (graph.inSummary)
				{
					if (graph.settings.mainStat.isSet)
					{
						AddStat(graph.settings.mainStat.value, graph.budget);
					}
					if (graph.settings.statString.isSet)
					{
						string statString = graph.settings.statString.value;
						string[] statNames = statString.Split(',');
						statNames = csvStats.GetStatNamesMatchingStringList(statNames).ToArray();
						foreach (string stat in statNames)
						{
							AddStat(stat, graph.budget);
						}
					}
				}
			}

			base.PostInit(reportTypeInfo, csvStats);
		}

		List<PeakSummarySection> peakSummarySections = new List<PeakSummarySection>();

		Dictionary<string, PeakStatInfo> statInfoLookup = new Dictionary<string, PeakStatInfo>();
		class PeakStatInfo
		{
			public PeakStatInfo(string inName, string inShortName)
			{
				budget = new Optional<double>();
				name = inName;
				shortName = inShortName;
			}
			public string name;
			public string shortName;
			public Optional<double> budget;
		};

		PeakStatInfo getOrAddStatInfo(string statName)
		{
			if (statInfoLookup.ContainsKey(statName))
			{
				return statInfoLookup[statName];
			}
			// Find the best (longest) prefix which matches this stat, and strip it off
			string shortStatName = statName;
			if (hideStatPrefix.Length > 0 && statName.ToLower().StartsWith(hideStatPrefix))
			{
				shortStatName = statName.Substring(hideStatPrefix.Length);
			}

			PeakStatInfo statInfo = new PeakStatInfo(statName, shortStatName);
			statInfoLookup.Add(statName, statInfo);
			return statInfo;
		}

		string hideStatPrefix;
	};

}