// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Data.Common;
using System.IO;
using System.Linq;
using System.Xml.Linq;

namespace PerfSummaries
{
	class StandardSummary : Summary
	{
		/*
		  A standard summary which is present in all report types. Requires no XML definition. Executed after all other summaries
		*/
	
		public StandardSummary()
		{
		}

		public override string GetName() { return "standard"; }


		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;

			// Compute the total time if the RowData doesn't already exist
			if (rowData != null)
			{
				string TotalTimeColumnName = "Total Time (s)";
				if (!rowData.Contains(TotalTimeColumnName.ToLower()))
				{
					if (csvStats.Stats.TryGetValue("frametime", out StatSamples frameTimes))
					{
						double totalTimeMs = 0.0;
						foreach (float frameTime in frameTimes.samples)
						{
							totalTimeMs += (double)frameTime;
						}
						rowData.Add(SummaryTableElement.Type.SummaryTableMetric, TotalTimeColumnName, totalTimeMs / 1000.0);
					}
				}
			}
			return htmlSection;
		}

	};

}