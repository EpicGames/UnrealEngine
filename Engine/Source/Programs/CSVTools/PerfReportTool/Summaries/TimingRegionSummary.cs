// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Xml.Linq;
using System.Linq;

namespace PerfSummaries
{
	class TimingRegionSummary : Summary
	{

		class RegionFilterInfo
		{
			public RegionFilterInfo(XElement element, XmlVariableMappings vars, int indexIn)
			{
				pattern = element.GetRequiredAttribute<string>("pattern");
				busyTimeMetric = element.GetSafeAttribute<string>("busyTimeMetric");
				hidePrefix = element.GetSafeAttribute<string>("hidePrefix");
				trackDisplayThreshold = element.GetSafeAttribute<int>("trackDisplayThreshold", 1);
				index = indexIn;
			}
			public string pattern;
			public string busyTimeMetric = null;
			public string hidePrefix = null;
			public int index = 0;
			public int trackDisplayThreshold = 0;
		}
		public TimingRegionSummary(XElement element, XmlVariableMappings vars, string baseXmlDirectory)
		{
			ReadStatsFromXML(element, vars);
			if (stats.Count != 0)
			{
				throw new Exception("<stats> element is not supported");
			}
			title = element.GetRequiredAttribute<string>(vars, "title");
			beginEventName = element.GetSafeAttribute<string>(vars, "beginEvent");
			string beginRegionName = element.GetSafeAttribute<string>(vars, "beginRegion");
			if (beginRegionName != null)
			{
				if (beginEventName != null)
				{
					throw new Exception("Timing region summary has both beginEvent and beginRegion defined!");
				}
				beginEventName = $"Region: {beginRegionName} - Begin";
			}

			endEventName = element.GetSafeAttribute<string>(vars, "endEvent");
			string endRegionName = element.GetSafeAttribute<string>(vars, "endRegion");
			if (endRegionName != null)
			{
				if ( endEventName != null)
				{
					throw new Exception("Timing region summary has both endEvent and endRegion defined!");
				}
				endEventName = $"Region: {endRegionName} - End";
			}

			bShowTimingTable = element.GetSafeAttribute<bool>(vars, "showTimingTable", true);
			bFindLast = element.GetSafeAttribute<bool>(vars, "findLast", false);

			metricPrefix = element.GetSafeAttribute<string>(vars, "metricPrefix");
			foreach (XElement regionFilterXml in element.Elements("regionFilter"))
			{
				RegionFilterInfo filterInfo = new RegionFilterInfo(regionFilterXml, vars, regionFilters.Count);
				regionFilters.Add(filterInfo);
			}
		}
		public TimingRegionSummary() { }

		public override string GetName() { return "timingregion"; }

		enum ERangeType
		{
			Before,
			Inside,
			After
		}

		class RegionEvent
		{
			public static RegionEvent TryCreateFromCsvEvent(CsvEvent csvEvent) 
			{
				const string regionPrefix = "Region: ";
				if (!csvEvent.Name.StartsWith(regionPrefix))
				{
					return null;
				}
				const string beginSuffix = " - Begin";
				const string endSuffix = " - End";
				RegionEvent regionEvent = new RegionEvent();
				string name = csvEvent.Name.Substring(regionPrefix.Length);
				if (name.EndsWith(beginSuffix))
				{
					regionEvent.name = name.Substring(0, name.Length - beginSuffix.Length);
					regionEvent.bBegin = true;
				}
				else if (name.EndsWith(endSuffix))
				{
					regionEvent.name = name.Substring(0, name.Length - endSuffix.Length);
					regionEvent.bBegin = false;
				}
				else
				{
					// No begin or end
					return null;
				}

				// Read the category if there is one
				int slashIndex = regionEvent.name.IndexOf('/');
				if (slashIndex != -1)
				{
					regionEvent.category = regionEvent.name.Substring(0, slashIndex);
					regionEvent.name = regionEvent.name.Substring(slashIndex+1);
				}
				else
				{
					// Can't have null as the category since we use it as a dictionary key
					regionEvent.category = "";
				}
				regionEvent.timestamp = csvEvent.Timestamp;
				regionEvent.frameNumber = csvEvent.Frame;
				return regionEvent;
			}

			public double GetPairDuration()
			{ 
				if (otherEvent == null)
				{
					return 0.0;
				}
				return Math.Abs(timestamp - otherEvent.timestamp);
			}

			public string GetFullName()
			{
				if (category == "")
				{
					return name;
				}
				return category + "/" + name;
			}

			public RegionEvent CreateTruncatedPairEvent(double startTime, double endTime)
			{
				// If this is an end event, make sure it's the last one before any begin events
				RegionEvent newEvent = new RegionEvent();
				newEvent.bBegin = !bBegin;
				newEvent.name = name;
				newEvent.category = category;
				newEvent.frameNumber = -1;
				newEvent.timestamp = bBegin ? endTime : startTime;
				newEvent.otherEvent = this;
				newEvent.rangeType = bBegin ? ERangeType.After : ERangeType.Before;
				newEvent.filterInfo = filterInfo;
				otherEvent = newEvent;
				return newEvent;
			}

			public string category;
			public string name;
			public int frameNumber = -1;
			public bool bBegin;
			public double timestamp;
			public RegionEvent otherEvent;
			public RegionFilterInfo filterInfo;
			public ERangeType rangeType = ERangeType.Inside;
		}

		bool TryGetRegionFilterInfo(RegionEvent regionEvent, out RegionFilterInfo outRegionFilterInfo)
		{
			outRegionFilterInfo = null;
			foreach (RegionFilterInfo filterInfo in regionFilters)
			{
				if (CsvStats.DoesSearchStringMatch(regionEvent.GetFullName(), filterInfo.pattern))
				{
					outRegionFilterInfo = filterInfo;
					return true;
				}
			}
			return false;
		}



		List<RegionEvent> GetRegionEvents(CsvStats csvStats, double startTime, double endTime)
		{
			// Find the region events
			List<RegionEvent> allRegionEvents = new List<RegionEvent>();

			// Make sure the events are sorted by timestamp. Note: We need a stable sort here, so we use Linq Orderby
			List<CsvEvent> csvEvents = csvStats.Events.OrderBy(e => e.Timestamp).ToList();
			foreach (CsvEvent ev in csvEvents)
			{
				RegionEvent regionEvent = RegionEvent.TryCreateFromCsvEvent(ev);
				if (regionEvent != null && TryGetRegionFilterInfo(regionEvent, out regionEvent.filterInfo))
				{
					// Detect and clamp out of range events
					if (ev.Timestamp < startTime)
					{
						regionEvent.rangeType = ERangeType.Before;
					}
					else if (ev.Timestamp > endTime)
					{
						regionEvent.rangeType = ERangeType.After;
					}
					regionEvent.timestamp = Math.Clamp(ev.Timestamp, startTime, endTime);
					allRegionEvents.Add(regionEvent);
				}
			}

			// Split the events by name
			Dictionary<string, List<RegionEvent>> regionEventsByName = new Dictionary<string, List<RegionEvent>>();
			foreach (RegionEvent ev in allRegionEvents)
			{
				string key = ev.GetFullName();
				List<RegionEvent> regionEventList;
				if (!regionEventsByName.TryGetValue(key, out regionEventList))
				{
					regionEventList = new List<RegionEvent>();
					regionEventsByName[key] = regionEventList;
				}
				regionEventList.Add(ev);
			}

			List<RegionEvent> allFilteredRegionEvents = new List<RegionEvent>();
			// Spin through each region by name
			foreach (List<RegionEvent> regionList in regionEventsByName.Values)
			{
				// Compute the depth of every event, including orphan events which span the boundary
				int depth = 0;
				int minDepth = int.MaxValue;

				Dictionary<RegionEvent, int> regionEventDepth = new Dictionary<RegionEvent, int>();
				foreach (RegionEvent ev in regionList) 
				{
					if (ev.bBegin)
					{
						regionEventDepth[ev] = depth;
						minDepth = Math.Min(minDepth, depth);
						depth++;
					}
					else
					{
						depth--;
						minDepth = Math.Min(minDepth, depth);
						regionEventDepth[ev] = depth;
					}
				}

				// Cull everything not at min depth and pair the adjacent begin/end nodes
				List<RegionEvent> filteredEvents = new List<RegionEvent>();
				RegionEvent beginEvent = null;
				foreach (RegionEvent ev in regionList)
				{
					if (regionEventDepth[ev] == minDepth)
					{
						if (ev.bBegin)
						{
							beginEvent = ev;
						}
						else if (beginEvent != null)
						{
							ev.otherEvent = beginEvent;
							beginEvent.otherEvent = ev;
							beginEvent = null;
						}
						filteredEvents.Add(ev);
					}
				}

				// Detect orphan events at the start and end and add dummy events connecting to them
				if (filteredEvents.Count > 0)
				{
					if (filteredEvents.First().otherEvent == null)
					{
						filteredEvents.Insert(0, filteredEvents.First().CreateTruncatedPairEvent(startTime, endTime));
					}
					if (filteredEvents.Last().otherEvent == null)
					{
						filteredEvents.Add(filteredEvents.Last().CreateTruncatedPairEvent(startTime, endTime));
					}
				}
				allFilteredRegionEvents.AddRange(filteredEvents);
			}

			// Cull region events which are entirely outside the range on one side or the other or the duration is very close to zero
			List<RegionEvent> newRegionEvents = new List<RegionEvent>();
			foreach (RegionEvent ev in allFilteredRegionEvents)
			{
				if (ev.rangeType == ERangeType.Inside || ev.otherEvent.rangeType == ERangeType.Inside)
				{
					if ( ev.GetPairDuration() > 0.0001 )
					{
						newRegionEvents.Add(ev);
					}
				}
			}

			newRegionEvents = newRegionEvents.OrderBy(e => e.timestamp).ToList();
			return newRegionEvents;
		}

		class RegionEventTrack : IComparable<RegionEventTrack>
		{
			public RegionEventTrack(string inCategory, double inStartTime, double inEndTime, RegionFilterInfo inFilterInfo, double inTrackWidth, double inTrackHeight)
			{
				category = inCategory;
				startTime = inStartTime;
				endTime = inEndTime;
				filterInfo = inFilterInfo;
				filterIndex = filterInfo.index;
				trackWidth = inTrackWidth;
				trackHeight = inTrackHeight;
				innerTrackWidth = trackWidth - 25;
				double trackDuration = (endTime - startTime);
				regionDurationThreshold = (inFilterInfo.trackDisplayThreshold / innerTrackWidth ) * trackDuration;
			}
			public bool TryAddEvent(RegionEvent beginEvent)
			{
				if (beginEvent.category != category || beginEvent.filterInfo != filterInfo || beginEvent.timestamp < lastEventTime)
				{
					return false;
				}
				if (beginEvent.GetPairDuration() < regionDurationThreshold)
				{
					return false;
				}
				if (firstEventTime == double.MaxValue)
				{
					firstEventTime = beginEvent.timestamp;
				}
				lastEventTime = (beginEvent.otherEvent != null) ? beginEvent.otherEvent.timestamp : endTime;
				events.Add(beginEvent);
				return true;
			}

			public void WriteToHtml(HtmlSection htmlSection, double xOffset, double yOffset, string backgroundColor="#ffffff")
			{
				// Strip the hidePrefix if necessary
				string backgroundStyle = "fill:" + backgroundColor;
				string textStyle = "font-family:verdana;font-size:12px";
				string arrowTextStyle = "font-family:verdana;font-size:18px";
				double trackDuration = endTime - startTime;
				double lastEventX=xOffset;
				RegionEvent lastEvent = null;
				foreach (RegionEvent beginEvent in events)
				{
					RegionEvent endEvent = beginEvent.otherEvent;
					double duration = endEvent.timestamp - beginEvent.timestamp;
					double t = beginEvent.timestamp - startTime;
					double x = Math.Round(xOffset + innerTrackWidth * t / trackDuration,0);
					double width = Math.Round(innerTrackWidth * duration / trackDuration,0);
					if (width < 3)
					{
						continue;
					}
					// Truncate the display name if required
					string displayName = beginEvent.name;
					if (filterInfo.hidePrefix != null && beginEvent.name.StartsWith(filterInfo.hidePrefix))
					{
						displayName = beginEvent.name.Substring(filterInfo.hidePrefix.Length);
					}

					string regionText = displayName + " - " + String.Format("{0:0.00}", duration) + "s";
					string style = "fill:"+CsvStats.GetColorFromString(displayName, 80, 220)+";stroke-width:1;stroke:black";
					string titleStr = "<title>" + regionText + "</title>";

					double overlap = 0;
					if ( Math.Abs(lastEventX - x) < 0.001 )
					{
						if ( lastEventX > 0.0)
						{
							overlap = 1; // Overlap consecutive blocks to avoid double border lines
						}
					}
					else
					{
						// Draw a white block before this event to cover any overflow text
						htmlSection.WriteLine($"<rect x='{lastEventX}' y='{yOffset}' width='{x - lastEventX}' height='{trackHeight}' style='{backgroundStyle}'/>");
					}
					htmlSection.WriteLine($"<rect x='{(x - overlap)}' y='{yOffset}' width='{width + overlap}' height='{trackHeight}' style='{style}'>");
					htmlSection.WriteLine(titleStr+"</rect>");
					if (width > 20)
					{
						htmlSection.WriteLine($"<text x='{x + 3}' y='{yOffset + 15}' style='{textStyle}' pointer-events='none'>{titleStr + regionText}</text>");
					}
					lastEventX = x + width;
					lastEvent = endEvent;
				}
				// Redraw the background on the RHS to deal with any overflow text
				htmlSection.WriteLine($"<rect x='{(lastEventX+1)}' y='{yOffset}' width='{(innerTrackWidth-(lastEventX-xOffset)-1)}' height='{trackHeight}' style='{backgroundStyle}'/>");
				htmlSection.WriteLine($"<rect x='{xOffset + innerTrackWidth}' y='{yOffset}' width='{(trackWidth - innerTrackWidth)}' height='{trackHeight}' fill='#ffffff'/>");

				// Draw left and right arrows to indicate regions which exceed the track
				if (lastEvent != null && lastEvent.rangeType == ERangeType.After)
				{
					htmlSection.WriteLine($"<text x='{xOffset + innerTrackWidth + 5}' y='{yOffset + 15}' style='{arrowTextStyle}' pointer-events='none'>&#8680;</text>");
				}
				RegionEvent firstEvent = events.Count == 0 ? null : events[0];
				if (firstEvent != null && firstEvent.rangeType == ERangeType.Before && firstEvent.timestamp > 0)
				{
					htmlSection.WriteLine($"<text x='{xOffset - 20}' y='{yOffset + 15}' style='{arrowTextStyle}' pointer-events='none'>&#8678;</text>");
				}


			}

			public int CompareTo(RegionEventTrack other)
			{
				int filterIndexCompare = filterIndex.CompareTo(other.filterIndex);
				if (filterIndexCompare != 0)
				{
					return filterIndexCompare;
				}
				return firstEventTime.CompareTo(other.firstEventTime);
			}

			public string category;
			public int filterIndex;
			public double lastEventTime;
			public double firstEventTime = double.MaxValue;
			public List<RegionEvent> events = new List<RegionEvent>();
			public double startTime;
			public double endTime;
			public double trackWidth;
			public double innerTrackWidth;
			public double trackHeight;
			public double regionDurationThreshold;
			public RegionFilterInfo filterInfo;
		}

		public override HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			HtmlSection htmlSection = null;

			StatSamples frameTimeStat = csvStatsUnstripped.GetStat("frametime");
			if (frameTimeStat == null)
			{
				throw new Exception("TimingRegion summary requires a frametime stat");
			}

			Tuple<CsvEvent, CsvEvent> eventPair = csvStatsUnstripped.FindEventPair(beginEventName, endEventName, bFindLast);

			CsvEvent beginEvent = eventPair.Item1;
			double startTime = (beginEvent != null) ? beginEvent.Timestamp : 0.0;

			CsvEvent endEvent = eventPair.Item2;
			double endTime = (endEvent != null) ? endEvent.Timestamp : frameTimeStat.total / 1000.0;

			double totalDuration = endTime - startTime;

			List<RegionEvent> regionEvents = GetRegionEvents(csvStatsUnstripped, startTime, endTime);
			if ( regionEvents.Count == 0 )
			{
				// No relevant events found
				return null;
			}

			// Output HTML
			string prevCategory = null;
			if (bWriteHtml)
			{
				htmlSection = new HtmlSection(title, bStartCollapsed, MakeSectionId(title));

				double imageWidth = 1425;
				double headerWidth = 150;
				double trackWidth = imageWidth - headerWidth;

				// Assign events to tracks
				List<RegionEventTrack> tracks = new List<RegionEventTrack>();
				foreach (RegionEvent ev in regionEvents)
				{
					if (ev.bBegin && ev.otherEvent != null)
					{
						// Add the event to an available track
						bool bFoundTrack = false;
						foreach (RegionEventTrack track in tracks)
						{
							if (track.TryAddEvent(ev))
							{
								bFoundTrack = true;
								break;
							}
						}
						if (!bFoundTrack)
						{
							RegionEventTrack newTrack = new RegionEventTrack(ev.category, startTime, endTime, ev.filterInfo, trackWidth, 20);
							if (newTrack.TryAddEvent(ev))
							{
								tracks.Add(newTrack);
							}
						}
					}
				}

				// Sort tracks
				tracks.Sort();

				double imageHeight = tracks.Count * 22;
				string lineStyle = "stroke:#808080;stroke-width:2";
				string categoryTextStyle = "font-family:verdana;font-size:12px";

				List<double> trackWidths = new List<double>();

				// Check if the file exists in the output directory
				htmlSection.WriteLine($"<svg version='1.1' xmlns='http://www.w3.org/2000/svg' xmlns:xlink='http://www.w3.org/1999/xlink' width='{imageWidth}' height='{imageHeight}'>");

				double yOffset = 0;
				int trackIndex = 0;
				double innerTrackWidth = imageWidth - 25;
				foreach (RegionEventTrack track in tracks)
				{
					if (prevCategory != track.category)
					{
						if (prevCategory != null)
						{
							double lineY = yOffset + 1;
							htmlSection.WriteLine($"<line x1='{0}' y1='{lineY}' x2='{innerTrackWidth}' y2='{lineY}' style='{lineStyle}' stroke-dasharray='5,5'/>");
							yOffset += 2;
						}
						prevCategory = track.category;
						htmlSection.WriteLine($"<text x='{0}' y='{yOffset+14}' style='{categoryTextStyle}'>{track.category}</text>");
					}
					track.WriteToHtml(htmlSection, headerWidth, yOffset, trackIndex%2==0 ? "#e0e0e0" : "#f0f0f0");
					yOffset += 20;
					trackIndex++;
				}
				// Draw vertical dashed lines showing the edges of the tracks
				htmlSection.WriteLine($"<line x1='{headerWidth-1}' y1='{0}' x2='{headerWidth-1}' y2='{imageHeight}' style='{lineStyle}' stroke-dasharray='5,5'/>");
				htmlSection.WriteLine($"<line x1='{headerWidth+innerTrackWidth-headerWidth}' y1='{0}' x2='{headerWidth+innerTrackWidth-headerWidth}' y2='{imageHeight}' style='{lineStyle}' stroke-dasharray='5,5'/>");

				htmlSection.WriteLine("</svg>");
			}

			// Output row data showing the total time for each region
			Dictionary<string, double> regionTotalTimes = new Dictionary<string, double>();
			foreach (RegionEvent ev in regionEvents)
			{
				if ( ev.bBegin )
				{
					if (!regionTotalTimes.ContainsKey(ev.GetFullName()))
					{
						regionTotalTimes[ev.GetFullName()] = 0;
					}
					regionTotalTimes[ev.GetFullName()] += ev.otherEvent.timestamp - ev.timestamp;
				}
			}

			if (metricPrefix != null && rowData != null)
			{
				foreach (KeyValuePair<string, double> pair in regionTotalTimes)
				{
					string name = metricPrefix + "/" + pair.Key;
					rowData.Add(new SummaryTableElement(SummaryTableElement.Type.SummaryTableMetric, name, pair.Value, null, ""));
				}
			}
			if (htmlSection != null && bShowTimingTable)
			{
				htmlSection.WriteLine("<h3 class='collapsibleHeading collapsed'>Region durations</h3>");
				htmlSection.WriteLine("<div class='collapsibleSection collapsed'>");
				htmlSection.WriteLine("<div class='collapsibleSectionInner'>");
				htmlSection.WriteLine("<table border='0' style='width:1000'>");
				htmlSection.WriteLine("<tr><th>Region</th><th>Time (s)</th></tr>");
				List<KeyValuePair<string, double>> regionTimingsSorted = new List<KeyValuePair<string, double>>(regionTotalTimes);
				regionTimingsSorted.Sort((a, b) => a.Key.CompareTo(b.Key));

				foreach (KeyValuePair<string, double> pair in regionTimingsSorted)
				{
					if (pair.Value>=0.01)
					{
						htmlSection.WriteLine($"<tr><td>{pair.Key}</td><td>{String.Format("{0:0.00}", pair.Value)}</td></tr>");
					}
				}
				htmlSection.WriteLine("</table>");
				htmlSection.WriteLine("</div>");
				htmlSection.WriteLine("</div>");
			}
			return htmlSection;
		}
		public override void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
		{
		}

		string title;
		string beginEventName;
		string endEventName;
		string metricPrefix;
		bool bShowTimingTable;
		bool bFindLast;
		List<RegionFilterInfo> regionFilters = new List<RegionFilterInfo>();
	};

}