// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Drawing;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.RegularExpressions;
using System.Xml.Linq;

namespace PerfSummaries
{
	class SummaryFactory
	{
		public static void Init()
		{
			summaryNameLookup = new Dictionary<string, Type>();
			Assembly assembly = typeof(SummaryFactory).GetTypeInfo().Assembly;
			foreach (Type type in assembly.GetTypes())
			{
				if (type.IsSubclassOf(typeof(Summary)))
				{
					Summary instance = (Summary)Activator.CreateInstance(type);
					summaryNameLookup.Add(instance.GetName(), type);
				}
			}
		}

		public static Summary Create(string summaryTypeName, XElement summaryXmlElement, XmlVariableMappings vars, string baseXmlDirectory)
		{
			if ( summaryNameLookup.TryGetValue(summaryTypeName, out System.Type summaryType ) )
			{
				object[] constructArgs = new object[3] { summaryXmlElement, vars, baseXmlDirectory };
				return (Summary)Activator.CreateInstance(summaryType, constructArgs);
			}
			throw new Exception("Summary type " + summaryTypeName + " not found!");
		}

		static Dictionary<string, System.Type> summaryNameLookup = null;
	};

	abstract class Summary
	{

		public class CaptureRange
        {
            public string name;
            public string startEvent;
            public string endEvent;
            public bool includeFirstFrame;
            public bool includeLastFrame;
            public CaptureRange(string inName, string start, string end)
            {
                name = inName;
                startEvent = start;
                endEvent = end;
                includeFirstFrame = false;
                includeLastFrame = false;
            }
        }
        public class CaptureData
        {
            public int startIndex;
            public int endIndex;
            public List<float> Frames;
            public CaptureData(int start, int end, List<float> inFrames)
            {
                startIndex = start;
                endIndex = end;
                Frames = inFrames;
            }
        }

        public Summary()
        {
            stats = new List<string>();
            captures = new List<CaptureRange>();
            StatThresholds = new Dictionary<string, ColourThresholdList>();
		}
		public virtual HtmlSection WriteSummaryData(bool bWriteHtml, CsvStats csvStats, CsvStats csvStatsUnstripped, bool bWriteSummaryCsv, SummaryTableRowData rowData, string htmlFileName)
		{
			return null;
		}

		public virtual void PostInit(ReportTypeInfo reportTypeInfo, CsvStats csvStats)
        {
			// Resolve wildcards and remove duplicates
			stats = csvStats.GetStatNamesMatchingStringList(stats.ToArray());
		}

		public abstract string GetName();

        public void ReadStatsFromXML(XElement element, XmlVariableMappings vars)
        {
			if (element == null) 
			{
				return;
			}
			bHideInDetailedReport = element.GetSafeAttribute<bool>(vars, "hideInDetailedReport", false);
			useUnstrippedCsvStats = element.GetSafeAttribute<bool>(vars, "useUnstrippedCsvStats", false);
			bStartCollapsed = element.GetSafeAttribute<bool>(vars, "collapsed", false);
			XElement statsElement = element.Element("stats");
			if (statsElement != null)
			{
				stats = statsElement.GetValue(vars).Split(',').ToList();
			}
            foreach (XElement child in element.Elements())
            {
                if (child.Name == "capture")
                {
                    string captureName = child.GetRequiredAttribute<string>(vars, "name");
                    string captureStart = child.GetRequiredAttribute<string>(vars, "startEvent");
                    string captureEnd = child.GetRequiredAttribute<string>(vars, "endEvent");
					bool incFirstFrame = child.GetSafeAttribute<bool>(vars, "includeFirstFrame", true);
					bool incLastFrame = child.GetSafeAttribute<bool>(vars, "includeLastFrame", true);
                    CaptureRange newRange = new CaptureRange(captureName, captureStart, captureEnd);
                    newRange.includeFirstFrame = incFirstFrame;
                    newRange.includeLastFrame = incLastFrame;
                    captures.Add(newRange);
                }
                else if (child.Name == "colourThresholds")
                {
                    if (child.Attribute("stat") == null)
                    {
                        continue;
                    }
                    string statName = child.GetRequiredAttribute<string>(vars, "stat");
					string hitchThresholdsStr = child.GetValue(vars);
					if (hitchThresholdsStr == "")
					{
						continue;
					}
                    string[] hitchThresholdsStrList = hitchThresholdsStr.Split(',');
					ColourThresholdList HitchThresholds = new ColourThresholdList();
					for (int i = 0; i < hitchThresholdsStrList.Length; i++)
                    {
						string hitchThresholdStr = hitchThresholdsStrList[i];
						string hitchThresholdNumStr = hitchThresholdStr;
						Colour thresholdColour = null;

						int openBracketIndex = hitchThresholdStr.IndexOf('(');
						if (openBracketIndex != -1 )
						{
							hitchThresholdNumStr = hitchThresholdStr.Substring(0, openBracketIndex);
							int closeBracketIndex = hitchThresholdStr.IndexOf(')');
							if (closeBracketIndex > openBracketIndex) 
							{
								string colourString = hitchThresholdStr.Substring(openBracketIndex+1, closeBracketIndex - openBracketIndex-1);
								thresholdColour = new Colour(colourString);
							}
						}
						double thresholdValue = Convert.ToDouble(hitchThresholdNumStr, System.Globalization.CultureInfo.InvariantCulture);

						HitchThresholds.Add(new ThresholdInfo(thresholdValue, thresholdColour));
                    }
                    if (HitchThresholds.Count == 4)
                    {
                        StatThresholds.Add(statName, HitchThresholds);
                    }
                }
            }
        }
        public CaptureData GetFramesForCapture(CaptureRange inCapture, List<float> FrameTimes, List<CsvEvent> EventsCaptured)
        {
            List<float> ReturnFrames = new List<float>();
            int startFrame = -1;
            int endFrame = FrameTimes.Count;
            for (int i = 0; i < EventsCaptured.Count; i++)
            {
                if (startFrame < 0 && EventsCaptured[i].Name.ToLower().Contains(inCapture.startEvent.ToLower()))
                {
                    startFrame = EventsCaptured[i].Frame;
                    if (!inCapture.includeFirstFrame)
                    {
                        startFrame++;
                    }
                }
                else if (endFrame >= FrameTimes.Count && EventsCaptured[i].Name.ToLower().Contains(inCapture.endEvent.ToLower()))
                {
                    endFrame = EventsCaptured[i].Frame;
                    if (!inCapture.includeLastFrame)
                    {
                        endFrame--;
                    }
                }
            }
            if (startFrame == -1 || endFrame == FrameTimes.Count || endFrame < startFrame)
            {
                return null;
            }
            ReturnFrames = FrameTimes.GetRange(startFrame, (endFrame - startFrame));
            CaptureData CaptureToUse = new CaptureData(startFrame, endFrame, ReturnFrames);
            return CaptureToUse;
        }

        public string[] GetUniqueStatNames()
        {
            HashSet<string> uniqueStats = new HashSet<string>();
            foreach (string stat in stats)
            {
                if (!uniqueStats.Contains(stat))
                {
                    uniqueStats.Add(stat);
                }
            }
            return uniqueStats.ToArray();
        }

		protected ColourThresholdList ReadColourThresholdListXML(XElement colourThresholdEl, XmlVariableMappings vars)
		{
			return ColourThresholdList.ReadColourThresholdListXML(colourThresholdEl, vars);
		}

		protected double [] ReadColourThresholdsXML(XElement colourThresholdEl, XmlVariableMappings vars)
		{
			return ColourThresholdList.ReadColourThresholdsXML(colourThresholdEl, vars);
		}

        public string GetStatThresholdColour(string StatToUse, double value)
        {
			ColourThresholdList Thresholds = GetStatColourThresholdList(StatToUse);
			if (Thresholds != null)
            {
				return Thresholds.GetColourForValue(value);
            }
			return "'#ffffff'";
        }

		public ColourThresholdList GetStatColourThresholdList(string StatToUse)
		{
			if (StatThresholds.ContainsKey(StatToUse))
			{
				return StatThresholds[StatToUse];
			}
			return null;
		}

		public void SetReportSummaryIndex(int index)
		{
			reportSummaryIndex = index;
		}

		protected string MakeSectionId(string title)
		{
			return Regex.Replace(title, @"[^a-zA-Z0-9_-]", "") + "_" + reportSummaryIndex.ToString();
		}


		public List<CaptureRange> captures;
        public List<string> stats;
        public Dictionary<string, ColourThresholdList> StatThresholds;
		public bool useUnstrippedCsvStats;
		public bool bStartCollapsed;
		public bool bHideInDetailedReport = false;
		public int reportSummaryIndex = -1;
	};



	class HtmlSection
	{
		public HtmlSection(string titleIn, bool bStartCollapsedIn, string elementIdIn = null, int headingLevelIn=2)
		{
			title = titleIn;
			bStartCollapsed = bStartCollapsedIn;
			elementId = elementIdIn;
			headingLevel = headingLevelIn;
			headingType = "h" + headingLevel.ToString();
		}

		public void WriteLine(string text) 
		{
			FlushPendingLine();
			lines.Add(text);
		}
		public void Write(string text)
		{
			pendingLine.Append(text);
		}

		public void WriteToFile( System.IO.StreamWriter htmlFile )
		{
			FlushPendingLine();
			BeginHtmlSection(htmlFile);
			foreach (string line in lines)
			{
				htmlFile.WriteLine(line);
			}
			EndHtmlSection(htmlFile);
		}

		private void FlushPendingLine()
		{
			if (pendingLine.Length > 0)
			{
				lines.Add(pendingLine.ToString());
				pendingLine.Clear();
			}
		}

		protected void BeginHtmlSection(System.IO.StreamWriter htmlFile)
		{
			if (htmlFile != null)
			{
				string headingClass = "collapsibleHeading";
				string divClass = "collapsibleSection";

				if (!bStartCollapsed)
				{
					headingClass += " expanded";
					divClass += " expanded";
				}

				string extraHeadingAttributes = "";
				if (elementId != null)
				{
					htmlFile.WriteLine("<a name='" + elementId + "'></a>");
					extraHeadingAttributes = "id='" + elementId + "'";

					extraHeadingAttributes += "style='display: inline-block;vertical-align: middle'";
					htmlFile.WriteLine("<br>");
					htmlFile.WriteLine("<a href='#" + elementId + "' class='sectionAnchorLink sectionAnchorText-" + headingType+"'>#</a>");

				}

				htmlFile.WriteLine("<"+ headingType + " class='" + headingClass + "' "+ extraHeadingAttributes + ">" + title + "</"+ headingType + ">");

				htmlFile.WriteLine("<div class='" + divClass + "'>");

				string extraSectionAttributes = "";
				// If this is a main section then add padding
				if (headingLevel <= 2)
				{
					extraSectionAttributes += "style='margin-left:12px'";
				}
				htmlFile.WriteLine("<div class='collapsibleSectionInner' "+ extraSectionAttributes + ">");
			}
		}

		protected void EndHtmlSection(System.IO.StreamWriter htmlFile)
		{
			if (htmlFile != null)
			{
				htmlFile.WriteLine("<br>");
				htmlFile.WriteLine("</div>");
				htmlFile.WriteLine("</div>");
			}
		}

		string title;
		StringBuilder pendingLine = new StringBuilder();
		List<string> lines = new List<string>();
		bool bStartCollapsed;
		int headingLevel;
		string elementId;
		string headingType;
	};






}
