// Copyright Epic Games, Inc. All Rights Reserved.

using CSVStats;
using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Data.Common;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Xml.Linq;

namespace PerfSummaries
{
	class StatDisplayInfo
	{
		public StatDisplayInfo(string inDisplayName, string inToolTip = null)
		{
			displayName = inDisplayName;
			toolTip = inToolTip;
		}
		public string displayName = null;
		public string toolTip = null;
	};

	class TableUtil
	{
		public static string FormatStatName(string inStatName)
		{
			// We add a zero width space so stats are split over multiple lines
			return inStatName.Replace("/", "/<wbr>");
		}

		public static string SanitizeHtmlString(string str)
		{
			return str.Replace("<", "&lt;").Replace(">", "&gt;");
		}

		public static string SafeTruncateHtmlTableValue(string inValue, int maxLength)
		{
			if (inValue.StartsWith("<a") && inValue.EndsWith("</a>"))
			{
				// Links require special handling. Only truncate what's inside
				int openAnchorEndIndex = inValue.IndexOf(">");
				int closeAnchorStartIndex = inValue.IndexOf("</a>");
				if (openAnchorEndIndex > 2 && closeAnchorStartIndex > openAnchorEndIndex)
				{
					string anchor = inValue.Substring(0, openAnchorEndIndex + 1);
					string text = inValue.Substring(openAnchorEndIndex + 1, closeAnchorStartIndex - (openAnchorEndIndex + 1));
					if (text.Length > maxLength)
					{
						text = SanitizeHtmlString(text.Substring(0, maxLength)) + "...";
					}
					return anchor + text + "</a>";
				}
			}
			else if (inValue.StartsWith("{LinkTemplate:"))
			{
				// Don't truncate link templates.
				return inValue;
			}
			return SanitizeHtmlString(inValue.Substring(0, maxLength)) + "...";
		}
	}

	// Allows overriding style aspects of the summary table HTML.
	// New overrides should be filled in here with valid defaults.
	class SummaryTableStyle
	{
		// Max width of the column header.
		// Used when the table is not transposed.
		public int maxColumnHeaderWidth = 220;
		public int minColumnHeaderWidth = 75;
	}

	class SummarySectionBoundaryInfo
	{
		public SummarySectionBoundaryInfo(string inStatName, string inStartToken, string inEndToken, int inLevel, bool inInCollatedTable, bool inInFullTable)
		{
			statName = inStatName;
			startToken = inStartToken;
			endToken = inEndToken;
			level = inLevel;
			inCollatedTable = inInCollatedTable;
			inFullTable = inInFullTable;
		}
		public string statName;
		public string startToken;
		public string endToken;
		public int level;
		public bool inCollatedTable;
		public bool inFullTable;
	};

	class SummaryTableInfo
	{
		public SummaryTableInfo(XElement tableElement, Dictionary<string,List<string>> substitutionsDict, string[] appendList, string[] rowSortAppendList, XmlVariableMappings variableMappings )
		{
			string rowSortStr = tableElement.GetSafeAttribute<string>(variableMappings, "rowSort");
			if (rowSortStr != null)
			{
				rowSortList.AddRange(rowSortStr.Split(',').Select(s => s.Trim()));
				ApplySubstitutionsToList(rowSortList, substitutionsDict);
			}
			if (rowSortAppendList != null)
			{
				rowSortList.AddRange(rowSortAppendList);
			}

			weightByColumn = tableElement.GetSafeAttribute<string>(variableMappings, "weightByColumn");
			if (weightByColumn != null)
			{
				weightByColumn = weightByColumn.ToLower();
			}

			XElement filterEl = tableElement.Element("filter");
			if (filterEl != null)
			{
				columnFilterList.AddRange(filterEl.GetValue(variableMappings).Split(',').Select(s => s.Trim()));
				ApplySubstitutionsToList(columnFilterList, substitutionsDict);
			}

			if (appendList != null)
			{
				columnFilterList.AddRange(appendList);
			}

			bReverseSortRows = tableElement.GetSafeAttribute<bool>(variableMappings, "reverseSortRows", false);
			bScrollableFormatting = tableElement.GetSafeAttribute<bool>(variableMappings, "scrollableFormatting", false);

			tableColorizeMode = tableElement.GetSafeAttribute<TableColorizeMode>(variableMappings, "colorizeMode", tableColorizeMode);
			dateCollationVisibility = tableElement.GetSafeAttribute<DateCollationVisibility>(variableMappings, "collatedDateVisibility", dateCollationVisibility);
			stringCollationVisibility = tableElement.GetSafeAttribute<StringCollationVisibility>(variableMappings, "collatedStringVisibility", stringCollationVisibility);
			columnSortMode = tableElement.GetSafeAttribute<TableColumnSortMode>(variableMappings, "columnSortMode", columnSortMode);

			collateAverageMethod = tableElement.GetSafeAttribute<CollateAverageMethod>(variableMappings, "collateAverageMethod", collateAverageMethod);

			// Table style overrides
			summaryTableStyle.maxColumnHeaderWidth = tableElement.GetSafeAttribute<int>("maxColumnHeaderWidth", summaryTableStyle.maxColumnHeaderWidth);
			summaryTableStyle.minColumnHeaderWidth = tableElement.GetSafeAttribute<int>("minColumnHeaderWidth", summaryTableStyle.minColumnHeaderWidth);
			
			statThreshold = tableElement.GetSafeAttribute<float>(variableMappings, "statThreshold", 0.0f);
			hideStatPrefix = tableElement.GetSafeAttribute<string>(variableMappings, "hideStatPrefix");

			foreach (XElement sectionBoundaryEl in tableElement.Elements("sectionBoundary"))
			{
				if (sectionBoundaryEl != null)
				{
					string statName = ApplySubstitution(sectionBoundaryEl.GetSafeAttribute<string>(variableMappings, "statName"), substitutionsDict);

					SummarySectionBoundaryInfo sectionBoundary = new SummarySectionBoundaryInfo(
						statName,
						sectionBoundaryEl.GetSafeAttribute<string>(variableMappings, "startToken"),
						sectionBoundaryEl.GetSafeAttribute<string>(variableMappings, "endToken"),
						sectionBoundaryEl.GetSafeAttribute<int>(variableMappings, "level", 0),
						sectionBoundaryEl.GetSafeAttribute<bool>(variableMappings, "inCollatedTable", true),
						sectionBoundaryEl.GetSafeAttribute<bool>(variableMappings, "inFullTable", true)
						);
					sectionBoundaries.Add(sectionBoundary);
				}
			}
		}

		private void ApplySubstitutionsToList(List<string> list, Dictionary<string, List<string>> substitutionsDict)
		{
			if (substitutionsDict == null)
			{
				return;
			}
			for (int i = 0; i < list.Count; i++)
			{
				if (substitutionsDict.TryGetValue(list[i].ToLowerInvariant(), out List<string> replaceList))
				{
					list.RemoveAt(i);
					list.InsertRange(i, replaceList);
					i += replaceList.Count - 1;
				}
			}
		}

		private string ApplySubstitution(string str, Dictionary<string, List<string>> substitutionsDict)
		{
			if (substitutionsDict == null)
			{
				return str;
			}
			if (substitutionsDict.TryGetValue(str.ToLowerInvariant(), out List<string> replaceList))
			{
				if (replaceList.Count>1 || replaceList.Count == 0)
				{
					// This method doesn't support one-to-many remapping, so ignore
					return str;
				}
				return replaceList[0];
			}
			return str;
		}


		public SummaryTableInfo(string filterListStr, string rowSortStr)
		{
			columnFilterList.AddRange(filterListStr.Split(',').Select(s => s.Trim()));
			rowSortList.AddRange(rowSortStr.Split(',').Select(s => s.Trim()));
		}

		public SummaryTableInfo()
		{
		}

		public List<string> rowSortList = new List<string>();
		public List<string> columnFilterList = new List<string>();
		public List<SummarySectionBoundaryInfo> sectionBoundaries = new List<SummarySectionBoundaryInfo>();
		public bool bReverseSortRows;
		public bool bScrollableFormatting;
		public TableColorizeMode tableColorizeMode = TableColorizeMode.Budget;
		public DateCollationVisibility dateCollationVisibility = DateCollationVisibility.Newest;
		public StringCollationVisibility stringCollationVisibility = StringCollationVisibility.Auto;
		public CollateAverageMethod collateAverageMethod = CollateAverageMethod.Mean;
		public TableColumnSortMode columnSortMode = TableColumnSortMode.Default;
		public SummaryTableStyle summaryTableStyle = new SummaryTableStyle();

		public float statThreshold;
		public string hideStatPrefix = null;
		public string weightByColumn = null;
	}


	class SummaryTableColumnFormatInfoCollection
	{
		public SummaryTableColumnFormatInfoCollection(XElement element)
		{
			foreach (XElement child in element.Elements("columnInfo"))
			{
				columnFormatInfoList.Add(new SummaryTableColumnFormatInfo(child));
			}
		}

		public SummaryTableColumnFormatInfo GetFormatInfo(string columnName)
		{
			string lowerColumnName = columnName.ToLower();
			if (lowerColumnName.StartsWith("avg ") || lowerColumnName.StartsWith("min ") || lowerColumnName.StartsWith("max "))
			{
				lowerColumnName = lowerColumnName.Substring(4);
			}
			foreach (SummaryTableColumnFormatInfo columnInfo in columnFormatInfoList)
			{
				int wildcardIndex = columnInfo.name.IndexOf('*');
				if (wildcardIndex == -1)
				{
					if (columnInfo.name == lowerColumnName)
					{
						return columnInfo;
					}
				}
				else
				{
					string prefix = columnInfo.name.Substring(0, wildcardIndex);
					if (lowerColumnName.StartsWith(prefix))
					{
						return columnInfo;
					}
				}
			}
			return defaultColumnInfo;
		}

		public static SummaryTableColumnFormatInfo DefaultColumnInfo
		{
			get { return defaultColumnInfo; }
		}

		List<SummaryTableColumnFormatInfo> columnFormatInfoList = new List<SummaryTableColumnFormatInfo>();
		static SummaryTableColumnFormatInfo defaultColumnInfo = new SummaryTableColumnFormatInfo();
	};

	enum TableColorizeMode
	{
		Off,		// No coloring.
		Budget,     // Colorize based on defined budgets. No coloring for stats without defined budgets.
		Auto,		// Auto calculate based on other values in the column.
	};

	enum TableColumnSortMode
	{
		Default,            // Default sort (uses specified filter order, alphabetical wildcard resolution)
		WildcardSortByAvg,  // When resolving wildcards, sort results by column average value
		WildcardSortByMax,  // When resolving wildcards, sort results by column max value
		SortByAvg,          // Sort all columns by average value (ignores specified filter order)
		SortByMax,          // Sort all columns by max value (ignores specified filter order)
	};

	enum AutoColorizeMode
	{
		Off,
		HighIsBad,
		LowIsBad,
	};

	enum ColumnAggregateType
	{
		None,
		Avg,
		Min,
		Max
	};

	enum DiffRowFrequency
	{
		None,
		Alternating,
		AfterEachPair,
		All
	};

	enum DateCollationVisibility
	{
		Hide,
		Newest,
		Oldest
	};

	enum StringCollationVisibility
	{
		Show,
		Hide,
		Auto
	};

	enum CollateAverageMethod
	{
		Mean,
		Median
	}

	class SummaryTableColumnFormatInfo
	{
		public SummaryTableColumnFormatInfo()
		{
			name = "Default";
			maxStringLength = Int32.MaxValue;
			maxStringLengthCollated = Int32.MaxValue;
		}
		public SummaryTableColumnFormatInfo(XElement element)
		{
			name = element.Attribute("name").Value.ToLower();

			string autoColorizeStr = element.GetSafeAttribute<string>("autoColorize", "highIsBad").ToLower();
			var modeList = Enum.GetValues(typeof(AutoColorizeMode));
			foreach (AutoColorizeMode mode in modeList)
			{
				if (mode.ToString().ToLower() == autoColorizeStr)
				{
					autoColorizeMode = mode;
					break;
				}
			}
			numericFormat = element.GetSafeAttribute<string>("numericFormat");
			maxStringLength = element.GetSafeAttribute<int>("maxStringLength", Int32.MaxValue );
			maxStringLengthCollated = element.GetSafeAttribute<int>("maxStringLengthCollated", Int32.MaxValue );
			if (maxStringLengthCollated == Int32.MaxValue)
			{
				maxStringLengthCollated = maxStringLength;
			}

			noWrap = element.GetSafeAttribute<string>("noWrap") == "true";

			if (IsDate())
			{
				dateFormat = element.GetSafeAttribute<string>("dateFormat");
				string timeZoneId = element.GetSafeAttribute<string>("dateTimeZoneId");
				dateTimeZone = TimeZoneInfo.Utc;
				if (timeZoneId != null)
				{
					// Matches time zones in HKEY_LOCAL_MACHINE\Software\Microsoft\Windows NT\CurrentVersion\Time Zones.
					// Will raise an exception if Id is invalid.
					dateTimeZone = TimeZoneInfo.FindSystemTimeZoneById(timeZoneId);
				}
			}

			includeValueWithBucketName = element.GetSafeAttribute<bool>("includeValueWithBucketName", true);
			string bucketNamesString = element.GetSafeAttribute<string>("valueBucketNames");
			if (bucketNamesString != null)
			{
				bucketNames = bucketNamesString.Split(',').ToList();
			}
			string bucketThresholdsString = element.GetSafeAttribute<string>("valueBucketThresholds");
			if (bucketThresholdsString != null)
			{
				bucketThresholds = bucketThresholdsString.Split(',').Select(valStr =>
				{
					if (float.TryParse(valStr, out float value))
					{
						return value;
					}
					return 0.0f;
				}).ToList();
			}

			colourThresholdList = ColourThresholdList.ReadColourThresholdListXML(element.Element("colourThresholds"), null);
		}

		public bool IsDate() => numericFormat == "date";

		public AutoColorizeMode autoColorizeMode = AutoColorizeMode.HighIsBad;
		public string name;
		public bool noWrap = false;
		public string numericFormat;
		public int maxStringLength;
		public int maxStringLengthCollated;
		// If we should display the actual value in parenthesis next to the bucket name (if a bucket exists).
		public bool includeValueWithBucketName = true;
		// The name of each bucket. The name is indexed by the threshold.
		public List<string> bucketNames = new List<string>();
		// The value thresholds that correspond to each bucket. If a name doesn't exist for a threshold, the last bucket name is used.
		public List<float> bucketThresholds = new List<float>();
		// Colour thresholds override for this column.
		public ColourThresholdList colourThresholdList = null;
		// Date properties
		public string dateFormat = null;
		public TimeZoneInfo dateTimeZone = null;
	};

	class SummaryTableColumn
	{
		public string name;
		public bool isNumeric = false;
		public string displayName;
		public bool isRowWeightColumn = false;
		public DiffRowFrequency diffRowFrequency = DiffRowFrequency.None;
		public bool bShowOnlyDiffRows = false;
		public bool isCountColumn = false;
		public bool isSortByColumn = false;
		// Column header tooltip. Displayed when hovering over the header.
		public string tooltip = null;
		// Multiplied with the colour of each cell to modify the colour (including header).
		public Colour columnColourModifier = null;
		public ColumnAggregateType aggregateType = ColumnAggregateType.None;
		public SummaryTableColumn aggregateBaseColumn = null; // For avg/min/max columns, this will point back to the base(avg) column

		List<double> doubleValues = new List<double>();
		List<string> stringValues = new List<string>();
		List<string> toolTips = new List<string>();
		// Per cell colour modifiers.
		Dictionary<int, Colour> colourModifiers = new Dictionary<int, Colour>();
		public SummaryTableElement.Type elementType;
		public SummaryTableColumnFormatInfo formatInfo = null;

		List<ColourThresholdList> colourThresholds = new List<ColourThresholdList>();
		ColourThresholdList colourThresholdOverride = null;
		public bool isVisible = true;

		public SummaryTableColumn(
			string inName,
			bool inIsNumeric,
			string inDisplayName,
			bool inIsRowWeightColumn,
			SummaryTableElement.Type inElementType,
			SummaryTableColumnFormatInfo inFormatInfo = null,
			string inTooltip = null,
			Colour inColumnColourModifier = null,
			ColumnAggregateType inAggregateType = ColumnAggregateType.None,
			SummaryTableColumn inAggregateBaseColumn = null,
			bool bInIsCountColumn = false,
			bool bInIsSortByColumn = false,
			bool bInIsVisible = true )
		{
			name = SummaryTableColumn.getAggregateTypePrefix(inAggregateType) + inName;
			isNumeric = inIsNumeric;
			displayName = inDisplayName;
			isRowWeightColumn = inIsRowWeightColumn;
			elementType = inElementType;
			formatInfo = inFormatInfo;
			tooltip = inTooltip;
			columnColourModifier = inColumnColourModifier;
			aggregateType = inAggregateType;
			aggregateBaseColumn = inAggregateBaseColumn;
			isCountColumn = bInIsCountColumn;
			isSortByColumn = bInIsSortByColumn;
			isVisible = bInIsVisible;
			if (inAggregateType == ColumnAggregateType.Avg && aggregateBaseColumn == null)
			{
				aggregateBaseColumn = this;
			}
		}

		public static string getAggregateTypePrefix(ColumnAggregateType aggregateType)
		{
			switch (aggregateType)
			{
				case ColumnAggregateType.Avg:
					return "Avg ";
				case ColumnAggregateType.Min:
					return "Min ";
				case ColumnAggregateType.Max:
					return "Max ";
				default:
					return "";
			}
		}

		public string getKey(bool bIncludeTypeQualifier = true)
		{
			return bIncludeTypeQualifier ? SummaryTable.GetElementTypeStatPrefix(this.elementType) + name.ToLower() : name.ToLower();
		}


		public string GetJsData()
		{
			string baseName = SummaryTable.GetBaseStatNameWithPrefixAndSuffix(name, out _, out string suffix);
			Dictionary<string, string> attributes = new Dictionary<string, string>();
			// Strip the column prefix (min/max/avg) since we're outputting the aggregate column types separately anyway
			// The suffix is included (e.g MemoryFreeMB Min)
			attributes.Add("name", baseName + suffix);
			if (aggregateType != ColumnAggregateType.None)
			{
				attributes.Add("aggregateType", aggregateType.ToString());
			}
			attributes.Add("elementType", elementType.ToString());
			return JsonSerializer.Serialize(attributes);
		}

		public SummaryTableColumn Clone()
		{
			SummaryTableColumn newColumn = new SummaryTableColumn(name, isNumeric, displayName, isRowWeightColumn, elementType, formatInfo, tooltip, columnColourModifier, aggregateType, aggregateBaseColumn, isCountColumn, isSortByColumn, isVisible );
			newColumn.doubleValues.AddRange(doubleValues);
			newColumn.stringValues.AddRange(stringValues);
			newColumn.colourThresholds.AddRange(colourThresholds);
			newColumn.toolTips.AddRange(toolTips);
			newColumn.colourModifiers = colourModifiers.ToDictionary(entry => entry.Key, entry => entry.Value); // Deep copy
			newColumn.diffRowFrequency = diffRowFrequency;
			return newColumn;
		}

		// In the case of CSV stats, returns the category. Otherwise, returns an empty string
		public string GetStatCategory()
		{
			if (elementType == SummaryTableElement.Type.CsvStatAverage || elementType == SummaryTableElement.Type.CsvStatMin || elementType == SummaryTableElement.Type.CsvStatMax)
			{
				int lastSlashIndex = name.LastIndexOf('/');
				if (lastSlashIndex != -1)
				{
					return name.Substring(0, lastSlashIndex);
				}
			}
			return "";
		}

		public bool IsDateFormat()
		{
			return formatInfo != null && formatInfo.IsDate();
		}

		private double FilterInvalidValue(double value)
		{
			return value == double.MaxValue ? 0.0 : value;
		}

		public double GetMaxValue()
		{
			if (!isNumeric)
			{
				return double.MaxValue;
			}
			double maxValue = double.MinValue;
			int count = 0;
			foreach (double value in doubleValues)
			{
				if (value != double.MaxValue)
				{
					maxValue = Math.Max(value, maxValue);
					count++;
				}
			}
			if (count == 0)
			{
				return 0.0;
			}
			return maxValue;
		}

		public double GetMinValue()
		{
			if (!isNumeric)
			{
				return double.MaxValue;
			}
			double minValue = double.MaxValue;
			int count = 0;
			foreach (double value in doubleValues)
			{
				if (value != double.MaxValue)
				{
					minValue = Math.Min(value, minValue);
					count++;
				}
			}
			if (count == 0)
			{
				return 0.0;
			}
			return minValue;
		}

		public double GetAvgValue()
		{
			if (!isNumeric)
			{
				return double.MaxValue;
			}
			double total = 0.0;
			int count = 0;
			foreach (double value in doubleValues)
			{
				if (value != double.MaxValue)
				{
					total += value;
					count++;
				}
			}
			if (count == 0)
			{
				return 0.0;
			}
			return total / (double)count;
		}


		public void AddDiffRows(bool bIsFirstColumn, DiffRowFrequency inDiffRowFrequency, bool bShowOnlyDiffRows)
		{
			if (diffRowFrequency != DiffRowFrequency.None)
			{
				throw new Exception("Column already has diff rows!");
			}
			// Add a diff row for every row after the first one
			int oldCount = GetCount();

			diffRowFrequency = inDiffRowFrequency;
			bool bDiffRowsAlternating = diffRowFrequency == DiffRowFrequency.Alternating;

			int diffRowCount = bDiffRowsAlternating ? oldCount - 1 : oldCount / 2;
			int newCount = oldCount + diffRowCount;

			// Create new lists with counts reserved
			List<double> newDoubleValues = new List<double>(doubleValues.Count > 0 ? newCount : 0);
			List<string> newStringValues = new List<string>(stringValues.Count > 0 ? newCount : 0);
			List<string> newToolTips = new List<string>(toolTips.Count > 0 ? newCount : 0);
			List<ColourThresholdList> newColourThresholds = new List<ColourThresholdList>(colourThresholds.Count > 0 ? newCount : 0);

			bool bComputeDiff = isNumeric && !IsDateFormat() && !isCountColumn;

			static bool NeedsDiffColumn(int originalRowIndex, bool bShowIntermediateDiffRows)
			{
				return bShowIntermediateDiffRows ? originalRowIndex > 0 : originalRowIndex % 2 == 1;
			}

			// Add diff rows to each of the arrays
			for (int i = 0; i < doubleValues.Count; i++)
			{
				newDoubleValues.Add(doubleValues[i]);
				if (NeedsDiffColumn(i, bDiffRowsAlternating))
				{
					if (bComputeDiff)
					{
						double thisValue = FilterInvalidValue(doubleValues[i]);
						double prevValue = FilterInvalidValue(doubleValues[i - 1]);
						newDoubleValues.Add(thisValue - prevValue);
					}
					else
					{
						double valueToShow = 0.0;
						if (bShowOnlyDiffRows && isCountColumn )
						{
							// If we're showing only diff rows then display the count column if the values are equal
							double thisValue = FilterInvalidValue(doubleValues[i]);
							double prevValue = FilterInvalidValue(doubleValues[i - 1]);
							if (thisValue == prevValue)
							{
								valueToShow = prevValue;
							}
						}

						newDoubleValues.Add(valueToShow);
					}
				}
			}
			for (int i = 0; i < stringValues.Count; i++)
			{
				newStringValues.Add(stringValues[i]);
				if (NeedsDiffColumn(i, bDiffRowsAlternating))
				{
					if (bShowOnlyDiffRows)
					{
						if (stringValues[i] == stringValues[i-1])
						{
							newStringValues.Add(stringValues[i]); 
						}
						else
						{
							newStringValues.Add("");
						}
					}
					else
					{
						newStringValues.Add(bIsFirstColumn ? "Diff" : "");
					}					
				}
			}
			for (int i = 0; i < toolTips.Count; i++)
			{
				newToolTips.Add(toolTips[i]);
				if (NeedsDiffColumn(i, bDiffRowsAlternating))
				{
					newToolTips.Add("");
				}
			}
			for (int i = 0; i < colourThresholds.Count; i++)
			{
				newColourThresholds.Add(colourThresholds[i]);
				if (NeedsDiffColumn(i, bDiffRowsAlternating))
				{
					newColourThresholds.Add(null);
				}
			}

			doubleValues = newDoubleValues;
			stringValues = newStringValues;
			toolTips = newToolTips;
			colourThresholds = newColourThresholds;
		}

		bool IsDiffRow(int rowIndex)
		{
			return SummaryTable.IsDiffRow(diffRowFrequency, rowIndex);
		}

		// Computes a score (significance indicator) for a column based on its diff values. This takes into account the max value. If LowIsBad for this column then the sign is reversed
		public double GetDiffScore()
		{
			if (diffRowFrequency == DiffRowFrequency.None || !isNumeric)
			{
				return 0.0;
			}
			bool bLowIsBad = formatInfo != null && formatInfo.autoColorizeMode == AutoColorizeMode.LowIsBad;

			// Find the max of all diff values for this column. If LowIsBad then we reverse the sign

			double maxDiffScore = double.MinValue;
			for (int rowIndex = 0; rowIndex < GetCount(); rowIndex++)
			{
				if (IsDiffRow(rowIndex))
				{
					double diffValue = GetValue(rowIndex);
					maxDiffScore = Math.Max(maxDiffScore, bLowIsBad ? -diffValue : diffValue);
				}
			}
			return maxDiffScore;
		}

		// Computes the max of the abs diff values for a column
		public double GetMaxAbsDiff()
		{
			if (diffRowFrequency == DiffRowFrequency.None || !isNumeric)
			{
				return 0.0;
			}
			double maxAbsDiff = double.MinValue;
			for (int rowIndex = 0; rowIndex < GetCount(); rowIndex++)
			{
				if (IsDiffRow(rowIndex))
				{
					maxAbsDiff = Math.Max(maxAbsDiff, Math.Abs(GetValue(rowIndex)));
				}
			}
			return maxAbsDiff;
		}

		public string GetDisplayName(string hideStatPrefix=null, bool bAddStatCategorySeparatorSpaces = true, bool bGreyOutStatCategories = false)
		{
			if (displayName != null)
			{
				return displayName;
			}
			// Trim the stat name suffix if necessary
			string statName = name;
			if (hideStatPrefix != null)
			{
				string baseStatName = SummaryTable.GetBaseStatNameWithPrefixAndSuffix(statName, out string prefix, out string suffix);
				if (baseStatName.ToLower().StartsWith(hideStatPrefix.ToLower() ) )
				{
					statName = prefix + baseStatName.Substring(hideStatPrefix.Length) + suffix;
				}
			}

			if (bGreyOutStatCategories)
			{
				string baseStatName = SummaryTable.GetBaseStatNameWithPrefixAndSuffix(statName, out string prefix, out _);
				int idx = baseStatName.LastIndexOf("/");
				if (idx >= 0)
				{
					statName = prefix + "<span class='greyText'>" + baseStatName.Substring(0, idx+1) + "</span><span class='blackText'>" + baseStatName.Substring(idx+1)+ "</span>";
				}
			}
			if (bAddStatCategorySeparatorSpaces)
			{
				// Add a space to enable line breaks between slashes to reduce column width.
				return statName.Replace("/", "/<wbr>");
			}
			return statName;
		}

		public void ReserveRows(int count)
		{
			if (isNumeric)
			{
				doubleValues.Capacity = count;
				colourThresholds.Capacity = count;
			}
			else
			{
				stringValues.Capacity = count;
			}			
		}

		public void SetValue(int index, double value)
		{
			if (!isNumeric)
			{
				// This is already a non-numeric column. Better treat this as a string value
				SetStringValue(index, value.ToString());
				return;
			}
			// Grow to fill if necessary
			if (index >= doubleValues.Count)
			{
				for (int i = doubleValues.Count; i <= index; i++)
				{
					doubleValues.Add(double.MaxValue);
				}
			}
			doubleValues[index] = value;
		}

		void convertToStrings()
		{
			if (isNumeric)
			{
				stringValues = new List<string>();
				foreach (float f in doubleValues)
				{
					stringValues.Add(f.ToString());
				}
				doubleValues = new List<double>();
				isNumeric = false;
			}
		}

		public void SetColourThresholds(int index, ColourThresholdList value)
		{
			// Grow to fill if necessary
			if (index >= colourThresholds.Count)
			{
				for (int i = colourThresholds.Count; i <= index; i++)
				{
					colourThresholds.Add(null);
				}
			}
			colourThresholds[index] = value;
		}

		public ColourThresholdList GetColourThresholds(int index)
		{
			if (index < colourThresholds.Count)
			{
				return colourThresholds[index];
			}
			return null;
		}

		public string GetBackgroundColor(int index)
		{
			ColourThresholdList thresholds = null;
			double value = GetValue(index);
			if (value == double.MaxValue || IsDiffRow(index))
			{
				return null;
			}

			if (formatInfo.colourThresholdList != null)
			{
				thresholds = formatInfo.colourThresholdList;
			}
			else if (colourThresholdOverride != null)
			{
				thresholds = colourThresholdOverride;
			}
			else
			{
				if (index < colourThresholds.Count)
				{
					thresholds = colourThresholds[index];
				}
				if (thresholds == null)
				{
					return null;
				}
			}

			Colour modifier = null;
			if (columnColourModifier != null)
			{
				// Column modifier takes precedence over the cell modifier
				modifier = columnColourModifier;
			}
			else if (colourModifiers.ContainsKey(index))
			{
				modifier = colourModifiers[index];
			}

			string colourString = thresholds.GetColourForValue(value);
			if (modifier != null)
			{
				var colour = new Colour(colourString.Replace("'", ""));
				colourString = (colour * modifier).ToHTMLString();
			}
			return colourString;
		}

		public string GetTextColor(int index)
		{
			const double absoluteIgnoreThreshold = 0.025;
        	if (!bShowOnlyDiffRows && isCountColumn)
			{
				// If we're showing only diff rows then don't display the count column's diff row as a diff for formatting
				return null;
			}
			if (IsDiffRow(index) && isNumeric && index < doubleValues.Count )
			{
				// For simplicity, just negate the diff value if lowIsBad
				double diffValue = doubleValues[index];
				if (formatInfo.autoColorizeMode == AutoColorizeMode.LowIsBad)
				{
					diffValue *= -1.0;
				}

				// Diff absolute value is insignificant: output faded color
				if (Math.Abs(diffValue) < absoluteIgnoreThreshold)
				{
					// Very close to zero: just output grey
					if (Math.Abs(diffValue) < 0.001f)
					{
						return "#A0A0A0";
					}
					// Slight red/green
					return diffValue > 0.0 ? "#B8A0A0" : "#A0B8A0";
				}

				double prevValue = FilterInvalidValue(doubleValues[index - 2]);
				double thisValue = FilterInvalidValue(doubleValues[index - 1]);

				double maxValue = Math.Max(thisValue, prevValue);
				double percentOfMax = 100.0 * diffValue / maxValue;
				string red = "#B00000";
				string green = "#008000";
				// More than half a percent of max: output full colours
				if (percentOfMax >= 0.5)
				{
					return red;
				}
				if (percentOfMax <= -0.5)
				{
					return green;
				}
				// Output faded red/green
				return diffValue > 0.0 ? "#B8A0A0" : "#A0B8A0";
			}
			return null;
		}


		public void ComputeColorThresholds(TableColorizeMode tableColorizeMode)
		{
			if ( tableColorizeMode == TableColorizeMode.Budget )
			{
				return;
			}
			if (tableColorizeMode == TableColorizeMode.Off)
			{
				// Set empty color thresholds. This clears existing thresholds from summaries
				colourThresholds = new List<ColourThresholdList>();
				return;
			}

			AutoColorizeMode autoColorizeMode = formatInfo.autoColorizeMode;
			if (autoColorizeMode == AutoColorizeMode.Off || !isNumeric)
			{
				return;
			}

			// Set a single colour threshold list for the whole column
			colourThresholds = new List<ColourThresholdList>();
			double maxValue = -double.MaxValue;
			double minValue = double.MaxValue;
			double totalValue = 0.0;
			double validCount = 0.0;
			for (int i = 0; i < doubleValues.Count; i++)
			{
				if (IsDiffRow(i))
				{
					continue;
				}
				double val = doubleValues[i];
				if (val != double.MaxValue)
				{
					maxValue = Math.Max(val, maxValue);
					minValue = Math.Min(val, minValue);
					totalValue += val;
					validCount += 1.0;
				}
			}
			if (minValue == maxValue || validCount == 0.0)
			{
				return;
			}

			double averageValue = totalValue / validCount;
			double range = maxValue - minValue;

			// Disable colorization where values are very similar
			// The range has to be outside 0.25% of the average and >0.01 to get colorized
			double colorizationRangeThreshold = Math.Max( Math.Abs(averageValue) * 0.0025, 0.01 ); 
			if (range < colorizationRangeThreshold)
			{
				return;
			}

			// Adjust Min/Max value to ensure close values are not just binary red/green. If min/max is within 1% of the average or 0.02, adjust accordingly
			double minColorizationRangeExtent = Math.Max( Math.Abs(averageValue) * 0.01, 0.02); 
			maxValue = Math.Max(maxValue, averageValue + minColorizationRangeExtent);
			minValue = Math.Min(minValue, averageValue - minColorizationRangeExtent);

			Colour green = Colour.Green;
			Colour yellow = Colour.Yellow;
			Colour red = Colour.Red;

			colourThresholdOverride = new ColourThresholdList();
			colourThresholdOverride.Add(new ThresholdInfo(minValue, (autoColorizeMode == AutoColorizeMode.HighIsBad) ? green : red));
			colourThresholdOverride.Add(new ThresholdInfo(averageValue, yellow));
			colourThresholdOverride.Add(new ThresholdInfo(averageValue, yellow));
			colourThresholdOverride.Add(new ThresholdInfo(maxValue, (autoColorizeMode == AutoColorizeMode.HighIsBad) ? red : green));
		}

		public List<string> GetHeaderAttributes()
		{
			var attributes = new List<string>();
			if (columnColourModifier != null)
			{
				attributes.Add($"style='background-color:{Colour.White * columnColourModifier};'");
			}
			// Generate the tooltip. This consists of a the raw name + element type, followed by a custom provided tooltip, if it exists
			string tooltipStr = SummaryTable.GetBaseStatNameWithPrefixAndSuffix(name, out _, out _);
			tooltipStr += " (" + this.elementType.ToString() + ")";
			if (tooltip != null)
			{
				// Append the custom tooltip with a line break between
				tooltipStr += "&#013;" + tooltip;
			}
			attributes.Add($"title='{tooltipStr}'");
			return attributes;
		}

		public int GetCount()
		{
			return Math.Max(doubleValues.Count, stringValues.Count);
		}
		public double GetValue(int index)
		{
			if (index >= doubleValues.Count)
			{
				return double.MaxValue;
			}
			return doubleValues[index];
		}

		public bool AreAllValuesOverThreshold(double threshold)
		{
			if (!isNumeric)
			{
				return true;
			}
			bool hasRealValue = false;
			foreach(double value in doubleValues)
			{
				if (value != double.MaxValue)
				{
					if (value <= threshold)
					{
						return false;
					}
					hasRealValue = true;
				}
			}
			return hasRealValue;
		}

		public void SetStringValue(int index, string value)
		{
			if (isNumeric)
			{
				// Better convert this to a string column, since we're trying to add a string to it
				convertToStrings();
			}
			// Grow to fill if necessary
			if (index >= stringValues.Count)
			{
				for (int i = stringValues.Count; i <= index; i++)
				{
					stringValues.Add("");
				}
			}
			stringValues[index] = value;
			isNumeric = false;
		}
		public string GetStringValue(int index, bool roundNumericValues = false, string forceNumericFormat = null)
		{
			if (isNumeric)
			{
				if (index >= doubleValues.Count || doubleValues[index] == double.MaxValue)
				{
					return "";
				}
				double val = doubleValues[index];

				string prefix = "";
				bool bIsDiffRow = IsDiffRow(index);
	        	if (!bShowOnlyDiffRows && isCountColumn)
				{
					// If we're showing only diff rows then don't display the count column's diff row as a diff for formatting
					bIsDiffRow=false;
				}

				if (bIsDiffRow)
				{
					if ( val == 0.0 )
					{
						return "";
					}
					if ( val > 0.0 )
					{
						prefix = "+";
					}
				}

				if (forceNumericFormat != null)
				{
					if (forceNumericFormat == "date" && val != double.MaxValue)
					{
						DateTimeOffset dateTimeOffset = DateTimeOffset.FromUnixTimeSeconds((long)val);
						TimeSpan timeZoneOffset = formatInfo.dateTimeZone.GetUtcOffset(dateTimeOffset);
						dateTimeOffset = dateTimeOffset.Add(timeZoneOffset);
						return dateTimeOffset.ToString(formatInfo.dateFormat);
					}
					return prefix + val.ToString(forceNumericFormat);
				}
				else if (roundNumericValues)
				{
					double absVal = Math.Abs(val);
					double frac = absVal - (double)Math.Truncate(absVal);
					if (absVal >= 250.0f || frac < 0.0001f)
					{
						return prefix+val.ToString("0");
					}
					if (absVal >= 50.0f)
					{
						return prefix + val.ToString("0.0");
					}
					if (absVal >= 0.1)
					{
						return prefix + val.ToString("0.00");
					}
					if (bIsDiffRow)
					{
						// Filter out close to zero results in diff columns
						if (absVal < 0.000)
						{
							return "";
						}
						return prefix + val.ToString("0.00");
					}
					else
					{
						return val.ToString("0.000");
					}
				}
				return prefix + val.ToString();
			}
			else
			{
				if (index >= stringValues.Count)
				{
					return "";
				}
				if (forceNumericFormat != null)
				{
					// We're forcing a numeric format on something that's technically a string, but since we were asked, we'll try to do it anyway 
					// Note: this is not ideal, but it's useful for collated table columns, which might get converted to non-numeric during collation
					try
					{
						return Convert.ToDouble(stringValues[index], System.Globalization.CultureInfo.InvariantCulture).ToString(forceNumericFormat);
					}
					catch { } // Ignore. Just fall through...
				}
				return stringValues[index];
			}
		}
		public void SetToolTipValue(int index, string value)
		{
			// Grow to fill if necessary
			if (index >= toolTips.Count)
			{
				for (int i = toolTips.Count; i <= index; i++)
				{
					toolTips.Add("");
				}
			}
			toolTips[index] = value;
		}
		public string GetToolTipValue(int index)
		{
			if (index >= toolTips.Count)
			{
				return "";
			}
			return toolTips[index];
		}

		public void DebugMarkRowInvalid(int index, string reason)
		{
			colourModifiers.Add(index, new Colour(0.5f, 0.5f, 0.5f, 0.5f));
			SetToolTipValue(index, $"Cell invalid: {reason}");
		}

		public void DebugMarkAsFiltered(string filterName, string reason)
		{
			tooltip = $"Filtered out by: {filterName}: {reason}";
			columnColourModifier = new Colour(0.5f, 0.5f, 0.5f, 0.5f);
		}
	};



	class SummaryTable
	{
		class SummaryTableColumnLookup
		{
			public SummaryTableColumnLookup()
			{
				Clear();
			}

			public void Clear()
			{
				for (int i = 0; i < (int)SummaryTableElement.Type.COUNT + 1; i++)
				{
					columnLookupByType[i] = new Dictionary<string, SummaryTableColumn>();
				}
			}

			public List<string> GetSortedKeyList(SummaryTableElement.Type type = SummaryTableElement.Type.ANY)
			{
				List<string> listOut = columnLookupByType[(int)type].Keys.ToList();
				listOut.Sort();
				return listOut;
			}

			public void Add(string key, SummaryTableColumn column)
			{
				// Is there a type qualifier prefix in the key? If so, determine the type from the qualifier and stip the prefix
				if (key.StartsWith("["))
				{
					throw new Exception("Key can't start with a name qualifier prefix ([)");
				}

				// Add to the specific type lookup
				columnLookupByType[(int)column.elementType][key] = column;

				// Try to add to the global lookup, but handle collisions with priority
				Dictionary<string, SummaryTableColumn> columnLookupAny = columnLookupByType[(int)SummaryTableElement.Type.ANY];
				if (columnLookupAny.TryGetValue(key, out SummaryTableColumn existingColumn))
				{
					// Handle collisions. These are allowed. If we want to be immune from collisions in lookups, we can specify a type
					// Prioritize summary table metrics over CSV stats, since this matches existing behaviour
					if (GetElementTypePriority(column.elementType) > GetElementTypePriority(existingColumn.elementType))
					{
						columnLookupAny[key] = column;
					}
				}
				else
				{
					columnLookupAny[key] = column;
				}
			}

			public bool ContainsKey(string key, SummaryTableElement.Type type = SummaryTableElement.Type.ANY)
			{
				return Get(key, type) != null;
			}

			public void Remove(string key, SummaryTableElement.Type type = SummaryTableElement.Type.ANY)
			{
				// Is there a type qualifier prefix in the key? If so, determine the type from the qualifier and stip the prefix
				if (type == SummaryTableElement.Type.ANY && key.StartsWith("["))
				{
					type = SummaryTable.GetQualfiedStatType(key, out key);
				}

				if (type == SummaryTableElement.Type.ANY)
				{
					// If a type isn't specified, we have to remove from all lookup tables, since there may be entries with different types and the same key
					for (int i = 0; i < (int)SummaryTableElement.Type.COUNT + 1; i++)
					{
						columnLookupByType[i].Remove(key);
					}
				}
				else
				{
					columnLookupByType[(int)type].Remove(key);

					// Only remove from the global column if the type is the same as specified
					Dictionary<string, SummaryTableColumn> columnLookupAny = columnLookupByType[(int)SummaryTableElement.Type.ANY];
					if (columnLookupAny.TryGetValue(key, out SummaryTableColumn column))
					{
						if (column.elementType == type)
						{
							columnLookupAny.Remove(key);
						}
					}
				}

			}

			public SummaryTableColumn Get(string key, SummaryTableElement.Type type = SummaryTableElement.Type.ANY)
			{
				// Is there a type qualifier prefix in the key? If so, determine the type from the qualifier and stip the prefix
				if (type == SummaryTableElement.Type.ANY && key.StartsWith("["))
				{
					type = SummaryTable.GetQualfiedStatType(key, out key);
				}

				if (columnLookupByType[(int)type].TryGetValue(key, out SummaryTableColumn columnOut))
				{
					return columnOut;
				}
				return null;
			}

			// Determines the priority in the event of collisions in the ANY lookup (used when no type is specified)
			public static int GetElementTypePriority(SummaryTableElement.Type type)
			{
				switch (type)
				{
					case SummaryTableElement.Type.CsvMetadata:
						return 2;
					case SummaryTableElement.Type.CsvStatAverage:
						return 1;
					case SummaryTableElement.Type.SummaryTableMetric:
						return 3;
					case SummaryTableElement.Type.ToolMetadata:
						return 4;
					case SummaryTableElement.Type.ExternalMetadata:
						return 5;
					case SummaryTableElement.Type.CsvStatMin:
						return -1;
					case SummaryTableElement.Type.CsvStatMax:
						return 0;
					default:
						return 6;
				}
			}

			Dictionary<string, SummaryTableColumn>[] columnLookupByType = new Dictionary<string, SummaryTableColumn>[(int)SummaryTableElement.Type.COUNT + 1];
		};

		public SummaryTable()
		{
		}

		public void SetColumnFormatInfo(SummaryTableColumnFormatInfoCollection collection)
		{
			foreach (SummaryTableColumn column in columns)
			{
				column.formatInfo = collection != null ? collection.GetFormatInfo(column.name) : SummaryTableColumnFormatInfoCollection.DefaultColumnInfo;
			}
		}

		string MakeRowSortKey(List<SummaryTableColumn> sortByColumnList, int rowIndex)
		{
			StringBuilder sb = new StringBuilder();
			foreach (SummaryTableColumn column in sortByColumnList)
			{
				sb.Append('{');
				sb.Append(column.GetStringValue(rowIndex));
				sb.Append('}');
			}
			return sb.ToString();
		}

		private SummaryTableColumn CollateColumn(SummaryTableColumn srcColumn, 
												 ColumnAggregateType aggregateType, 
												 List<int> collatedRowSourceRowCounts, 
												 SummaryTableColumn aggregateBaseColumn = null, 
												 bool bShowMergedStringValues = false, 
												 CollateAverageMethod collateAverageMethod = CollateAverageMethod.Mean)
		{
			return CollateColumn(srcColumn, aggregateType, collatedRowSourceRowCounts, aggregateBaseColumn, bShowMergedStringValues, out int temp, collateAverageMethod);
		}

		private SummaryTableColumn CollateColumn(SummaryTableColumn srcColumn, 
												 ColumnAggregateType aggregateType, 
												 List<int> collatedRowSourceRowCounts, 
												 SummaryTableColumn aggregateBaseColumn, 
												 bool bShowMergedStringValues,
												 out int validMultiRowCollationsCount,
												 CollateAverageMethod collateAverageMethod = CollateAverageMethod.Mean)
		{
			SummaryTableColumn collatedColumn = new SummaryTableColumn(srcColumn.name, srcColumn.isNumeric, null, false, srcColumn.elementType, srcColumn.formatInfo, srcColumn.tooltip, srcColumn.columnColourModifier, aggregateType, aggregateBaseColumn, false, srcColumn.isSortByColumn);
			collatedColumn.ReserveRows(collatedRowSourceRowCounts.Count);

			validMultiRowCollationsCount = 0;
			int srcRowIndexOffset = 0;
			for (int collatedRowIndex = 0; collatedRowIndex < collatedRowSourceRowCounts.Count; collatedRowIndex++)
			{
				int mergedRowCount = collatedRowSourceRowCounts[collatedRowIndex];
				int mergedRowEndIndex = srcRowIndexOffset + mergedRowCount;
				if (srcColumn.isNumeric)
				{
					// Set color thresholds based on the source column
					for (int srcRowIndex=srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
					{
						collatedColumn.SetColourThresholds(collatedRowIndex, srcColumn.GetColourThresholds(srcRowIndex));
					}

					switch (aggregateType)
					{
						case ColumnAggregateType.Avg:
							{
								if (collateAverageMethod == CollateAverageMethod.Mean)
								{
									double total = 0.0;
									double totalWeight = 0.0;
									for (int srcRowIndex = srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
									{
										double rowValue = srcColumn.GetValue(srcRowIndex);
										if (rowValue != double.MaxValue)
										{
											double rowWeight = (rowWeightings != null) ? rowWeightings[srcRowIndex] : 1.0;
											total += rowValue * rowWeight;
											totalWeight += rowWeight;
										}
									}
									if (totalWeight > 0.0)
									{
										collatedColumn.SetValue(collatedRowIndex, total / totalWeight);
									}
								}
								else if (collateAverageMethod == CollateAverageMethod.Median)
								{
									List<double> values = new List<double>();
									for (int srcRowIndex = srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
									{
										double rowValue = srcColumn.GetValue(srcRowIndex);
										if (rowValue != double.MaxValue)
										{
											values.Add(rowValue);
										}
									}
									if (values.Count > 0)
									{
										values.Sort();
										int midIndex = values.Count/2;
										double medianValue = (values.Count % 2 == 0) ? (values[midIndex-1]+values[midIndex])*0.5 : values[midIndex];
										collatedColumn.SetValue(collatedRowIndex, medianValue);
									}
								}
							}
							break;
						case ColumnAggregateType.Min:
							{
								double minValue = double.MaxValue;
								for (int srcRowIndex = srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
								{
									double rowValue = srcColumn.GetValue(srcRowIndex);
									if (rowValue != double.MaxValue)
									{
										minValue = Math.Min(minValue, srcColumn.GetValue(srcRowIndex));
									}
								}
								if (minValue != double.MaxValue)
								{
									collatedColumn.SetValue(collatedRowIndex, minValue);
								}
							}
							break;
						case ColumnAggregateType.Max:
							{
								double maxValue = double.MinValue;
								for (int srcRowIndex = srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
								{
									double rowValue = srcColumn.GetValue(srcRowIndex);
									if (rowValue != double.MaxValue)
									{
										maxValue = Math.Max(maxValue, srcColumn.GetValue(srcRowIndex));
									}
								}
								if (maxValue != double.MinValue)
								{
									collatedColumn.SetValue(collatedRowIndex, maxValue);
								}								
							}
							break;
					}
					if (mergedRowCount > 1)
					{
						validMultiRowCollationsCount++;
					}					
				}
				else 
				{
					// Collate non-numeric values
					HashSet<string> uniqueStringValues = new HashSet<string>();
					for (int srcRowIndex = srcRowIndexOffset; srcRowIndex < mergedRowEndIndex; srcRowIndex++)
					{
						uniqueStringValues.Add(srcColumn.GetStringValue(srcRowIndex));
					}

					bool bStringsMatch = true;
					string mergedValue = "";
					if (uniqueStringValues.Count > 1)
					{
						if (bShowMergedStringValues)
						{
							// Just show all the unique string values, separated by ;
							mergedValue = string.Join(';', uniqueStringValues);
						}
						else
						{
							// Don't attempt to merge
							mergedValue = "<i>(multiple)</i>";
						}
						bStringsMatch = false;
					}
					else if (uniqueStringValues.Count == 1)
					{
						mergedValue = uniqueStringValues.First();
					}

					// If the rows match and are not null or empty then count them.
					// Note that we do not count as valid merged rows unless Count>1 since otherwise a single count of 1 would cause a row to be visible if CollatedStringVisibility is Auto
					if (mergedValue != "" && bStringsMatch && mergedRowCount > 1)
					{
						validMultiRowCollationsCount++;
					}
					collatedColumn.SetStringValue(collatedRowIndex, mergedValue == null ? "" : mergedValue);
				}
				srcRowIndexOffset += mergedRowCount;
			}
			return collatedColumn;
		}

		public SummaryTable CollateSortedTable(List<string> collateByList, bool addMinMaxColumns, StringCollationVisibility stringCollationVisibility, DateCollationVisibility dateCollationVisibility, CollateAverageMethod collateAverageMethod)
		{
			// Find the collateBy columns
			HashSet<SummaryTableColumn> collateByColumns = new HashSet<SummaryTableColumn>();
			foreach (string collateBy in collateByList)
			{
				string key = collateBy.ToLower();
				if (columnLookup.ContainsKey(key))
				{
					collateByColumns.Add(columnLookup.Get(key));
				}
			}
			if (collateByColumns.Count == 0)
			{
				throw new Exception("None of the metadata strings were found:" + string.Join(", ", collateByList));
			}

			// Generate the filtered sortBy column list
			List<SummaryTableColumn> sortByColumnList = new List<SummaryTableColumn>();
			foreach (SummaryTableColumn srcColumn in columns)
			{
				if (collateByColumns.Contains(srcColumn))
				{
					// finalSortByList.Add(srcColumn.name.ToLower());
					sortByColumnList.Add(srcColumn);
					// Early out if we've found all the columns
					if (sortByColumnList.Count == collateByColumns.Count)
					{
						break;
					}
				}
			}

			// Count the number of rows for each collated row
			List<int> collatedRowSourceRowCounts = new List<int>();
			int sourceRowCount = 0;
			string currentRowSortKey = MakeRowSortKey(sortByColumnList, 0);
			for (int i = 0; i < rowCount; i++)
			{
				string nextSortKey = (i + 1 < rowCount) ? MakeRowSortKey(sortByColumnList, i + 1) : null;
				sourceRowCount++;

				// If this is the last row (nextSortKey=null) or if the sort key is different then write out the count
				if (nextSortKey != currentRowSortKey)
				{
					collatedRowSourceRowCounts.Add(sourceRowCount);
					currentRowSortKey = nextSortKey;
					sourceRowCount = 0;
				}
			}

			List<SummaryTableColumn> newColumns = new List<SummaryTableColumn>();

			// Generate the collated sortby columns
			foreach (SummaryTableColumn srcColumn in sortByColumnList)
			{
				SummaryTableColumn newColumn = new SummaryTableColumn(srcColumn.name, false, srcColumn.displayName, false, srcColumn.elementType, srcColumn.formatInfo, bInIsSortByColumn:true);
				newColumns.Add(newColumn);
				int srcRowIndex = 0;
				for (int i=0; i<collatedRowSourceRowCounts.Count; i++)
				{
					newColumn.SetStringValue(i, srcColumn.GetStringValue(srcRowIndex));
					srcRowIndex += collatedRowSourceRowCounts[i];
				}				
			}

			// Add the count column
			SummaryTableColumn countColumn = new SummaryTableColumn("Count", true, null, false, SummaryTableElement.Type.ToolMetadata, bInIsCountColumn: true);
			for (int i=0; i< collatedRowSourceRowCounts.Count; i++)
			{
				countColumn.SetValue(i, collatedRowSourceRowCounts[i]);
			}
			newColumns.Add(countColumn);

			int nonCollatedStartIndex = newColumns.Count;
			foreach (SummaryTableColumn column in columns)
			{
				// Add avg/min/max columns for this column if it's numeric and we didn't already add it above 
				if (collateByColumns.Contains(column))
				{
					continue;
				}
				if (column.isNumeric)
				{
					if (column.IsDateFormat())
					{
						if (dateCollationVisibility != DateCollationVisibility.Hide)
						{
							// If we're showing collated dates then determine min/max based on the visibility mode
							// TODO: expose this to format so we can set per column
							newColumns.Add(CollateColumn(column, dateCollationVisibility == DateCollationVisibility.Newest ? ColumnAggregateType.Max : ColumnAggregateType.Min, collatedRowSourceRowCounts));
						}
					}
					else
					{
						// Generate aggregate min/max/avg values
						SummaryTableColumn avgColumn = CollateColumn(column, ColumnAggregateType.Avg, collatedRowSourceRowCounts, collateAverageMethod: collateAverageMethod);
						newColumns.Add(avgColumn);
						if (addMinMaxColumns)
						{
							newColumns.Add(CollateColumn(column, ColumnAggregateType.Min, collatedRowSourceRowCounts, avgColumn));
							newColumns.Add(CollateColumn(column, ColumnAggregateType.Max, collatedRowSourceRowCounts, avgColumn));
						}
					}
				}
				else
				{
					if (stringCollationVisibility != StringCollationVisibility.Hide)
					{
						SummaryTableColumn stringDataColumn = CollateColumn(column, ColumnAggregateType.None, collatedRowSourceRowCounts, null, false, out int validMultiRowCollationsCount);
						// If StringCollationVisibility is Auto, only show the column if there were successful valid multi-row collations (ie ignoring collations where row count is 1)
						if (validMultiRowCollationsCount > 0 || stringCollationVisibility == StringCollationVisibility.Show)
						{
							newColumns.Add(stringDataColumn);
						}
					}
					// Special handling for the CSVID column. Make a hidden CSVIDs column with bShowMergedStringValues=true so we can track the source CSV IDs
					if (column.elementType == SummaryTableElement.Type.CsvMetadata && column.name == "csvid")
					{
						//SummaryTableColumn csvIdsColumn = new SummaryTableColumn("csvids", false, "csvids", false, SummaryTableElement.Type.ToolMetadata, bInIsVisible: false);
						SummaryTableColumn csvIdsColumn = CollateColumn(column, ColumnAggregateType.None, collatedRowSourceRowCounts, bShowMergedStringValues: true);
						csvIdsColumn.name = "csvids";
						csvIdsColumn.isVisible = false;
						csvIdsColumn.elementType = SummaryTableElement.Type.ToolMetadata;
						csvIdsColumn.aggregateType = ColumnAggregateType.None;
						newColumns.Add(csvIdsColumn);
					}
				}
			}

			SummaryTable newTable = new SummaryTable();
			newTable.columns = newColumns;
			newTable.InitColumnLookup();
			newTable.rowCount = collatedRowSourceRowCounts.Count;
			newTable.firstStatColumnIndex = nonCollatedStartIndex; 
			newTable.isCollated = true;
			newTable.hasMinMaxColumns = addMinMaxColumns;
			return newTable;
		}

		public Dictionary<string, dynamic> ToFlatJsonDict()
		{
			Dictionary<string, dynamic> result = new Dictionary<string, dynamic>();

			SummaryTableColumn csvIdColumn = columnLookup.Get("csvid");
			List<SummaryTableColumn> sortByColumns = columns.Where(c => c.isSortByColumn).ToList();

			for (int rowIndex = 0; rowIndex < rowCount; rowIndex++)
			{
				string rowKey;
				if (isCollated && sortByColumns.Count > 0)
				{
					rowKey = string.Join("_", sortByColumns.Select(c => c.GetStringValue(rowIndex)));
				}
				else if (csvIdColumn != null)
				{
					rowKey = csvIdColumn.GetStringValue(rowIndex);
				}
				else
				{
					rowKey = rowIndex.ToString();
				}

				Dictionary<string, dynamic> rowDict = new Dictionary<string, dynamic>();
				foreach (SummaryTableColumn column in columns)
				{
					if (!column.isVisible)
					{
						continue;
					}
					if (column.isNumeric)
					{
						double val = column.GetValue(rowIndex);
						if (val == double.MaxValue)
						{
							continue;
						}
						try
						{
							rowDict[column.name] = new decimal(val);
						}
						catch (OverflowException)
						{
							rowDict[column.name] = val;
						}
					}
					else
					{
						string val = column.GetStringValue(rowIndex);
						if (!string.IsNullOrEmpty(val))
						{
							rowDict[column.name] = val;
						}
					}
				}
				result[rowKey] = rowDict;
			}
			return result;
		}

		// Finds a particular aggregate column corresponding to a specified column
		private SummaryTableColumn GetAggregateColumn(SummaryTableColumn inColumn, ColumnAggregateType aggregateType)
		{
			// Add the avg column and the corresponding min/max columns
			if (inColumn.aggregateType == ColumnAggregateType.None)
			{
				throw new Exception("Column isn't an aggregate column");
			}

			// Aggregate columns keep a ref to the avg/base column so use that if appropriate
			if (aggregateType == ColumnAggregateType.Avg && inColumn.aggregateBaseColumn != null)
			{
				return inColumn.aggregateBaseColumn;
			}

			string prefix = GetBaseStatPrefix(inColumn.name);
			string lookupKey = ( SummaryTableColumn.getAggregateTypePrefix(aggregateType) + inColumn.name.Substring(prefix.Length) ).ToLower();
			SummaryTableColumn outColumn = columnLookup.Get(lookupKey);
			if (outColumn == null)
			{
				throw new Exception("Aggregate column "+lookupKey+" not found!");
			}
			return outColumn;
		}

		SummaryTableColumnLookup GetColumnLookup()
		{
			return columnLookup;
		}

		public static string GetElementTypeStatPrefix(SummaryTableElement.Type type)
		{
			switch (type)
			{
				case SummaryTableElement.Type.SummaryTableMetric:
					return "[metric]";
				case SummaryTableElement.Type.CsvStatAverage:
					return "[csv]";
				case SummaryTableElement.Type.CsvStatMin:
					return "[csvmin]";
				case SummaryTableElement.Type.CsvStatMax:
					return "[csvmax]";
				case SummaryTableElement.Type.CsvMetadata:
					return "[meta]";
				case SummaryTableElement.Type.ToolMetadata:
					return "[toolmeta]";
				case SummaryTableElement.Type.ExternalMetadata:
					return "[extmeta]";
				default:
					return "[invalid]";
			}

		}

		public static SummaryTableElement.Type GetQualfiedStatType(string fullNameWithQualifier)
		{
			if (fullNameWithQualifier.StartsWith("["))
			{
				if (fullNameWithQualifier.StartsWith("[metric]"))
				{
					return SummaryTableElement.Type.SummaryTableMetric;
				}
				else if (fullNameWithQualifier.StartsWith("[csv]"))
				{
					return SummaryTableElement.Type.CsvStatAverage;
				}
				else if (fullNameWithQualifier.StartsWith("[csvmin]"))
				{
					return SummaryTableElement.Type.CsvStatMin;
				}
				else if (fullNameWithQualifier.StartsWith("[csvmax]"))
				{
					return SummaryTableElement.Type.CsvStatMax;
				}
				else if (fullNameWithQualifier.StartsWith("[meta]"))
				{
					return SummaryTableElement.Type.CsvMetadata;
				}
				else if (fullNameWithQualifier.StartsWith("[toolmeta]"))
				{
					return SummaryTableElement.Type.ToolMetadata;
				}
				else if (fullNameWithQualifier.StartsWith("[extmeta]"))
				{
					return SummaryTableElement.Type.ExternalMetadata;
				}
			}
			return SummaryTableElement.Type.COUNT;
		}

		public static SummaryTableElement.Type GetQualfiedStatType(string fullNameWithQualifier, out string statNameOut)
		{
			if (fullNameWithQualifier.StartsWith("["))
			{
				int endIdx = fullNameWithQualifier.IndexOf("]");
				if (endIdx == -1)
				{
					throw new Exception("Filter stat " + fullNameWithQualifier + " includes a start bracket but no end bracket");
				}
				// Strip off the stat qualifier
				statNameOut = fullNameWithQualifier.Substring(endIdx+1);

				if (fullNameWithQualifier.StartsWith("[metric]"))
				{
					return SummaryTableElement.Type.SummaryTableMetric;
				}
				else if (fullNameWithQualifier.StartsWith("[csv]"))
				{
					return SummaryTableElement.Type.CsvStatAverage;
				}
				else if (fullNameWithQualifier.StartsWith("[csvmin]"))
				{
					return SummaryTableElement.Type.CsvStatMin;
				}
				else if (fullNameWithQualifier.StartsWith("[csvmax]"))
				{
					return SummaryTableElement.Type.CsvStatMax;
				}
				else if (fullNameWithQualifier.StartsWith("[meta]"))
				{
					return SummaryTableElement.Type.CsvMetadata;
				}
				else if (fullNameWithQualifier.StartsWith("[toolmeta]"))
				{
					return SummaryTableElement.Type.ToolMetadata;
				}
				else if (fullNameWithQualifier.StartsWith("[extmeta]"))
				{
					return SummaryTableElement.Type.ExternalMetadata;
				}
			}
			statNameOut = fullNameWithQualifier;
			return SummaryTableElement.Type.COUNT;
		}

		public SummaryTable SortAndFilter(List<string> columnFilterList, List<string> rowSortList, bool bReverseSort, string weightByColumnName, IEnumerable<ISummaryTableColumnFilter> additionalFilters, bool bSortTrailingDigitsAsNumeric, TableColumnSortMode tableColumnSortMode)
		{
			SummaryTable newTable = SortRows(rowSortList, bReverseSort, bSortTrailingDigitsAsNumeric);

			// Generate a column lookup 
			SummaryTableColumnLookup newColumnLookup = newTable.GetColumnLookup();

			// Make a list of all unique keys for each element type, including ANY
			List<string>[] allMetadataKeysLists = new List<string>[(int)SummaryTableElement.Type.COUNT + 1];
			for (int i = 0; i < allMetadataKeysLists.Length; i++)
			{
				allMetadataKeysLists[i] = newColumnLookup.GetSortedKeyList((SummaryTableElement.Type)i);
			}

			// Add columns from the column filter list in the order they appear
			List<SummaryTableColumn> newColumnList = new List<SummaryTableColumn>();
			foreach (string filterStr in columnFilterList)
			{
				string filterStrLower = filterStr.Trim().ToLower();

				// Check for a qualifier which specifies the stat type
				bool startWild = filterStrLower.StartsWith("*");
				bool endWild = filterStrLower.EndsWith("*");
				if (startWild || endWild)
				{
					// Use the qualified list for wildcard matching if this entry was qualified with a type
					SummaryTableElement.Type statType = GetQualfiedStatType(filterStrLower, out string unqualifiedKey);

					List<string> allKeysList = allMetadataKeysLists[(int)statType];
					List<string> keyList = CsvStats.WildcardMatchStringList(allKeysList, unqualifiedKey, false, true);

					// Resolve the keyList and output the columns 
					List<SummaryTableColumn> foundColumns = new List<SummaryTableColumn>();
					foreach (string key in keyList)
					{
						SummaryTableColumn column = newColumnLookup.Get(key, statType);
						if (column != null)
						{
							foundColumns.Add(column);
						}
					}
					// Apply sorting to wildcard columns if requested
					if (tableColumnSortMode == TableColumnSortMode.WildcardSortByMax)
					{
						foundColumns.Sort((a, b) => b.GetMaxValue().CompareTo(a.GetMaxValue()));
					}
					else if (tableColumnSortMode == TableColumnSortMode.WildcardSortByAvg)
					{
						foundColumns.Sort((a, b) => b.GetAvgValue().CompareTo(a.GetAvgValue()));
					}

					foreach (SummaryTableColumn column in foundColumns)
					{
						newColumnList.Add(column);
					}

				}
				else
				{
					SummaryTableColumn column = newColumnLookup.Get(filterStrLower);
					if (column != null)
					{
						newColumnList.Add(column);
					}
				}
			}

			SummaryTableColumn csvIdColumn = newColumnLookup.Get("csvid");

			// Remove duplicates
			newColumnList = newColumnList.Distinct().ToList();

			// Apply sorting to all columns if requested
			if (tableColumnSortMode == TableColumnSortMode.SortByMax)
			{
				newColumnList.Sort((a, b) => b.GetMaxValue().CompareTo(a.GetMaxValue()));
			}
			else if (tableColumnSortMode == TableColumnSortMode.SortByAvg)
			{
				newColumnList.Sort((a, b) => b.GetAvgValue().CompareTo(a.GetAvgValue()));
			}

			// Compute row weights
			if (weightByColumnName != null && newColumnLookup.ContainsKey(weightByColumnName))
			{
				SummaryTableColumn rowWeightColumn = newColumnLookup.Get(weightByColumnName);
				newTable.rowWeightings = new List<double>(rowWeightColumn.GetCount());
				for (int i = 0; i < rowWeightColumn.GetCount(); i++)
				{
					newTable.rowWeightings.Add(rowWeightColumn.GetValue(i));
				}
			}

			// Run the additional filters on the columns
			if (additionalFilters != null)
			{
				var filteredColumns = new HashSet<string>();
				foreach (ISummaryTableColumnFilter filter in additionalFilters)
				{
					newColumnList = newColumnList.Where(column => !filter.ShouldFilter(column, this)).ToList();
				}
			}
			newTable.columns = newColumnList;
			newTable.rowCount = rowCount;
			newTable.InitColumnLookup();

			// Add a hidden CSV ID column if there isn't a visible one already
			if (csvIdColumn != null && !newTable.columnLookup.ContainsKey("csvid"))
			{
				SummaryTableColumn destCol = csvIdColumn.Clone();
				destCol.isVisible = false;
				newTable.columns.Add(destCol);
				newTable.InitColumnLookup();
			}

			// Determine the rowSort columns
			foreach (string columnName in rowSortList )
			{
				SummaryTableColumn column = newTable.columnLookup.Get(columnName.ToLower());
				if (column != null)
				{
					column.isSortByColumn = true;
				}
			}
			return newTable;
		}

		public void AddDiffRows(bool bSortColumnsByDiff, double columnDiffDisplayThreshold, bool bInShowOnlyDiffRows, bool bDiffRowsAlternating)
		{
			// We can't add diff rows unless we have at least 2 rows
			if (rowCount < 2)
			{
				return;
			}
			diffRowFrequency = bDiffRowsAlternating ? DiffRowFrequency.Alternating : DiffRowFrequency.AfterEachPair;
			bShowOnlyDiffRows = bInShowOnlyDiffRows;
			for (int i=0; i<columns.Count; i++)
			{
				columns[i].AddDiffRows(i == 0, diffRowFrequency, bInShowOnlyDiffRows);
			}

			if (bDiffRowsAlternating)
			{
				rowCount += rowCount - 1;
			}
			else
			{
				rowCount += rowCount / 2;
			}
			if ( columnDiffDisplayThreshold > 0.0 )
			{
				FilterColumnsByDiffThreshold(columnDiffDisplayThreshold);
			}

			if ( bSortColumnsByDiff )
			{
				SortColumnsByDiffRows();
			}
			// Just set rowWeightings to null for now. We shouldn't need it, since we should have already collated by this point
			rowWeightings = null;
		}

		// Gives columns a display-friendly name and add a tooltip, if provided
		public void ApplyDisplayNameMapping(Dictionary<string, StatDisplayInfo> statDisplayNameMapping)
		{
			if (statDisplayNameMapping == null)
			{
				return;
			}
			foreach (SummaryTableColumn column in columns)
			{
				string statName = GetBaseStatNameWithPrefixAndSuffix(column.name, out string prefix, out string suffix);
				if (statDisplayNameMapping.TryGetValue(statName.ToLower(), out StatDisplayInfo statDisplayInfo))
				{
					if (column.displayName == null && statDisplayInfo.displayName != null)
					{
						column.displayName = prefix + statDisplayInfo.displayName + suffix;
					}
					if (column.tooltip == null && statDisplayInfo.toolTip != null)
					{
						column.tooltip = statDisplayInfo.toolTip;
					}
				}
			}
		}

		public static string GetBaseStatPrefix(string inName)
		{
			GetBaseStatNameWithPrefixAndSuffix(inName, out string prefix, out _);
			return prefix;
		}

		public static string GetBaseStatName(string inName)
		{
			return GetBaseStatNameWithPrefixAndSuffix(inName, out _, out _);
		}

		public static string GetBaseStatNameWithPrefixAndSuffix(string inName, out string prefix, out string suffix)
		{
			suffix = "";
			prefix = "";
			string statName = inName;
			if (inName.StartsWith("Avg ") || inName.StartsWith("Max ") || inName.StartsWith("Min "))
			{
				prefix = inName.Substring(0, 4);
				statName = inName.Substring(4);
			}
			if (statName.EndsWith(" Avg") || statName.EndsWith(" Max") || statName.EndsWith(" Min"))
			{
				suffix = statName.Substring(statName.Length - 4);
				statName = statName.Substring(0, statName.Length - 4);
			}
			return statName;
		}

		public void WriteToCSV(string csvFilename)
		{
			System.IO.StreamWriter csvFile = new System.IO.StreamWriter(csvFilename, false);
			List<string> headerRow = new List<string>();
			foreach (SummaryTableColumn column in columns)
			{
				headerRow.Add(column.name);
			}
			csvFile.WriteLine(string.Join(",", headerRow));

			for (int i = 0; i < rowCount; i++)
			{
				List<string> rowStrings = new List<string>();
				foreach (SummaryTableColumn column in columns)
				{
					string cell = column.GetStringValue(i, false);
					// Sanitize so it opens in a spreadsheet (e.g. for buildversion) 
					cell = cell.TrimStart('+');
					rowStrings.Add(cell);
				}
				csvFile.WriteLine(string.Join(",", rowStrings));
			}
			csvFile.Close();
		}

		/// <summary>
		/// An enumerable representation of the SummaryTable, by rows.
		/// </summary>
		/// <returns>The row of data.</returns>
		public IEnumerable<IEnumerable<KeyValuePair<SummaryTableColumn, dynamic>>> EnumerateTableDataByRow()
		{
			for (int i = 0; i < rowCount; i++)
			{
				IList<KeyValuePair<SummaryTableColumn,dynamic>> row = new List<KeyValuePair<SummaryTableColumn, dynamic>>(columns.Count);

				foreach (var col in columns)
				{
					dynamic value = col.isNumeric
						? col.GetValue(i)
						: col.GetStringValue(i);

					row.Add(new KeyValuePair<SummaryTableColumn, dynamic>(col, value));
				}

				yield return row;
			}
		}

		// Sorts columns by their diff score. 
		// Requires max rows. Also works with collated tables with min/max
		private void SortColumnsByDiffRows()
		{
			List<Tuple<SummaryTableColumn, double>> numericColumnSortKeyPairs = new List<Tuple<SummaryTableColumn, double>>();
			List<SummaryTableColumn> staticColumns = new List<SummaryTableColumn>();

			foreach (SummaryTableColumn column in columns)
			{
				if (column.isNumeric && !column.isCountColumn )
				{
					double maxDiffScore = column.GetDiffScore();
					if (hasMinMaxColumns)
					{
						// If we have a collated table with avg/min/max then sort by the avg column and we'll re-add the min/max columns later
						// Columns without a prefix will be treated as static and ordered first
						if (column.aggregateType == ColumnAggregateType.Avg)
						{
							numericColumnSortKeyPairs.Add(new Tuple<SummaryTableColumn, double>(column, maxDiffScore));
						}
						else if (column.aggregateType == ColumnAggregateType.None)
						{
							staticColumns.Add(column);
						}
					}
					else
					{
						numericColumnSortKeyPairs.Add(new Tuple<SummaryTableColumn, double>(column, maxDiffScore));
					}
				}
				else
				{
					staticColumns.Add(column);
				}
			}

			columns = new List<SummaryTableColumn>();
			columns.AddRange(staticColumns);

			// Sort the numeric columns by stat
			numericColumnSortKeyPairs.Sort((a, b) => -a.Item2.CompareTo(b.Item2));
			List<SummaryTableColumn> sortedNumericColumns = new List<SummaryTableColumn>();
			foreach (Tuple<SummaryTableColumn, double> pair in numericColumnSortKeyPairs)
			{
				sortedNumericColumns.Add(pair.Item1);
			}

			// Stable sort the columns by stat prefix
			IEnumerable<SummaryTableColumn> sortedNumericColumnsOrderedByCategory = sortedNumericColumns.OrderBy(column => column.GetStatCategory());

			foreach (SummaryTableColumn column in sortedNumericColumnsOrderedByCategory)
			{
				columns.Add(column);
				if (hasMinMaxColumns)
				{
					columns.Add(GetAggregateColumn(column, ColumnAggregateType.Min));
					columns.Add(GetAggregateColumn(column, ColumnAggregateType.Max));
				}
			}
		}

		// Filters out columns with a a max abs diff value below thes specified threshold
		// Requires max rows. Also works with collated tables with min/max
		private void FilterColumnsByDiffThreshold(double threshold)
		{
			List<SummaryTableColumn> newColumns = new List<SummaryTableColumn>();
			foreach (SummaryTableColumn column in columns)
			{
				if (column.isNumeric && !column.isCountColumn)
				{
					// If this is a min or max column then we use the Avg column to compute the diff score. All 3 columns will be filtered together
					double maxAbsDiff = ( hasMinMaxColumns && column.aggregateBaseColumn != null ) ? column.aggregateBaseColumn.GetMaxAbsDiff() : column.GetMaxAbsDiff();
					if (maxAbsDiff >= threshold)
					{
						newColumns.Add(column);
					}
				}
				else
				{
					newColumns.Add(column);
				}
			}
			columns = newColumns;
		}


		public void WriteToHTML(
			string htmlFilename,
			string VersionString,
			bool bSpreadsheetFriendlyStrings,
			List<SummarySectionBoundaryInfo> sectionBoundaries,
			bool bScrollableTable,
			TableColorizeMode tableColorizeMode,
			bool bAddMinMaxColumns,
			string hideStatPrefix,
			int maxColumnStringLength,
			string weightByColumnName,
			CollateAverageMethod collateAverageMethod,
			string title,
			bool bTranspose,
			SummaryTableStyle summaryTableStyle,
			bool bEmitJsData,
			bool showFilteredColumnDebug = false )
		{
			summaryTableStyle = summaryTableStyle ?? new SummaryTableStyle();

			System.IO.StreamWriter htmlFile = new System.IO.StreamWriter(htmlFilename, false);

			string cellPadding = "2px 4px 2px 4px";
			if (isCollated)
			{
				cellPadding = "4px 4px 4px 4px";
			}

			// Generate an automatic title
			if (title==null)
			{
				title = htmlFilename.Replace("_Email.html", "").Replace(".html", "").Replace("\\", "/");
				title = title.Substring(title.LastIndexOf('/') + 1);
			}

			htmlFile.WriteLine("<html>");
			htmlFile.WriteLine("<head><title>Perf Summary: "+ title + "</title>");

			bool bAddStatNameSpacing = !bTranspose;
			bool bGreyOutStatPrefixes = bScrollableTable && bTranspose;

			// Figure out the sticky column count
			int stickyColumnCount = 0;
			if (bScrollableTable)
			{
				stickyColumnCount = 1;
				if (isCollated && !bTranspose)
				{
					for (int i = 0; i < columns.Count; i++)
					{
						if (columns[i].isCountColumn)
						{
							stickyColumnCount = i + 1;
							break;
						}
					}
				}
			}

			// Automatically colorize the table if requested.
			// We run this even if colorize is off as we need to overwrite the values.
			if (tableColorizeMode == TableColorizeMode.Auto ||
				tableColorizeMode == TableColorizeMode.Off)
			{
				foreach (SummaryTableColumn column in columns)
				{
					column.ComputeColorThresholds(tableColorizeMode);
				}
			}

			//
			// Output the script block
			//
			List<string> scriptLines = new List<string>();
			if ( bEmitJsData )
			{
				scriptLines.Add( GetRowColumnJsData() );
			}
			scriptLines.Add($"SummaryTableIsTransposed={(bTranspose ? "true" : "false")};");


			if (bScrollableTable)
			{
				// Insert some javascript to make the columns sticky. It's not possible to do this for multiple columns with pure CSS, since you need to compute the X offset dynamically
				// We need to do this when the page is loaded or the window is resized
				scriptLines.Add(
					"var originalStyleElement = null; \n" +
					"document.addEventListener('DOMContentLoaded', function(event) { regenerateStickyColumnCss(); }) \n" +
					"window.addEventListener('resize', function(event) { regenerateStickyColumnCss(); }) \n" +
					"\n" +
					"function regenerateStickyColumnCss() { \n" +
					"  var styleElement=document.getElementById('pageStyle'); \n" +
					"  var table=document.getElementById('mainTable'); \n" +
					"  if ( table.rows.length < 2 ) \n" +
					"	return; \n" +
					"  if (originalStyleElement == null) \n" +
					"    originalStyleElement = styleElement.textContent; \n" +
					"  else \n" +
					"    styleElement.textContent = originalStyleElement  \n"
				);

				// Make the columns Sticky and compute their X offsets
				scriptLines.Add(
					"  var numStickyCols=" + stickyColumnCount + "; \n" +
					"  var xOffset=0; \n" +
					"  for (var i=0;i<numStickyCols;i++) \n" +
					"  { \n" +
					"	 var rBorderParam=(i==numStickyCols-1) ? 'border-right: 2px solid black;':''; \n" +
					"	 styleElement.textContent+='tr.lastHeaderRow th:nth-child('+(i+1)+') {  z-index: 8;  border-top: 2px solid black;  font-size: 11px; left:'+xOffset+'px;'+rBorderParam+'}'; \n" +
					"	 styleElement.textContent+='td:nth-child('+(i+1)+') {  position: -webkit-sticky;  position: sticky; z-index: 7; left:'+xOffset+'px; '+rBorderParam+'}'; \n" +
					"	 xOffset+=table.rows[1].cells[i].offsetWidth; \n" +
					"  } \n" +
					"} \n"
					);
			}

			if (scriptLines.Count > 0)
			{
				htmlFile.WriteLine("<script>");
				foreach (string line in scriptLines)
				{
					htmlFile.WriteLine(line);
				}
				htmlFile.WriteLine("</script>");
			}

			//
			// Output CSS
			//
			htmlFile.WriteLine("<style type='text/css' id='pageStyle'>");
			htmlFile.WriteLine("p {  font-family: 'Verdana', Times, serif; font-size: 12px }");
			htmlFile.WriteLine("h3 {  font-family: 'Verdana', Times, serif; font-size: 14px }");
			htmlFile.WriteLine("h2 {  font-family: 'Verdana', Times, serif; font-size: 16px }");
			htmlFile.WriteLine("h1 {  font-family: 'Verdana', Times, serif; font-size: 20px }");
			string tableCss = "";


			if (bScrollableTable)
			{
				int headerMinWidth = bTranspose ? 50 : summaryTableStyle.minColumnHeaderWidth;
				int headerMaxWidth = bTranspose ? 165 : summaryTableStyle.maxColumnHeaderWidth;
				int cellFontSize = bTranspose ? 12 : 10;
				int headerCellFontSize = bTranspose ? 10 : 9;
				int firstColVerticalPadding = bTranspose ? 5 : 0;
				string cellAlign = bTranspose ? "right" : "left";

				if (bAddMinMaxColumns)
				{
					headerMinWidth = 35;
				}

				tableCss =
					"table {table-layout: fixed;} \n" +
					"table, th, td { border: 0px solid black; border-spacing: 0; border-collapse: separate; padding: " + cellPadding + "; vertical-align: center; font-family: 'Verdana', Times, serif; font-size: " + cellFontSize + "px;} \n" +
					"td {" +
					"  border-right: 1px solid black;" +
					"  max-width: 450;" +
					"  overflow: hidden;" +
					"  overflow: hidden;" +
					"} \n" +
					"td:first-child {" +
					"  padding-right:10px; padding-left:5px;" +
					"  padding-top:" + firstColVerticalPadding + "px;padding-bottom:" + firstColVerticalPadding + "px;" +
					"} \n" +
					"td:not(:first-child) {" +
					"  text-align: " + cellAlign + ";" +
					"} \n" +
					"tr:first-element { border-top: 2px; border-bottom: 2px } \n" +
					"th {" +
					"  width: auto;" +
					"  max-width: " + headerMaxWidth + "px;" +
					"  min-width: " + headerMinWidth + "px;" +
					"  position: -webkit-sticky;" +
					"  position: sticky;" +
					"  border-right: 1px solid black;" +
					"  border-top: 2px solid black;" +
					"  z-index: 5;" +
					"  background-color: #ffffff;" +
					"  top:0;" +
					"  font-size: " + headerCellFontSize + "px;" +
					"  word-wrap: break-word;" +
					"  overflow: hidden;" +
					"  height: 60;" +
					"} \n" +
					"span.greyText {" +
					"  color: #808080;" +
					"  display: inline-block;"+
					"} \n"+
					"span.blackText {" +
					"  color: #000000;" +
					"  display: inline-block;" +
					"} \n";

				// Top-left cell of the table is always on top, big font, thick border
				tableCss += "tr:first-child th:first-child { z-index: 100;  border-right: 2px solid black; border-top: 2px solid black; font-size: 11px; top:0; left: 0px; } \n";

				tableCss += "th:first-child, td:first-child { border-left: 2px solid black; white-space: nowrap; max-width:800px;} \n";

				if (bAddMinMaxColumns && isCollated)
				{
					tableCss += "tr.lastHeaderRow th { top:60px; height:20px; } \n";
				}

				if (!isCollated)
				{
					tableCss += "td { max-height: 40px; height:40px } \n";
				}
				tableCss += "tr:last-child td{border-bottom: 2px solid black;} \n";

			}
			else
			{
				tableCss =
					"table, th, td { border: 2px solid black; border-collapse: collapse; padding: " + cellPadding + "; vertical-align: center; font-family: 'Verdana', Times, serif; font-size: 11px;} \n";
			}


			bool bOddRowsGray = !(!bAddMinMaxColumns || !isCollated);

			string oddColor = bOddRowsGray ? "#eaeaea" : "#ffffff";
			string evenColor = bOddRowsGray ? "#ffffff" : "#eaeaea";

			tableCss += "tr:nth-child(odd) {background-color: "+oddColor+";} \n";
			tableCss += "tr:nth-child(even) {background-color: "+evenColor+";} \n";

			tableCss += "tr:first-child {background-color: #ffffff;} \n";
			tableCss += "tr.lastHeaderRow th { border-bottom: 2px solid black; } \n";

			// Section start row styles
			tableCss += "tr.sectionStartLevel0 td { border-top: 2px solid black; } \n";
			tableCss += "tr.sectionStartLevel1 td { border-top: 1px solid black; } \n";
			tableCss += "tr.sectionStartLevel2 td { border-top: 1px dashed black; } \n";

			htmlFile.WriteLine(tableCss);

			htmlFile.WriteLine("</style>");
			htmlFile.WriteLine("</head><body>");


			HtmlTable htmlTable = new HtmlTable("id='mainTable'", 1, columns.Count);

			// Make a header row, but don't add it to the table yet
			HtmlTable.Row headerRow = new HtmlTable.Row();

			// Write the header
			if (isCollated)
			{
				HtmlTable.Row topHeaderRow = headerRow;

				// Add the special columns (up to Count) to the lower header row
				for (int i = 0; i < firstStatColumnIndex; i++)
				{
					SummaryTableColumn column = columns[i];
					if (column.isVisible)
					{
						headerRow.AddCell(column.GetDisplayName(hideStatPrefix, bAddStatNameSpacing, bGreyOutStatPrefixes), String.Join(" ", column.GetHeaderAttributes()));
					}
				}

				if (bAddMinMaxColumns)
				{
					topHeaderRow = htmlTable.CreateRow();
					htmlTable.numHeaderRows = 2;
					if (bScrollableTable)
					{
						topHeaderRow.AddCell("<h3>" + title + "</h3>", "colspan='" + firstStatColumnIndex + "'");
					}
					else
					{
						topHeaderRow.AddCell("", "colspan='" + firstStatColumnIndex + "'");
					}
					// Add the stat columns
					int aggregateColumnCount = 0;
					for (int i = firstStatColumnIndex; i < columns.Count; i++)
					{
						SummaryTableColumn column = columns[i];
						if (!column.isVisible)
						{
							continue;
						}

						string statName = GetBaseStatNameWithPrefixAndSuffix(column.GetDisplayName(hideStatPrefix, bAddStatNameSpacing, bGreyOutStatPrefixes), out string prefix, out string suffix);
						string cellText = prefix.Trim();
						if (column.aggregateType == ColumnAggregateType.None)
						{
							cellText = "-";
						}
						else if (column.IsDateFormat())
						{
							cellText = column.aggregateType == ColumnAggregateType.Max ? "<i>newest</i>" : "<i>oldest</i>";
						}
						headerRow.AddCell(cellText, "");
						aggregateColumnCount++;

						// Update the top row if this is the last column of an aggregate (ie max)
						if (column.aggregateType == ColumnAggregateType.None || column.aggregateBaseColumn == null || column.aggregateType == ColumnAggregateType.Max)
						{
							string attributes = "";
							if (aggregateColumnCount > 1)
							{
								attributes = $"colspan='{aggregateColumnCount}' ";
							}
							attributes += String.Join(" ", column.GetHeaderAttributes());
							topHeaderRow.AddCell(statName + suffix, attributes);
							aggregateColumnCount = 0;
						}

					}
				}
				else
				{
					// Add the stat columns
					for (int i = firstStatColumnIndex; i < columns.Count; i++)
					{
						SummaryTableColumn column = columns[i];
						if (column.isVisible)
						{
							string statName = GetBaseStatNameWithPrefixAndSuffix(column.GetDisplayName(hideStatPrefix, bAddStatNameSpacing, bGreyOutStatPrefixes), out _, out string suffix);
							headerRow.AddCell(statName + suffix, String.Join(" ", column.GetHeaderAttributes()));
						}
					}
				}
			}
			else
			{
				for (int i = 0; i < columns.Count; i++)
				{
					SummaryTableColumn column = columns[i];
					if (column.isVisible)
					{
						headerRow.AddCell(column.GetDisplayName(hideStatPrefix, bAddStatNameSpacing, bGreyOutStatPrefixes), String.Join(" ", column.GetHeaderAttributes()));
					}
				}
			}
			htmlTable.AddRow(headerRow);

			// Work out which rows are major/minor section boundaries
			Dictionary<int, int> rowSectionBoundaryLevel = new Dictionary<int, int>();
			if (sectionBoundaries != null && !bTranspose)
			{
				foreach (SummarySectionBoundaryInfo sectionBoundaryInfo in sectionBoundaries)
				{
					// Skip this section boundary info if it's not in this table type
					if (isCollated && !sectionBoundaryInfo.inCollatedTable)
					{
						continue;
					}
					if (!isCollated && !sectionBoundaryInfo.inFullTable)
					{
						continue;
					}
					string prevSectionName = "";
					for (int i = 0; i < rowCount; i++)
					{
						int boundaryLevel = 0;
						if (sectionBoundaryInfo != null)
						{
							// Work out the section name if we have section boundary info. When it changes, apply the sectionStart CSS class
							string sectionName = "";
							if (sectionBoundaryInfo != null && columnLookup.ContainsKey(sectionBoundaryInfo.statName)) 
							{
								// Get the section name
								if (!columnLookup.ContainsKey(sectionBoundaryInfo.statName))
								{
									continue;
								}
								SummaryTableColumn col = columnLookup.Get(sectionBoundaryInfo.statName);
								sectionName = col.GetStringValue(i);

								// if we have a start token then strip before it
								if (sectionBoundaryInfo.startToken != null)
								{
									int startTokenIndex = sectionName.IndexOf(sectionBoundaryInfo.startToken);
									if (startTokenIndex != -1)
									{
										sectionName = sectionName.Substring(startTokenIndex + sectionBoundaryInfo.startToken.Length);
									}
								}

								// if we have an end token then strip after it
								if (sectionBoundaryInfo.endToken != null)
								{
									int endTokenIndex = sectionName.IndexOf(sectionBoundaryInfo.endToken);
									if (endTokenIndex != -1)
									{
										sectionName = sectionName.Substring(0, endTokenIndex);
									}
								}
							}
							if (sectionName != prevSectionName && i > 0)
							{
								// Update the row's boundary type info
								boundaryLevel = sectionBoundaryInfo.level;
								if (rowSectionBoundaryLevel.ContainsKey(i))
								{
									// Lower level values override higher ones
									boundaryLevel = Math.Min(rowSectionBoundaryLevel[i], boundaryLevel);
								}
								rowSectionBoundaryLevel[i] = boundaryLevel;
							}
							prevSectionName = sectionName;
						}
					}
				}
			}

			// Add the rows to the table
			for (int rowIndex = 0; rowIndex < rowCount; rowIndex++)
			{
				if (!IsRowVisible(rowIndex))
				{
					continue;
				}
				string rowClassStr = "";

				// Is this a major/minor section boundary
				if (rowSectionBoundaryLevel.ContainsKey(rowIndex))
				{
					int sectionLevel = rowSectionBoundaryLevel[rowIndex];
					if (sectionLevel < 3)
					{
						rowClassStr = " class='sectionStartLevel" + sectionLevel + "'";
					}
				}

				HtmlTable.Row currentRow = htmlTable.CreateRow(rowClassStr);
				int columnIndex = 0;
				foreach (SummaryTableColumn column in columns)
				{
					if (!column.isVisible)
					{
						continue;
					}
					List<string> attributes = new List<string>();

					// Add the tooltip for non-collated tables
					if (!isCollated)
					{
						string toolTip = column.GetToolTipValue(rowIndex);
						if (toolTip == "")
						{
							if (column.isNumeric && column.formatInfo != null && column.formatInfo.IsDate())
							{
								// For dates use a standard UTC timestamp for the tooltip
								double val = column.GetValue(rowIndex);
								if (val < double.MaxValue )
								{
									DateTimeOffset dateTimeOffset = DateTimeOffset.FromUnixTimeSeconds((long)val);
									toolTip = dateTimeOffset.ToString("yyyy-MM-dd HH:mm:ss (UTC)");
								}
							}
							else
							{
								toolTip = column.GetDisplayName();
							}
						}
						attributes.Add("title='" + toolTip + "'");
					}
					string bgColour = column.GetBackgroundColor(rowIndex);

					if (bgColour != null)
					{
						attributes.Add("bgcolor=" + bgColour);
					}

					Dictionary<string, string> styleAttributes = new Dictionary<string, string>();
					string textColour = column.GetTextColor(rowIndex);
					if (textColour != null)
					{
						styleAttributes.Add("color",  textColour);
					}

					if (column.formatInfo != null && column.formatInfo.noWrap)
					{
						// Disable whitespace wrapping so the column is sized to the width of the longest cell item.
						styleAttributes.Add("white-space", "nowrap");
					}

					bool bold = false;
					SummaryTableColumnFormatInfo columnFormat = column.formatInfo;
					int maxStringLength = Math.Min( isCollated ? columnFormat.maxStringLengthCollated : columnFormat.maxStringLength, maxColumnStringLength);

					string numericFormat = columnFormat.numericFormat;
					string stringValue = column.GetStringValue(rowIndex, true, numericFormat);

					// Check if we have any value buckets, if so lookup the bucket name for the value and display that with the value.
					if (column.isNumeric && stringValue.Length > 0 && columnFormat.bucketNames.Count > 0 && columnFormat.bucketThresholds.Count > 0)
					{
						double value = column.GetValue(rowIndex);
						int bucketIndex = 0;
						for (bucketIndex = 0; bucketIndex < columnFormat.bucketThresholds.Count; ++bucketIndex)
						{
							if (value <= columnFormat.bucketThresholds[bucketIndex])
							{
								break;
							}
						}

						bucketIndex = Math.Min(bucketIndex, columnFormat.bucketNames.Count-1);
						if (columnFormat.includeValueWithBucketName)
						{
							stringValue = columnFormat.bucketNames[bucketIndex] + " (" + stringValue + ")";
						}
						else
						{
							stringValue = columnFormat.bucketNames[bucketIndex];
						}
					}

					string fullStringValue = stringValue;
					if (stringValue.Length > maxStringLength)
					{
						stringValue = TableUtil.SafeTruncateHtmlTableValue(stringValue, maxStringLength);
						// Show full value on hover for truncated cells
						string sanitizedFull = TableUtil.SanitizeHtmlString(fullStringValue).Replace("'", "&#39;");
						int titleIndex = attributes.FindIndex(a => a.StartsWith("title="));
						if (titleIndex >= 0)
						{
							attributes[titleIndex] = "title='" + sanitizedFull + "'";
						}
						else
						{
							attributes.Add("title='" + sanitizedFull + "'");
						}
					}
					if (bSpreadsheetFriendlyStrings && !column.isNumeric)
					{
						stringValue = "'" + stringValue;
					}
					currentRow.AddCell( (bold ? "<b>" : "") + stringValue + (bold ? "</b>" : ""), attributes, styleAttributes);
					columnIndex++;
				}
			}

			// Transpose the table if requested
			if (bTranspose)
			{
				htmlTable = htmlTable.Transpose();
				htmlTable.Set(0, 0, new HtmlTable.Cell(title, ""));

				// Add a section boundary where the stats start
				int firstStatRowIndex = htmlTable.numHeaderRows + firstStatColumnIndex - 1;
				if (firstStatRowIndex < htmlTable.rows.Count)
				{
					htmlTable.rows[firstStatRowIndex].attributes = " class='sectionStartLevel2'";
				}
			}

			// Assign IDs to rows and columns
			for (int i = 0; i < htmlTable.rows.Count; i++)
			{
				HtmlTable.Row row = htmlTable.rows[i];
				if (i < htmlTable.numHeaderRows)
				{
					row.id = $"headerRow_{i}";
					for (int j=0; j<row.cells.Count; j++)
					{
						row.cells[j].id = (i == htmlTable.numHeaderRows - 1) ? $"col_{j}" : $"topCol_{j}";
					}
				}
				else
				{
					row.id = $"row_{i - htmlTable.numHeaderRows}";
				}
			}

			// Apply final formatting
			htmlTable.rows[htmlTable.numHeaderRows-1].attributes += " class='lastHeaderRow'";

			if (bScrollableTable)
			{
				// Apply stripe colors to the sticky columns to make them opaque (these need to render on top).	Can't use per-cell CSS because we don't want to override existing cell colors
				string[] stripeColors = { "bgcolor='"+oddColor+"'", "bgcolor='"+evenColor+"'" };
				for (int rowIndex = htmlTable.numHeaderRows; rowIndex < htmlTable.rows.Count; rowIndex++)
				{
					HtmlTable.Row row = htmlTable.rows[rowIndex];
					for (int colIndex = 0; colIndex<Math.Min(row.cells.Count,stickyColumnCount); colIndex++ )
					{
						HtmlTable.Cell cell = row.cells[colIndex];
						if (cell != null && !cell.attributes.Contains("bgcolor="))
						{
							cell.attributes += " " + stripeColors[rowIndex % 2];
						}
					}
				}
			}

			htmlTable.WriteToHtml(htmlFile);

			string extraString = "";
			if (isCollated)
			{
				if (collateAverageMethod == CollateAverageMethod.Median)
				{
					extraString += " - Average: median";
				}
				else
				{
					extraString += " - Average: mean";
					if (weightByColumnName != null)
					{
						extraString += ", weighted";
					}
				}

			}

			htmlFile.WriteLine("<p style='font-size:8'>Created with PerfReportTool " + VersionString + extraString + "</p>");
			htmlFile.WriteLine("</body></html>");

			htmlFile.Close();
		}

		public SummaryTable SortRows(List<string> rowSortList, bool reverseSort, bool sortTrailingDigitsAsNumeric = false)
		{
			// Generate a sort key for each row
			List<KeyValuePair<string, int>> columnRemapping = new List<KeyValuePair<string, int>>();
			for (int i = 0; i < rowCount; i++)
			{
				string key = "";
				foreach (string s in rowSortList)
				{
					SummaryTableColumn column = columnLookup.Get(s.ToLower());
					if (column != null)
					{
						string columnKey = column.GetStringValue(i, false, "0000000000.0000000000");
						if (sortTrailingDigitsAsNumeric && !column.isNumeric)
						{
							// If there's an integer suffix in the column value, pad it with zeroes for sorting purposes
							int integerSuffixStartIndex = -1;
							for (int ci = columnKey.Length-1; ci>0; ci-- )
							{
								char c = columnKey[ci];
								if (c < '0' || c > '9')
								{
									break;
								}
								integerSuffixStartIndex = ci;
							}
							if (integerSuffixStartIndex >= 0)
							{
								string integerSuffixPadded = columnKey.Substring(integerSuffixStartIndex).PadLeft(12,'0');
								columnKey = columnKey.Substring(0, integerSuffixStartIndex) + integerSuffixPadded;
							}
						}
						key += "{" + columnKey + "}";
					}
					else
					{
						key += "{}";
					}
				}
				columnRemapping.Add(new KeyValuePair<string, int>(key, i));
			}

			// Sort the columns
			columnRemapping.Sort(delegate (KeyValuePair<string, int> m1, KeyValuePair<string, int> m2)
			{
				return m1.Key.CompareTo(m2.Key);
			});

			// Reorder the data in each column
			List<SummaryTableColumn> newColumns = new List<SummaryTableColumn>();
			foreach (SummaryTableColumn srcCol in columns)
			{
				SummaryTableColumn destCol = new SummaryTableColumn(srcCol.name, srcCol.isNumeric, null, false, srcCol.elementType, srcCol.formatInfo);
				for (int i = 0; i < rowCount; i++)
				{
					int srcIndex = columnRemapping[i].Value;
					int destIndex = reverseSort ? rowCount - 1 - i : i;
					if (srcCol.isNumeric)
					{
						destCol.SetValue(destIndex, srcCol.GetValue(srcIndex));
					}
					else
					{
						destCol.SetStringValue(destIndex, srcCol.GetStringValue(srcIndex));
					}
					destCol.SetColourThresholds(destIndex, srcCol.GetColourThresholds(srcIndex));
					destCol.SetToolTipValue(destIndex, srcCol.GetToolTipValue(srcIndex));
				}
				newColumns.Add(destCol);
			}
			SummaryTable newTable = new SummaryTable();
			newTable.columns = newColumns;
			newTable.rowCount = rowCount;
			newTable.firstStatColumnIndex = firstStatColumnIndex; 
			newTable.isCollated = isCollated;
			newTable.InitColumnLookup();
			return newTable;
		}

		// Writes a couple of javascript objects containing row and column info
		string GetRowColumnJsData()
		{
			List<string> lines = new List<string>();
			List<string> RowEntries = new List<string>();
			for (int rowIndex = 0; rowIndex < rowCount; rowIndex++)
			{
				if (!IsRowVisible(rowIndex))
				{
					continue;
				}
				// Output a dict for the row containing just the entries referenced in rowSort
				Dictionary<string, dynamic> rowDict = new Dictionary<string, dynamic>();

				// Add the csvID row data. Use the csvids column (hidden collated column containing all source csvids) if it exists, otherwise csvid
				SummaryTableColumn csvIdsColumn = columnLookup.Get("csvids", SummaryTableElement.Type.ToolMetadata);
				if (csvIdsColumn != null)
				{
					rowDict.Add(csvIdsColumn.name, csvIdsColumn.GetStringValue(rowIndex).Split(';'));
				}
				else
				{
					SummaryTableColumn csvIdColumn = columnLookup.Get("csvid", SummaryTableElement.Type.CsvMetadata);
					rowDict.Add(csvIdColumn.name, csvIdColumn.GetStringValue(rowIndex));
				}

				foreach (SummaryTableColumn column in columns)
				{ 
					if ( column.isSortByColumn && !rowDict.ContainsKey(column.name))
					{
						string value = column.formatInfo == null ? column.GetStringValue(rowIndex) : column.GetStringValue(rowIndex, false, column.formatInfo.numericFormat);
						rowDict.Add(column.name, value);
					}
				}
				RowEntries.Add(JsonSerializer.Serialize(rowDict));
			}
			lines.Add($"SummaryTableRowInfo=[{String.Join(", ", RowEntries)}];\n");

			List<string> ColumnEntries = new List<string>();
			foreach (SummaryTableColumn column in columns)
			{
				ColumnEntries.Add(column.GetJsData());
			}
			lines.Add($"SummaryTableColumnInfo=[{String.Join(", ", ColumnEntries)}];\n");

			return String.Join("\n", lines);
		}

		void InitColumnLookup()
		{
			columnLookup.Clear();
			foreach (SummaryTableColumn col in columns)
			{
				columnLookup.Add(col.name.ToLower(), col);
			}
		}

		public void AddRowData(SummaryTableRowData metadata, bool bIncludeCsvStatAverages, bool bIncludeCsvStatMin, bool bIncludeCsvStatMax, bool bIncludeHiddenStats)
		{
			foreach (string key in metadata.dict.Keys)
			{
				SummaryTableElement value = metadata.dict[key];
				if (value.type == SummaryTableElement.Type.CsvStatAverage && !bIncludeCsvStatAverages)
				{
					continue;
				}
				if (value.type == SummaryTableElement.Type.CsvStatMin && !bIncludeCsvStatMin)
				{
					continue;
				}
				if (value.type == SummaryTableElement.Type.CsvStatMax && !bIncludeCsvStatMax)
				{
					continue;
				}
				if (value.GetFlag(SummaryTableElement.Flags.Hidden) && !bIncludeHiddenStats)
				{
					continue;
				}
				SummaryTableColumn column = null;

				if (!columnLookup.ContainsKey(key))
				{
					column = new SummaryTableColumn(value.name, value.isNumeric, null, false, value.type);
					columnLookup.Add(key, column);
					columns.Add(column);
				}
				else
				{
					column = columnLookup.Get(key);
				}

				if (value.isNumeric)
				{
					column.SetValue(rowCount, (double)value.numericValue);
				}
				else
				{
					column.SetStringValue(rowCount, value.value);
				}
				column.SetColourThresholds(rowCount, value.colorThresholdList);
				column.SetToolTipValue(rowCount, value.tooltip);
			}
			rowCount++;
		}

		public int Count
		{
			get { return rowCount; }
		}

		public SummaryTableColumn GetColumnByName(string name)
		{
			return columnLookup.Get(name);
		}

		public bool IsRowVisible(int rowIndex)
		{
			return SummaryTable.IsRowVisible(diffRowFrequency, bShowOnlyDiffRows, rowIndex);
		}

		public static bool IsRowVisible(DiffRowFrequency diffRowFrequency, bool bShowOnlyDiffRows, int rowIndex)
		{
			if (bShowOnlyDiffRows)
			{
				return IsDiffRow(diffRowFrequency, rowIndex);
			}
			else
			{
				return true;
			}
		}

		public static bool IsDiffRow(DiffRowFrequency diffRowFrequency, int rowIndex)
		{
			if (rowIndex < 2)
			{
				return false;
			}
			switch (diffRowFrequency)
			{
				case DiffRowFrequency.Alternating:
					return ((rowIndex - 2) % 2) == 0;
				case DiffRowFrequency.AfterEachPair:
					return ((rowIndex - 2) % 3) == 0;
				case DiffRowFrequency.None:
				default:
					return false;
			}
		}


		SummaryTableColumnLookup columnLookup = new SummaryTableColumnLookup();
		List<SummaryTableColumn> columns = new List<SummaryTableColumn>();
		List<double> rowWeightings = null;
		int rowCount = 0;
		int firstStatColumnIndex = 0;
		bool isCollated = false;
		bool hasMinMaxColumns = false;
		DiffRowFrequency diffRowFrequency = DiffRowFrequency.None;
		bool bShowOnlyDiffRows = false;
	};

	class HtmlTable
	{
		public class Row
		{
			public Row(string inAttributes="", int reserveColumnCount=0, string inId=null)
			{
				attributes = inAttributes;
				id = inId;
				if (reserveColumnCount > 0)
				{
					cells = new List<Cell>(reserveColumnCount);
				}
				else
				{
					cells = new List<Cell>();
				}
			}

			public void AddCell(string contents="", string attributes="", string cellId = null)
			{
				cells.Add(new Cell(contents, attributes, false, cellId));
			}

			public void AddCell(string contents, List<string> attributes, Dictionary<string, string> styleAttributes, string cellId = null)
			{
				List<string> styleLines = styleAttributes.Select(pair => $"{pair.Key}:{pair.Value}").ToList();
				attributes.Add($"style='{String.Join(";", styleLines)}'");

				cells.Add(new Cell(contents, String.Join(" ",attributes), false, cellId));
			}


			public void WriteToHtml(System.IO.StreamWriter htmlFile, bool bIsHeaderRow )
			{
				string attributesStr = attributes.Length == 0 ? "" : " " + attributes;
				string idStr = id == null ? "" : $" id=\"{id}\"";
				htmlFile.Write("<tr" + idStr + attributesStr + ">");
				foreach (Cell cell in cells)
				{
					if (cell == null)
					{
						htmlFile.Write("<td/>");
					}
					else
					{
						cell.WriteToHtml(htmlFile, bIsHeaderRow);
					}
				}
				htmlFile.WriteLine("</tr>");
			}

			public void Set(int ColIndex, Cell cell)
			{
				if (cells.Count < ColIndex + 1)
				{
					cells.Capacity = ColIndex + 1;
					while (cells.Count < ColIndex + 1)
					{
						cells.Add(null);
					}
				}
				cells[ColIndex] = cell;
			}

			public string attributes;
			public string id;
			public List<Cell> cells;
		}

		public class Cell
		{
			public Cell()
			{
			}
			public Cell(Cell srcCell)
			{
				contents = srcCell.contents;
				attributes = srcCell.attributes;
				id = srcCell.id;
			}

			public Cell(string inContents, string inAttributes, bool bInIsHeader=false, string inId = null)
			{
				contents = inContents;
				attributes = inAttributes;
				id = inId;
			}

			public void WriteToHtml(System.IO.StreamWriter htmlFile, bool bIsHeaderRow)
			{
				string AttributesStr = attributes.Length == 0 ? "" : " "+ attributes;
				string IdStr = id == null ? "" : $" id=\"{id}\"";
				string CellType = bIsHeaderRow ? "th" : "td";
				htmlFile.Write("<" + CellType + IdStr + AttributesStr+">" + contents + "</" + CellType + ">");
			}

			public string contents;
			public string attributes;
			public string id;
		}

		public HtmlTable(string inAttributes="", int inNumHeaderRows=1, int reserveColumnCount=0)
		{
			rows = new List<Row>();
			attributes = inAttributes;
			numHeaderRows = inNumHeaderRows;
			reserveColumnCount = 0;
		}

		public HtmlTable Transpose()
		{
			// NOTE: Row attributes are stripped when transposing!
			HtmlTable transposedTable = new HtmlTable(attributes, 1, rows.Count);
			for (int i=0; i<rows.Count; i++)
			{
				Row row = rows[i];
				for (int j=0; j< row.cells.Count; j++)
				{
					Cell newCell = new Cell(row.cells[j]);
					transposedTable.Set(j, i, newCell);
				}
			}
			return transposedTable;
		}

		public void WriteToHtml(System.IO.StreamWriter htmlFile)
		{
			string attributesStr = attributes.Length == 0 ? "" : " " + attributes;
			htmlFile.WriteLine("<table"+attributesStr+">");

			// Compute the number of columns, just in case not all rows are equal
			for (int i=0; i<rows.Count; i++)
			{
				rows[i].WriteToHtml(htmlFile, i<numHeaderRows);
			}
			htmlFile.WriteLine("</table>");
		}

		public void Set(int RowIndex, int ColIndex, Cell cell)
		{
			// Make sure we have enough rows
			if (rows.Count < RowIndex + 1)
			{
				rows.Capacity = RowIndex + 1;
				while (rows.Count < RowIndex + 1)
				{
					rows.Add(new Row("", reserveColumnCount));
				}
			}
			// Make sure the row is big enough
			rows[RowIndex].Set(ColIndex, cell);
		}

		public Row CreateRow(string attributes="")
		{
			Row newRow = new Row(attributes, reserveColumnCount);
			rows.Add(newRow);
			return newRow; 
		}
		public void AddRow(Row row)
		{
			rows.Add(row);
		}

		public List<Row> rows;
		public string attributes;
		public int numHeaderRows = 0;
		int reserveColumnCount = 0;
	};

}