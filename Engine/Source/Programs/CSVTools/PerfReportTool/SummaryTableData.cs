// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;
using PerfReportTool;
using System.IO;
using System.Text.Json;
using System.Threading.Tasks;
using CSVStats;

namespace PerfSummaries
{

	class SummaryTableDataJsonWriteHelper
	{
		public SummaryTableDataJsonWriteHelper(string InJsonFilenameOrDirectory, bool bInSeparateFiles, bool bInCsvMetadataOnly, bool bInWriteAllElementData, bool bInWriteIndented=false)
		{
			if (bInSeparateFiles)
			{
				JsonDirectory = InJsonFilenameOrDirectory;
			}
			else
			{
				JsonFilename = InJsonFilenameOrDirectory;
			}
			bCsvMetadataOnly = bInCsvMetadataOnly;
			bWriteAllElementData = bInWriteAllElementData;
			bWriteIndented = bInWriteIndented;
		}
		public void AddRowData(SummaryTableRowData rowData)
		{
			if (!rowData.dict.ContainsKey("csvid"))
			{
				Console.WriteLine("Warning: SummaryTableDataJsonHelper.AddRowData - Row data contains no CSV ID! Skipping");
				return;
			}

			Dictionary<string, dynamic> RowDict = rowData.ToJsonDict(bCsvMetadataOnly, bWriteAllElementData);
			Dict.Add(rowData.dict["csvid"].value, RowDict);
		}

		public void WriteToJson(bool bSerializeToStream)
		{
			if (JsonDirectory != null)
			{
				Console.WriteLine("Writing summary table row data to directory : " + JsonDirectory);
				foreach (string csvId in Dict.Keys)
				{
					string filename = Path.Combine(JsonDirectory, csvId+".json");
					WriteJsonFile(bSerializeToStream, filename, Dict[csvId], bWriteIndented);
				}
			}
			else
			{
				Console.WriteLine("Writing summary table row data to file: " + JsonFilename);
				WriteJsonFile(bSerializeToStream, JsonFilename, Dict, bWriteIndented);
			}

		}

		public static void WriteJsonFile(bool bSerializeToStream, string filename, Dictionary<string, dynamic> dictToSerialize, bool bWriteIndented)
		{
			JsonSerializerOptions options = new JsonSerializerOptions { WriteIndented = bWriteIndented, NumberHandling = System.Text.Json.Serialization.JsonNumberHandling.AllowNamedFloatingPointLiterals };

			if (bSerializeToStream)
			{
				Task task = WriteJsonFileAsyncPrivate(options, filename, dictToSerialize);
				task.Wait();
			}
			else
			{
				string jsonString = JsonSerializer.Serialize(dictToSerialize, options);
				File.WriteAllText(filename, jsonString);
			}
		}


		private static async Task WriteJsonFileAsyncPrivate(JsonSerializerOptions options, string Filename, Dictionary<string, dynamic> DictToSerialize) 
		{
			// serialize JSON directly to a file
			FileStream createStream = File.Create(Filename);
			await JsonSerializer.SerializeAsync(createStream, DictToSerialize);
			createStream.Dispose(); // DisposeAsync doesn't exist in .Net 4.8 so we can't await it
		}

		Dictionary<string, dynamic> Dict = new Dictionary<string, dynamic>();
		string JsonFilename = null;
		string JsonDirectory = null;
		bool bCsvMetadataOnly;
		bool bWriteAllElementData;
		bool bWriteIndented;
	}



	class SummaryTableElement
	{
		// Bump this when making changes!
		public static int CacheVersion = 1;

		// NOTE: this is serialized. Don't change the order!
		public enum Type
		{
			CsvStatAverage,
			CsvMetadata,
			SummaryTableMetric,
			ToolMetadata,
			ExternalMetadata,
			CsvStatMin,
			CsvStatMax,
			COUNT,
			ANY=COUNT				
		};

		public enum Flags
		{
			Hidden = 0x01
		};

		private SummaryTableElement()
		{
		}

		public SummaryTableElement(Type inType, string inName, double inValue, ColourThresholdList inColorThresholdList, string inToolTip, uint inFlags = 0)
		{
			if (inName == "")
			{
				throw new Exception("Name cannot be empty!");
			}
			type = inType;
			name = inName;
			isNumeric = true;
			numericValue = inValue;
			value = inValue.ToString();
			colorThresholdList = inColorThresholdList;
			tooltip = inToolTip;
			flags = inFlags;
		}
		public SummaryTableElement(Type inType, string inName, string inValue, ColourThresholdList inColorThresholdList, string inToolTip, uint inFlags = 0)
		{
			if (inName == "")
			{
				throw new Exception("Name cannot be empty!");
			}
			type = inType;
			name = inName;
			numericValue = 0.0;
			isNumeric = false;
			colorThresholdList = inColorThresholdList;
			value = inValue;
			tooltip = inToolTip;
			flags = inFlags;
		}

		public static SummaryTableElement ReadFromCache(BinaryReader reader)
		{
			SummaryTableElement val = new SummaryTableElement();
			val.type = (Type)reader.ReadUInt32();
			val.name = reader.ReadString();
			val.value = reader.ReadString();
			val.tooltip = reader.ReadString();
			val.numericValue = reader.ReadDouble();
			val.isNumeric = reader.ReadBoolean();
			val.flags = reader.ReadUInt32();
			bool hasThresholdList = reader.ReadBoolean();
			if (hasThresholdList)
			{
				int thresholdCount = reader.ReadInt32();
				val.colorThresholdList = new ColourThresholdList();
				// TODO: support serializing ColourThresholdList.LerpColours. This will require adding backwards compat to the serialization system.
				for (int i = 0; i < thresholdCount; i++)
				{
					bool bHasColour = reader.ReadBoolean();
					Colour thresholdColour = null;
					if (bHasColour)
					{
						thresholdColour = new Colour(reader.ReadString());
					}
					double thresholdValue = reader.ReadDouble();
					ThresholdInfo info = new ThresholdInfo(thresholdValue, thresholdColour);
					val.colorThresholdList.Add(info);
				}
			}
			return val;
		}



		public void WriteToCache(BinaryWriter writer)
		{
			writer.Write((uint)type);
			writer.Write(name);
			writer.Write(value);
			writer.Write(tooltip);
			writer.Write(numericValue);
			writer.Write(isNumeric);
			writer.Write(flags);
			writer.Write(colorThresholdList != null);
			if (colorThresholdList != null)
			{
				writer.Write((int)colorThresholdList.Count);
				// TODO: support serializing ColourThresholdList.LerpColours. This will require adding backwards compat to the serialization system.
				foreach (ThresholdInfo thresholdInfo in colorThresholdList.Thresholds)
				{
					writer.Write(thresholdInfo.colour != null);
					if (thresholdInfo.colour != null)
					{
						writer.Write(thresholdInfo.colour.ToString());
					}
					writer.Write(thresholdInfo.value);
				}
			}
		}

		public SummaryTableElement Clone()
		{
			return (SummaryTableElement)MemberwiseClone();
		}

		public void SetFlag(Flags flag, bool value)
		{
			if (value)
			{
				flags |= (uint)flag;
			}
			else
			{
				flags &= ~(uint)flag;
			}
		}
		public bool GetFlag(Flags flag)
		{
			return (flags & (uint)flag) != 0;
		}

		public SummaryTableElement(Type typeIn, string elementName, Dictionary<string, dynamic> jsonDict)
		{
			name = elementName;
			type = typeIn;
			dynamic jsonValue = jsonDict["value"];

			isNumeric = !(jsonValue is string);
			if (isNumeric)
			{
				numericValue = (double)jsonValue;
				value = numericValue.ToString();
			}
			else
			{
				value = jsonValue;
			}
			if (jsonDict.TryGetValue("colorThresholdList", out dynamic colorThresholdListDynamic))
			{
				Dictionary<string, dynamic> colorThresholdDict = colorThresholdListDynamic;
				colorThresholdList = new ColourThresholdList(colorThresholdDict);
			}

			tooltip = "";
			if ( jsonDict.TryGetValue("tooltip", out dynamic tooltipDynamic ) )
			{
				tooltip = tooltipDynamic;
			}

			if (jsonDict.TryGetValue("flags", out dynamic flagsDynamic))
			{
				List<dynamic> flagStrings = flagsDynamic;
				flags = 0;
				foreach (string flagStr in flagStrings)
				{
					if ( Enum.TryParse<Flags>(flagStr, out Flags flag ) )
					{
						flags |= (uint)flag;
					}
				}
			}

		}

		public Dictionary<string, dynamic> ToJsonDict(bool bWriteType)
		{
			Dictionary<string, dynamic> Dict = new Dictionary<string, dynamic>();
			if (bWriteType)
			{
				Dict.Add("type", type.ToString());
			}
			if (isNumeric)
			{
				try
				{
					// Serialize as Decimal to avoid 0.1 being serialized as 0.1000000000000001 etc
					Dict.Add("value", new Decimal(numericValue));
				}
				catch (OverflowException)
				{
					// if Decimal constructor threw then this is a value it doesn't like such as NaN, inf, etc. In those
					// cases just pass through the value as a double
					Dict.Add("value", numericValue);
				}
			}
			else
			{
				Dict.Add("value", value);
			}
			if (!string.IsNullOrEmpty(tooltip))
			{
				Dict.Add("tooltip", tooltip);
			}
			if ( colorThresholdList != null)
			{
				Dict.Add("colorThresholdList", colorThresholdList.ToJsonDict());
			}
			List<string> FlagStrings = new List<string>();
			var FlagValues = Enum.GetValues(typeof(Flags));
			foreach (var FlagValue in FlagValues)
			{
				if ( (flags & (uint)(int)FlagValue) != 0)
				{
					FlagStrings.Add(FlagValue.ToString());
				}
			}
			if (FlagStrings.Count > 0)
			{
				Dict.Add("flags", FlagStrings);
			}
			return Dict;
		}
		public dynamic DynamicValue
		{
			get
			{
				if (isNumeric)
				{
					return numericValue;
				}
				return value;
			}
		}

		public Type type;
		public string name;
		public string value;
		public string tooltip;
		public ColourThresholdList colorThresholdList;
		public double numericValue;
		public bool isNumeric;
		public uint flags;
	}

	class SummaryTableRowData
	{
		public SummaryTableRowData()
		{
		}

		// TODO: If this is bumped beyond 6, we need to implement backwards compatibility
		static int CacheVersion = 6;

		public static SummaryTableRowData TryReadFromCache(string summaryTableCacheDir, string csvId)
		{
			string filename = Path.Combine(summaryTableCacheDir, csvId + ".prc");
			return TryReadFromCacheFile(filename);
		}

		public CsvMetadata ReadCsvMetadata()
		{
			CsvMetadata metadata = null;
			foreach (KeyValuePair<string, SummaryTableElement> entry in dict)
			{
				if (entry.Value.type == SummaryTableElement.Type.CsvMetadata)
				{
					if (metadata == null)
					{
						metadata = new CsvMetadata();
					}
					metadata.Values.Add(entry.Key, entry.Value.value);
				}
				if (entry.Value.type == SummaryTableElement.Type.CsvStatAverage)
				{
					// Metadata values appear first so we can early out
					break;
				}
			}
			return metadata;
		}

		public static SummaryTableRowData TryReadFromCacheFile(string filename, bool bReadJustInitialMetadata = false)
		{
			SummaryTableRowData metaData = null;
			if (!File.Exists(filename))
			{
				return null;
			}
			try
			{
				using (FileStream fileStream = new FileStream(filename, FileMode.Open, FileAccess.Read))
				{
					BinaryReader reader = new BinaryReader(fileStream);
					int version = reader.ReadInt32();
					int metadataValueVersion = reader.ReadInt32();
					if (version == CacheVersion && metadataValueVersion == SummaryTableElement.CacheVersion)
					{
						bool bEarlyOut = false;
						metaData = new SummaryTableRowData();
						int dictEntryCount = reader.ReadInt32();
						for (int i = 0; i < dictEntryCount; i++)
						{
							string key = reader.ReadString();
							SummaryTableElement value = SummaryTableElement.ReadFromCache(reader);
							// If we're just reading initial metadata then skip everything after ToolMetadata and CsvMetadata
							if (bReadJustInitialMetadata && value.type != SummaryTableElement.Type.ToolMetadata && value.type != SummaryTableElement.Type.CsvMetadata)
							{
								bEarlyOut = true;
								break;
							}
							metaData.dict.Add(key, value);
						}

						if (!bEarlyOut)
						{
							string endString = reader.ReadString();
							if (endString != "END")
							{
								Console.WriteLine("Corruption detected in " + filename + ". Skipping read");
								metaData = null;
							}
						}
					}
					reader.Close();
				}
			}
			catch (Exception e)
			{
				metaData = null;
				Console.WriteLine("Error reading from cache file " + filename + ": " + e.Message);
			}
			return metaData;
		}


		// Filter out external entries for caching purposes
		List<KeyValuePair<string, SummaryTableElement>> GetCacheableEntries()
		{
			List<KeyValuePair<string, SummaryTableElement>> cacheableEntries = new List<KeyValuePair<string, SummaryTableElement>>();
			foreach (KeyValuePair<string, SummaryTableElement> entry in dict)
			{
				if (entry.Value.type != SummaryTableElement.Type.ExternalMetadata)
				{
					cacheableEntries.Add(entry);
				}
			}
			return cacheableEntries;
		}

		public bool WriteToCache(string summaryTableCacheDir, string csvId)
		{
			string filename = Path.Combine(summaryTableCacheDir, csvId + ".prc");
			try
			{
				using (FileStream fileStream = new FileStream(filename, FileMode.Create))
				{
					BinaryWriter writer = new BinaryWriter(fileStream);
					writer.Write(CacheVersion);
					writer.Write(SummaryTableElement.CacheVersion);

					// Only write non-external entries to the cache
					List<KeyValuePair<string, SummaryTableElement>> cacheableEntries = GetCacheableEntries();
					writer.Write(cacheableEntries.Count);
					foreach (KeyValuePair<string, SummaryTableElement> entry in cacheableEntries)
					{
						writer.Write(entry.Key);
						entry.Value.WriteToCache(writer);
					}
					writer.Write("END");
					writer.Close();
				}
			}
			catch (IOException)
			{
				Console.WriteLine("Failed to write to cache file " + filename + ".");
				return false;
			}
			return true;
		}

		public int GetFrameCount()
		{
			if (!dict.ContainsKey("framecount"))
			{
				return 0;
			}
			return (int)dict["framecount"].numericValue;
		}

		public bool Contains(string key)
		{
			return dict.ContainsKey(key);
		}
		public SummaryTableElement Get(string key)
		{
			if ( dict.TryGetValue(key, out SummaryTableElement elementOut) )
			{
				return elementOut;
			}
			return null;
		}

		public void RemoveSafe(string name)
		{
			string key = name.ToLower();
			if (dict.ContainsKey(key))
			{
				dict.Remove(key);
			}
		}

		public void Add(SummaryTableElement.Type type, string name, double value, ColourThresholdList colorThresholdList = null, string tooltip = "", uint flags = 0)
		{
			string key = name.ToLower();
			SummaryTableElement metadataValue = new SummaryTableElement(type, name, value, colorThresholdList, tooltip, flags);
			try
			{
				dict.Add(key, metadataValue);
			}
			catch (System.ArgumentException)
			{
				//throw new Exception("Summary metadata key " + key + " has already been added");
				Console.WriteLine("Warning: Key " + key + " has already been added. Ignoring...");
			}
		}

		public void Add(SummaryTableElement.Type type, string name, string value, ColourThresholdList colorThresholdList = null, string tooltip = "", uint flags = 0)
		{
			string key = name.ToLower();
			double numericValue = double.MaxValue;

			// Always treat csvid as string
			bool bIsCsvId = type == SummaryTableElement.Type.CsvMetadata && name == "csvid";
			if (!bIsCsvId)
			{
				try
				{
					numericValue = Convert.ToDouble(value, System.Globalization.CultureInfo.InvariantCulture);
				}
				catch { }
			}


			SummaryTableElement metadataValue = null;
			if (numericValue != double.MaxValue)
			{
				metadataValue = new SummaryTableElement(type, name, numericValue, colorThresholdList, tooltip, flags);
			}
			else
			{
				metadataValue = new SummaryTableElement(type, name, value, colorThresholdList, tooltip, flags);
			}

			try
			{
				dict.Add(key, metadataValue);
			}
			catch (System.ArgumentException)
			{
				//throw new Exception("Summary metadata key " + key + " has already been added");
				Console.WriteLine("Warning: Key " + key + " has already been added. Ignoring...");
			}
		}
		public void Add(SummaryTableElement element)
		{
			dict.Add(element.name.ToLower(), element);
		}


		public void AddString(SummaryTableElement.Type type, string name, string value, ColourThresholdList colorThresholdList = null, string tooltip = "")
		{
			string key = name.ToLower();
			SummaryTableElement metadataValue = new SummaryTableElement(type, name, value, colorThresholdList, tooltip);
			dict.Add(key, metadataValue);
		}

		public SummaryTableRowData(Dictionary<string, dynamic> jsonDict)
		{
			foreach (string dataTypeStr in jsonDict.Keys)
			{
				if ( Enum.TryParse(dataTypeStr, out SummaryTableElement.Type dataType) )
				{
					Dictionary<string, dynamic> DataTypeDict = jsonDict[dataTypeStr];
					foreach(string elementName in DataTypeDict.Keys)
					{
						SummaryTableElement newElement = new SummaryTableElement(dataType, elementName, DataTypeDict[elementName]);
						Add(newElement);
					}
				}
			}
		}

		public Dictionary<string, dynamic> ToJsonDict(bool bCsvMetadataOnly, bool bWriteAllElementData)
		{
			Dictionary<string, dynamic> DictOut = new Dictionary<string, dynamic>();

			// Make a dictionary for each data type
			if (bCsvMetadataOnly)
			{
				DictOut[SummaryTableElement.Type.CsvMetadata.ToString()] = new Dictionary<string, dynamic>();
			}
			else
			{
				var DataTypes = Enum.GetValues(typeof(SummaryTableElement.Type));
				foreach (SummaryTableElement.Type dataType in DataTypes)
				{
					// External metadata is never cached
					if (dataType != SummaryTableElement.Type.COUNT && dataType != SummaryTableElement.Type.ExternalMetadata)
					{
						DictOut[dataType.ToString()] = new Dictionary<string, dynamic>();
					}
				}
			}

			foreach (string key in dict.Keys)
			{
				SummaryTableElement Element = dict[key];
				if (bCsvMetadataOnly && Element.type != SummaryTableElement.Type.CsvMetadata)
				{
					continue;
				}
				// External metadata is never cached
				if (Element.type == SummaryTableElement.Type.ExternalMetadata)
				{
					continue;
				}
				if (bWriteAllElementData)
				{
					DictOut[Element.type.ToString()][Element.name] = Element.ToJsonDict(false);
				}
				else
				{
					DictOut[Element.type.ToString()][Element.name] = Element.DynamicValue;
				}
			}

			return DictOut;
		}

		public Dictionary<string, SummaryTableElement> dict = new Dictionary<string, SummaryTableElement>();
	};

}