// Copyright Epic Games, Inc. All Rights Reserved.

using PerfReportTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml.Linq;

namespace PerfSummaries
{
	class Colour
	{
		public Colour(string str)
		{
			string hexStr = str.TrimStart('#');
			int hexValue = Convert.ToInt32(hexStr, 16);
			byte rb = (byte)((hexValue >> 16) & 0xff);
			byte gb = (byte)((hexValue >> 8) & 0xff);
			byte bb = (byte)((hexValue >> 0) & 0xff);
			r = ((float)rb) / 255.0f;
			g = ((float)gb) / 255.0f;
			b = ((float)bb) / 255.0f;
			alpha = 1.0f;
		}
		public Colour(uint hex, float alphaIn = 1.0f)
		{
			byte rb = (byte)((hex >> 16) & 0xff);
			byte gb = (byte)((hex >> 8) & 0xff);
			byte bb = (byte)((hex >> 0) & 0xff);

			r = ((float)rb) / 255.0f;
			g = ((float)rb) / 255.0f;
			b = ((float)rb) / 255.0f;

			alpha = alphaIn;
		}
		public Colour(Colour colourIn) { r = colourIn.r; g = colourIn.g; b = colourIn.b; alpha = colourIn.alpha; }
		public Colour(float rIn, float gIn, float bIn, float aIn = 1.0f) { r = rIn; g = gIn; b = bIn; alpha = aIn; }

		public static float Lerp(float A, float B,  float T)
		{
			return A * (1.0f - T) + B * T;
		}

		public static Colour Lerp(Colour Colour0, Colour Colour1, float t)
		{
			return new Colour(
				Lerp(Colour0.r, Colour1.r, t),
				Lerp(Colour0.g, Colour1.g, t),
				Lerp(Colour0.b, Colour1.b, t),
				Lerp(Colour0.alpha, Colour1.alpha, t));
		}

		public Colour LinearRGBToHSV()
		{
			float RGBMin = Math.Min(Math.Min(r, g), b);
			float RGBMax = Math.Max(Math.Max(r, g), b);
			float RGBRange = RGBMax - RGBMin;

			float Hue = (RGBMax == RGBMin ? 0.0f :
						 RGBMax == r    ? ((((g - b) / RGBRange) * 60.0f) + 360.0f % 360.0f) :
						 RGBMax == g    ? (((b - r) / RGBRange) * 60.0f) + 120.0f :
						 RGBMax == b    ? (((r - g) / RGBRange) * 60.0f) + 240.0f :
						   0.0f);
			
			float Saturation = (RGBMax == 0.0f ? 0.0f : RGBRange / RGBMax);
			float Value = RGBMax;

			// In the new color, R = H, G = S, B = V, A = A
			return new Colour(Hue, Saturation, Value, alpha);
		}

		/** Converts an HSV color to a linear space RGB color */
		public Colour HSVToLinearRGB()
		{
			// In this color, R = H, G = S, B = V
			float Hue = r;
			float Saturation = g;
			float Value = b;

			float HDiv60 = Hue / 60.0f;
			float HDiv60_Floor = (float)Math.Floor(HDiv60);
			float HDiv60_Fraction = HDiv60 - HDiv60_Floor;

			float[] RGBValues = {
				Value,
				Value * (1.0f - Saturation),
				Value * (1.0f - (HDiv60_Fraction * Saturation)),
				Value * (1.0f - ((1.0f - HDiv60_Fraction) * Saturation)),
			};

			uint[,] RGBSwizzle = {
				{0, 3, 1},
				{2, 0, 1},
				{1, 0, 3},
				{1, 2, 0},
				{3, 1, 0},
				{0, 1, 2},
			};
			uint SwizzleIndex = ((uint)HDiv60_Floor) % 6;

			return new Colour(RGBValues[RGBSwizzle[SwizzleIndex, 0]],
								RGBValues[RGBSwizzle[SwizzleIndex, 1]],
								RGBValues[RGBSwizzle[SwizzleIndex, 2]],
								alpha);
		}

		public static Colour LerpUsingHSV(Colour From, Colour To, float T)
		{
			Colour FromHSV = From.LinearRGBToHSV();
			Colour ToHSV = To.LinearRGBToHSV();

			float FromHue = FromHSV.r;
			float ToHue = ToHSV.r;

			// Take the shortest path to the new hue
			if(Math.Abs(FromHue - ToHue) > 180.0f)
			{
				if(ToHue > FromHue)
				{
					FromHue += 360.0f;
				}
				else
				{
					ToHue += 360.0f;
				}
			}

			float NewHue = Lerp(FromHue, ToHue, T);

			NewHue = NewHue % 360.0f;
			if( NewHue < 0.0f )
			{
				NewHue += 360.0f;
			}

			float NewSaturation = Lerp(FromHSV.g, ToHSV.g, T);
			float NewValue = Lerp(FromHSV.b, ToHSV.b, T);
			Colour Interpolated = new Colour(NewHue, NewSaturation, NewValue).HSVToLinearRGB();

			float NewAlpha = Lerp(From.alpha, To.alpha, T);
			Interpolated.alpha = NewAlpha;
			return Interpolated;
		}

		public static Colour operator *(Colour a, Colour b)
		{
			return new Colour(a.r * b.r, a.g * b.g, a.b * b.b, a.alpha * b.alpha);
		}

		public string ToHTMLString()
		{
			return "'" + ToString() + "'";
		}

		public override string ToString()
		{
			int rI = (int)(r * 255.0f);
			int gI = (int)(g * 255.0f);
			int bI = (int)(b * 255.0f);
			return "#" + rI.ToString("x2") + gI.ToString("x2") + bI.ToString("x2");
		}

		public static Colour White = new Colour(1.0f, 1.0f, 1.0f, 1.0f);
		public static Colour Black = new Colour(0, 0, 0, 1.0f);
		public static Colour Red = new Colour(1.0f, 0.4f, 0.4f);
		public static Colour Orange = new Colour(1.0f, 0.7f, 0.4f, 1.0f);
		public static Colour Yellow = new Colour(1.0f, 1.0f, 0.5f);
		public static Colour Green = new Colour(0.53f, 0.83f, 0.53f);

		public float r, g, b;
		public float alpha;
	};


	class ThresholdInfo
	{
		public ThresholdInfo(double inValue, Colour inColourOverride = null)
		{
			value = inValue;
			colour = inColourOverride;
		}
		public double value;
		public Colour colour;
	};

	class ColourThresholdList
	{
		public ColourThresholdList()
		{ }

		public ColourThresholdList(double redValue, double orangeValue, double yellowValue, double greenValue, bool lerpColours = true)
		{
			Add(new ThresholdInfo(greenValue, null));
			Add(new ThresholdInfo(yellowValue, null));
			Add(new ThresholdInfo(orangeValue, null));
			Add(new ThresholdInfo(redValue, null));
			LerpColours = lerpColours;
		}

		public ColourThresholdList(IEnumerable<ThresholdInfo> thresholds, bool lerpColours)
		{
			Thresholds = thresholds.ToList();
			LerpColours = lerpColours;
		}

		public ColourThresholdList(Dictionary<string, dynamic> JsonDict)
		{ 
			if (JsonDict.ContainsKey("thresholds"))
			{
				List<dynamic> thresholdsList = JsonDict["thresholds"];
				Thresholds = new List<ThresholdInfo>(thresholdsList.Count);
				foreach(dynamic thresholdValue in thresholdsList)
				{
					Add(new ThresholdInfo(thresholdValue, null));
				}
				if (JsonDict.ContainsKey("colors"))
				{
					List<dynamic> colorsList = JsonDict["colors"];
					int n = Math.Min(colorsList.Count, thresholdsList.Count);
					for (int i = 0; i < n; i++)
					{
						Thresholds[i].colour = new Colour(colorsList[i]);
					}
				}
				if (JsonDict.ContainsKey("lerpColours"))
				{
					LerpColours = JsonDict["lerpColours"];
				}
			}
		}

		public Dictionary<string, dynamic> ToJsonDict()
		{
			Dictionary<string, dynamic> Dict = new Dictionary<string, dynamic>();
			if (Thresholds.Count > 0)
			{
				List<double> ThresholdValues = new List<double>();
				List<string> ThresholdColors = new List<string>();

				bool bHasValidColors = false;
				foreach (ThresholdInfo thresholdInfo in Thresholds)
				{
					ThresholdValues.Add(thresholdInfo.value);
					ThresholdColors.Add(thresholdInfo.colour == null ? "null" : thresholdInfo.colour.ToString());

					if (thresholdInfo.colour != null)
					{
						bHasValidColors = true;
					}
				}

				Dict["thresholds"] = ThresholdValues;
				if (bHasValidColors)
				{
					Dict["colors"] = ThresholdColors;
				}
				Dict["lerpColours"] = LerpColours;
			}
			return Dict;
		}

		public static string GetThresholdColour(double value, List<ThresholdInfo> thresholds, bool lerpColours = true)
		{
			if (thresholds.Count == 0)
			{
				return Colour.White.ToHTMLString();
			}

			Colour col = null;
			for (int i = 0; i < thresholds.Count; ++i)
			{
				ThresholdInfo threshold = thresholds[i];
				if (threshold.colour == null)
				{
					throw new ArgumentException("All thresholds must have a valid colour set.", "thresholds");
				}

				if (value <= threshold.value)
				{
					if (i == 0)
					{
						col = threshold.colour;
						break;
					}

					if (lerpColours)
					{
						ThresholdInfo prevThreshold = thresholds[i - 1];
						double t = (value - prevThreshold.value) / (threshold.value - prevThreshold.value);
						col = Colour.Lerp(prevThreshold.colour, threshold.colour, (float)t);
					}
					else
					{
						col = threshold.colour;
					}
					break;
				}
			}

			if (col == null)
			{
				col = thresholds.Last().colour;
			}
			return col.ToHTMLString();
		}

		public static string GetThresholdColour(double value, double redValue, double orangeValue, double yellowValue, double greenValue,
			Colour redOverride = null, Colour orangeOverride = null, Colour yellowOverride = null, Colour greenOverride = null, bool lerpColours = true)
		{
			Colour green = (greenOverride != null) ? greenOverride : Colour.Green;
			Colour orange = (orangeOverride != null) ? orangeOverride : Colour.Orange;
			Colour yellow = (yellowOverride != null) ? yellowOverride : Colour.Yellow;
			Colour red = (redOverride != null) ? redOverride : Colour.Red;

			if (redValue > orangeValue)
			{
				redValue = -redValue;
				orangeValue = -orangeValue;
				yellowValue = -yellowValue;
				greenValue = -greenValue;
				value = -value;
			}

			List<ThresholdInfo> thresholds = new List<ThresholdInfo>
			{
				new ThresholdInfo(redValue, red),
				new ThresholdInfo(orangeValue, orange),
				new ThresholdInfo(yellowValue, yellow),
				new ThresholdInfo(greenValue, green)
			};
			return GetThresholdColour(value, thresholds, lerpColours);
		}

		public void Add(ThresholdInfo info)
		{
			Thresholds.Add(info);
		}
		public int Count
		{
			get { return Thresholds.Count; }
		}

		public string GetColourForValue(string value)
		{
			try
			{
				return GetColourForValue(Convert.ToDouble(value, System.Globalization.CultureInfo.InvariantCulture));
			}
			catch
			{
				return "'#ffffff'";
			}
		}

		public string GetColourForValue(double value)
		{
			if (Thresholds.Count == 4)
			{
				return GetThresholdColour(value, Thresholds[3].value, Thresholds[2].value, Thresholds[1].value, Thresholds[0].value, Thresholds[3].colour, Thresholds[2].colour, Thresholds[1].colour, Thresholds[0].colour, LerpColours);
			}
			else if (Thresholds.Count > 0)
			{
				return GetThresholdColour(value, Thresholds, LerpColours);
			}
			return "'#ffffff'";
		}

		public static string GetSafeColourForValue(ColourThresholdList list, string value)
		{
			if (list == null)
			{
				return "'#ffffff'";
			}
			return list.GetColourForValue(value);
		}
		public static string GetSafeColourForValue(ColourThresholdList list, double value)
		{
			if (list == null)
			{
				return "'#ffffff'";
			}
			return list.GetColourForValue(value);
		}

		public static ColourThresholdList ReadColourThresholdListXML(XElement colourThresholdEl, XmlVariableMappings vars)
		{
			if (colourThresholdEl != null)
			{
				ColourThresholdList thresholdList = new ColourThresholdList();

				thresholdList.LerpColours = colourThresholdEl.GetSafeAttribute<bool>(vars, "lerpColours", thresholdList.LerpColours);

				XElement thresholdsElement = colourThresholdEl.Element("thresholds");
				XElement coloursElement = colourThresholdEl.Element("colours");
				if (thresholdsElement != null && coloursElement != null)
				{
					// Colour thresholds support variables so resolve them if provided
					string[] thresholdStrings = thresholdsElement.GetValue(vars).Split(',');
					string[] colourStrings = coloursElement.GetValue(vars).Split(',');
					if (thresholdStrings.Length != colourStrings.Length)
					{
						throw new Exception("Number of colour thresholds and colours must match");
					}

					for (int i = 0; i < thresholdStrings.Length; ++i)
					{
						double value = double.Parse(thresholdStrings[i]);
						Colour colour = new Colour(colourStrings[i]);
						thresholdList.Add(new ThresholdInfo(value, colour));
					}
				}
				else
				{
					// Revert to previous logic if sub elements don't exist.
					double[] values = ReadColourThresholdsXML(colourThresholdEl, vars);
					if (values != null)
					{
						foreach (double value in values)
						{
							thresholdList.Add(new ThresholdInfo(value));
						}
					}
				}
				return thresholdList;
			}
			return null;
		}

		public static double[] ReadColourThresholdsXML(XElement colourThresholdEl, XmlVariableMappings vars )
		{
			if (colourThresholdEl != null)
			{
				string colourThresholdElementStr = colourThresholdEl.GetValue(vars);
				string[] colourStrings = colourThresholdElementStr.Split(',');
				if (colourStrings.Length != 4)
				{
					Console.Write("Incorrect number of colourthreshold entries. Should be 4: "+ colourThresholdEl.ToString());
					return null;
				}
				double[] colourThresholds = new double[4];
				for (int i = 0; i < colourStrings.Length; i++)
				{
					colourThresholds[i] = Convert.ToDouble(colourStrings[i], System.Globalization.CultureInfo.InvariantCulture);
				}
				return colourThresholds;
			}
			return null;
		}

		public List<ThresholdInfo> Thresholds = new List<ThresholdInfo>();
		// TODO: support serializing LerpColours. This will require adding backwards compat to the serialization system.
		public bool LerpColours = true;
	};
}