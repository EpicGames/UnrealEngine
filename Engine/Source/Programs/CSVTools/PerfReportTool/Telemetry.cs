// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Analytics;
using EpicGames.Analytics.PerformanceTrends;
using EpicGames.Horde;
using EpicGames.Horde.Telemetry;
using Microsoft.Extensions.DependencyInjection;
using PerfSummaries;
using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;

namespace PerfReportTool
{
	/// <summary>
	/// Telemetry class used to emit telemetry events for PerfReportTool.
	/// </summary>
	internal static class Telemetry
	{
		private static object _lock = new object();
		private const string DefaultAppId = "Horde";
		private const int DefaultAppVersion = 1;
		private const string DefaultAppEnvironment = "Dev";
		private const string EventName = "State.Performance";
		private const string DefaultSummaryTableName = "default";
		private const int CurrentSchemaVersion = 2;
		private const int DefaultTimeoutSeconds = 15;
		private static ServiceProvider _serviceProvider = null;
		private static IHordeClient _hordeClient = null;
		private static HordeHttpClient _hordeHttpClient = null;
		private readonly static List<Task<bool>> _queuedRequests = new List<Task<bool>>(16);
		private readonly static string _sessionId = Guid.NewGuid().ToString("N");

		/// <summary>
		/// Gets the HordeClient used to create Horde Http Clients.
		/// </summary>
		internal static IHordeClient HordeClient
		{
			get
			{
				if (_hordeClient == null)
				{
					if (_serviceProvider == null)
					{
						Uri defaultServer = HordeOptions.GetDefaultServerUrl() ?? HordeOptions.GetServerUrlFromEnvironment();

						if (defaultServer == null)
						{
							Console.WriteLine("[Telemetry] Unable to construct HttpClient due to invalid default server.");
							return null;
						}

						ServiceCollection services = new();
						services.AddHorde(o => o.ServerUrl = defaultServer);
						_serviceProvider = services.BuildServiceProvider();
					}

					_hordeClient = _serviceProvider.GetRequiredService<IHordeClient>();
				}

				return _hordeClient;
			}
		}

		/// <summary>
		/// Gets the Horde Http Client used to emit events.
		/// </summary>
		internal static HordeHttpClient HordeHttpClient
		{
			get
			{
				if (_hordeHttpClient == null)
				{
					IHordeClient client = HordeClient;
					if (client != null)
					{
						_hordeHttpClient = HordeClient.CreateHttpClient();
					}
				}

				return _hordeHttpClient;
			}
		}

		/// <summary>
		/// Flushes all telemetry events that have been queued.
		/// </summary>
		internal static void Flush()
		{
			Console.WriteLine("[Telemetry] Flushing {0} telemetry events.", _queuedRequests.Count);

			if (_queuedRequests.Count == 0)
			{
				return;
			}

			Task[] tasks;
			lock (_lock)
			{
				tasks = _queuedRequests.ToArray();
				_queuedRequests.Clear();
			}

			try
			{
				Task.WhenAll(tasks).GetAwaiter().GetResult();
			}
			catch
			{
				Console.WriteLine("[Telemetry] A task falted while awaiting all queued telemetry tasks.");
			}

			int succeeded = 0;
			int failed = 0;

			foreach (Task task in tasks)
			{
				if (task.IsCompletedSuccessfully)
				{
					succeeded++;
				}
				else if (task.IsFaulted || task.IsCanceled)
				{
					failed++;

					if (task.Exception != null)
					{
						foreach (Exception ex in task.Exception.Flatten().InnerExceptions)
						{
							Console.WriteLine("[Telemetry] Nested task exception: {0}", ex);
						}
					}
				}
			}

			Console.WriteLine("[Telemetry] Telemetry flush complete. Succeeded: {0}, Failed: {1}", succeeded, failed);
		}

		/// <summary>
		/// Releases underlying singletons for the Telemetry helper.
		/// </summary>
		internal static void Dispose()
		{
			if (_serviceProvider != null)
			{
				_serviceProvider.DisposeAsync().AsTask().GetAwaiter().GetResult();
				_serviceProvider = null;

				// This is disposed as a part of the service provider.
				_hordeClient = null;
			}

			if (_hordeHttpClient != null)
			{
				_hordeHttpClient.Dispose();
				_hordeHttpClient = null;
			}
		}

		#region -- SumamryTableRowData --

		/// <summary>
		/// Posts a rowData data's SummaryTableMetric to the Horde event stream, asynchronously.
		/// </summary>
		/// <param name="rowData">The rowData data to post.</param>
		/// <param name="sessionLabel">Optional session label used to correlate telemetry events.</param>
		/// <param name="testIdentity">Optional identity used to group telemetry events.</param>
		/// <returns>True if telemetry was emitted successfully, false otherwise.</returns>
		internal async static Task<bool> PostSummaryTableMetricEventAsync(SummaryTableRowData rowData, string sessionLabel = null, string testIdentity = null)
		{
			if (HordeClient == null)
			{
				Console.WriteLine("[Telemetry] Skipping emission of rowData due to invalid HttpClient.");
				return false;
			}

			Hashtable telemetryPayload = null;

			try
			{
				telemetryPayload = CreatePayload(rowData, sessionLabel, testIdentity);
			}
			catch (InvalidOperationException ex)
			{
				Console.WriteLine("[Telemetry] Skipping emission of rowData due to invalid payload construction. Exception: {0}", ex.Message);
				return false;
			}

			return await EmitPayloadsAsync([telemetryPayload]);
		}

		/// <summary>
		/// Enqueues a task to posts a rowData data's SummaryTableMetric to the Horde event stream.
		/// </summary>
		/// <param name="rowData">The rowData data to post.</param>
		/// <param name="sessionLabel">Optional session label used to correlate telemetry events.</param>
		/// <param name="testIdentity">Optional identity used to group telemetry events.</param>
		/// <returns>A readonly list containing the queued tasks.</returns>
		internal static IReadOnlyList<Task<bool>> EnqueuePostSummaryTableMetricEvent(SummaryTableRowData rowData, string sessionLabel = null, string testIdentity = null)
		{
			if (HordeClient == null)
			{
				Console.WriteLine("[Telemetry] Skipping emission of rowData due to invalid HttpClient.");
				return _queuedRequests;
			}

			lock (_lock)
			{
				_queuedRequests.Add(PostSummaryTableMetricEventAsync(rowData, sessionLabel, testIdentity));
			}

			return _queuedRequests;
		}

		/// <summary>
		/// Posts a rowData data's SummaryTableMetric to the Horde event stream.
		/// </summary>
		/// <param name="rowData">The rowData data to post.</param>
		/// <param name="sessionLabel">Optional session label used to correlate telemetry events.</param>
		/// <param name="testIdentity">Optional identity used to group telemetry events.</param>
		/// <returns>True if telemetry was emitted successfully, false otherwise.</returns>
		internal static bool PostSummaryTableMetricEvent(SummaryTableRowData rowData, string sessionLabel = null, string testIdentity = null)
		{
			if (HordeClient == null)
			{
				Console.WriteLine("[Telemetry] Skipping emission of rowData due to invalid HttpClient.");
				return false;
			}

			Hashtable telemetryPayload = null;

			try
			{
				telemetryPayload = CreatePayload(rowData, sessionLabel, testIdentity);
			}
			catch (InvalidOperationException ex)
			{
				Console.WriteLine("[Telemetry] Skipping emission of rowData due to invalid payload construction. Exception: {0}", ex.Message);
				return false;
			}

			return EmitPayloadsAsync([telemetryPayload]).GetAwaiter().GetResult();
		}

		private static Hashtable CreatePayload(SummaryTableRowData rowData, string sessionLabel = null, string testIdentity = null)
		{
			Dictionary<string, dynamic> jsonDict = rowData.ToJsonDict(false, false);
			string metadataMetricKey = SummaryTableElement.Type.CsvMetadata.ToString();
			dynamic metadataMetric;

			string summaryTableMetricKey = SummaryTableElement.Type.SummaryTableMetric.ToString();
			dynamic summaryTableMetric;

			if (!jsonDict.TryGetValue(summaryTableMetricKey, out summaryTableMetric))
			{
				return null;
			}

			if (summaryTableMetric == null)
			{
				return null;
			}

			if (!jsonDict.TryGetValue(metadataMetricKey, out metadataMetric))
			{
				return null;
			}

			if (metadataMetric == null)
			{
				return null;
			}

			// Create resulting payload
			// - Coalesce the metric data into the telemetry payload
			// - Filter the telemetry payload of everything except the minimal metadata schema
			// - Coalesce the summary items into the telemetry payload
			// - Annotate the event as a global payload
			Hashtable telemetryPayload = [];

			CoalesceDictionary(metadataMetric, telemetryPayload);

			telemetryPayload = StandardTelemetryMetadata.CreateFilteredHashtable(telemetryPayload);

			CoalesceDictionary(summaryTableMetric, telemetryPayload);

			telemetryPayload = StandardTelemetryMetadata.SanitizeHashtable(telemetryPayload);

			if (!telemetryPayload.ContainsKey("TestIdentity") && !String.IsNullOrEmpty(testIdentity))
			{
				telemetryPayload["TestIdentity"] = testIdentity;
			}

			HordeContextTelemetryRecord telemetryRecord = new HordeContextTelemetryRecord($"{EventName}.Global", CurrentSchemaVersion, _sessionId, sessionLabel);
			telemetryPayload = AnalyticsTelemetryHelpers.MergeLeftCopy(telemetryPayload, telemetryRecord.ToHashtable());

			return telemetryPayload;
		}

		private static void CoalesceDictionary(dynamic summaryTableEntry, Hashtable telemetryPayload)
		{
			// Coalesce summary table metric values into telemetry payload via IDictionary || IDictionary<string,TValue> interfaces.
			if (summaryTableEntry is IDictionary)
			{
				foreach (dynamic kvp in summaryTableEntry)
				{
					telemetryPayload[kvp.Key] = kvp.Value;
				}
			}
			else if (summaryTableEntry is IDictionary<string, object> stringKeyGenericDict)
			{
				foreach (KeyValuePair<string, object> kvp in stringKeyGenericDict)
				{
					telemetryPayload[kvp.Key] = kvp.Value;
				}
			}
			else
			{
				throw new InvalidOperationException("[Telemetry] SummaryTable entry must be a dictionary or Dictionary<string, object>.");
			}
		}

		#endregion -- SummaryTableRowData --

		#region -- SummaryTable --

		/// <summary>
		/// Enqueues a task to posts an entire summary table to the Horde event stream.
		/// </summary>
		/// <param name="summaryTableName">The name of the table.</param>
		/// <param name="tableData">The table to post as an event.</param>
		/// <param name="isCollated">If the table represents collated data or not.</param>
		/// <param name="sessionLabel">Optional session label used to correlate telemetry events.</param>
		/// <param name="testIdentity">Optional identity used to group telemetry events.</param>
		/// <returns>A readonly list containing the queued tasks.</returns>
		internal static IReadOnlyList<Task<bool>> EnqueuePostSummaryTableMetricEvent(string summaryTableName, SummaryTable tableData, bool isCollated, string sessionLabel = null, string testIdentity = null)
		{
			if (String.IsNullOrEmpty(summaryTableName))
			{
				summaryTableName = DefaultSummaryTableName;
			}

			IList<Hashtable> emissionPayloads = new List<Hashtable>(16);
			string eventName = isCollated ? $"{EventName}.{summaryTableName}.Collated" : $"{EventName}.{summaryTableName}";
			HordeContextTelemetryRecord telemetryRecord = new HordeContextTelemetryRecord(eventName, CurrentSchemaVersion, _sessionId, sessionLabel);
			Hashtable baseContext = telemetryRecord.ToHashtable();

			foreach (IEnumerable<KeyValuePair<SummaryTableColumn, dynamic>> rowData in tableData.EnumerateTableDataByRow())
			{
				Hashtable telemetryPayload = [];

				foreach (KeyValuePair<SummaryTableColumn, dynamic> cell in rowData)
				{
					SummaryTableColumn col = cell.Key;
					dynamic val = cell.Value;

					telemetryPayload[col.name] = val;
				}

				telemetryPayload["SummaryName"] = summaryTableName;

				if (!telemetryPayload.ContainsKey("TestIdentity") && !String.IsNullOrEmpty(testIdentity))
				{
					telemetryPayload["TestIdentity"] = testIdentity;
				}

				telemetryPayload["Collated"] = isCollated;

				telemetryPayload = StandardTelemetryMetadata.SanitizeHashtable(telemetryPayload);
				telemetryPayload = AnalyticsTelemetryHelpers.MergeLeft(telemetryPayload, baseContext);

				emissionPayloads.Add(telemetryPayload);
			}

			lock (_lock)
			{
				_queuedRequests.Add(EmitPayloadsAsync(emissionPayloads.ToArray()));
			}

			return _queuedRequests;
		}

		#endregion -- SummaryTable

		#region -- Payload Emission --

		private static async Task<bool> EmitPayloadsAsync(Hashtable[] telemetryPayloads)
		{
			if (telemetryPayloads == null || telemetryPayloads.Length == 0)
			{
				return false;
			}

			try
			{
				using CancellationTokenSource cancellationToken = new CancellationTokenSource(TimeSpan.FromSeconds(DefaultTimeoutSeconds));
				List<string> missingItems = new List<string>();
				for (int i = 0; i < telemetryPayloads.Count(); ++i)
				{
					ColumnValidator<KeyStatTelemetryRecord>.HasCompletePropertySet(telemetryPayloads[i], true, null);
				}

				if (missingItems.Count > 0)
				{
					for (int i = 0; i < missingItems.Count; ++i)
					{
						Console.WriteLine("[Telemetry] Property was not found in telemetry payload: {0}", missingItems[i]);
					}
				}

				using HttpResponseMessage response = await HordeHttpClient.PostTelemetryAsync(DefaultAppId, DefaultAppVersion, DefaultAppEnvironment, telemetryPayloads, _sessionId, cancellationToken.Token);

				if (response == null)
				{
					return false;
				}

				bool isSuccessStatusCode = response.IsSuccessStatusCode;
				return isSuccessStatusCode;
			}
			catch (Exception ex)
			{
				Console.WriteLine("[Telemetry] Encountered an exception while posting event. Exception: {0}", ex.Message);
				return false;
			}
		}

		#endregion -- Payload Emission --
	}
}