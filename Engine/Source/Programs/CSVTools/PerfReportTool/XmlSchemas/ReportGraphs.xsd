<?xml version="1.0" encoding="utf-8"?>
<xs:schema attributeFormDefault="unqualified" elementFormDefault="qualified" xmlns:xs="http://www.w3.org/2001/XMLSchema" targetNamespace="https://www.unrealengine.com/PerfReportTool" xmlns="https://www.unrealengine.com/PerfReportTool">

	<xs:element name="baseSettings">
		<xs:complexType>
			<!-- Note: these settings need to be duplicated below -->
			<xs:attribute name="showAverages" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="showTotals" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="maxY" type="xs:string" use="optional" />
			<xs:attribute name="thickness" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="smooth" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="snapToPeaks" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="maxAutoMaxY" type="xs:unsignedShort" use="optional" />
			<xs:attribute name="height" type="xs:unsignedInt" use="optional" />
			<xs:attribute name="stacked" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="mainStat" type="xs:string" use="optional" />
			<xs:attribute name="compression" type="xs:decimal" use="optional" />
			<xs:attribute name="smoothKernelSize" type="xs:int" use="optional" />
			<xs:attribute name="smoothKernelPercent" type="xs:decimal" use="optional" />
			<xs:attribute name="hideStatPrefix" type="xs:string" use="optional" />
			<xs:attribute name="showEvents" type="xs:string" use="optional" />
			<xs:attribute name="stackTotalStat" type="xs:string" use="optional" />
			<xs:attribute name="ignoreStats" type="xs:string" use="optional" />
			<xs:attribute name="legendAverageThreshold" type="xs:string" use="optional" />
			<xs:attribute name="width" type="xs:unsignedInt" use="optional" />
			<xs:attribute name="maxHierarchyDepth" type="xs:byte" use="optional" />
			<xs:attribute name="threshold" type="xs:string" use="optional" />
			<xs:attribute name="filterOutZeros" type="xs:unsignedByte" use="optional" />
			<xs:attribute name="startEvent" type="xs:string" use="optional" />
			<xs:attribute name="startEventOffset" type="xs:string" use="optional" />
			<xs:attribute name="endEvent" type="xs:string" use="optional" />
			<xs:attribute name="endEventOffset" type="xs:string" use="optional" />
			<xs:attribute name="requiredEvents" type="xs:string" use="optional" />
			<xs:attribute name="minFilterStatName" type="xs:string" use="optional" />
			<xs:attribute name="minFilterStatValue" type="xs:string" use="optional" />
			<xs:attribute name="budget" type="xs:string" use="optional" />
			<xs:attribute name="statMultiplier" type="xs:string" use="optional" />
			<xs:attribute name="minX" type="xs:unsignedInt" use="optional" />
			<xs:attribute name="requiresDetailedStats" type="xs:unsignedByte" use="optional" />
		</xs:complexType>
	</xs:element>	
	
	<xs:element name="graphGroups">
		<xs:complexType>
			<xs:sequence>
				<xs:element ref="baseSettings" minOccurs="0" maxOccurs="1"/>
				<xs:element maxOccurs="unbounded" name="graphGroup">
					<xs:complexType>
						<xs:sequence>
							<xs:element ref="baseSettings" minOccurs="0" maxOccurs="1"/>
							<xs:element name="graph" maxOccurs="unbounded">
								<xs:complexType>
									<xs:sequence minOccurs="0">
										<xs:element name="statString" type="xs:string" />
									</xs:sequence>
									<!-- Note: this is just a duplication of graphSettings above -->
									<xs:attribute name="title" type="xs:string" use="required" />
									<xs:attribute name="showAverages" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="showTotals" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="maxY" type="xs:string" use="optional" />
									<xs:attribute name="thickness" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="smooth" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="snapToPeaks" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="maxAutoMaxY" type="xs:unsignedShort" use="optional" />
									<xs:attribute name="height" type="xs:unsignedInt" use="optional" />
									<xs:attribute name="stacked" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="mainStat" type="xs:string" use="optional" />
									<xs:attribute name="compression" type="xs:decimal" use="optional" />
									<xs:attribute name="smoothKernelSize" type="xs:int" use="optional" />
									<xs:attribute name="smoothKernelPercent" type="xs:decimal" use="optional" />
									<xs:attribute name="hideStatPrefix" type="xs:string" use="optional" />
									<xs:attribute name="showEvents" type="xs:string" use="optional" />
									<xs:attribute name="stackTotalStat" type="xs:string" use="optional" />
									<xs:attribute name="ignoreStats" type="xs:string" use="optional" />
									<xs:attribute name="legendAverageThreshold" type="xs:string" use="optional" />
									<xs:attribute name="width" type="xs:unsignedInt" use="optional" />
									<xs:attribute name="maxHierarchyDepth" type="xs:byte" use="optional" />
									<xs:attribute name="threshold" type="xs:string" use="optional" />
									<xs:attribute name="filterOutZeros" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="startEvent" type="xs:string" use="optional" />
									<xs:attribute name="startEventOffset" type="xs:string" use="optional" />
									<xs:attribute name="endEvent" type="xs:string" use="optional" />
									<xs:attribute name="endEventOffset" type="xs:string" use="optional" />
									<xs:attribute name="requiredEvents" type="xs:string" use="optional" />
									<xs:attribute name="minFilterStatName" type="xs:string" use="optional" />
									<xs:attribute name="minFilterStatValue" type="xs:string" use="optional" />
									<xs:attribute name="budget" type="xs:string" use="optional" />
									<xs:attribute name="statMultiplier" type="xs:string" use="optional" />
									<xs:attribute name="minX" type="xs:unsignedInt" use="optional" />
									<xs:attribute name="requiresDetailedStats" type="xs:unsignedByte" use="optional" />									
								</xs:complexType>
							</xs:element>
						</xs:sequence>
						<xs:attribute name="name" type="xs:string" use="required" />
					</xs:complexType>
				</xs:element>
			</xs:sequence>
		</xs:complexType>
	</xs:element>
</xs:schema>