<?xml version="1.0" encoding="utf-8"?>
<xs:schema attributeFormDefault="unqualified" elementFormDefault="qualified" xmlns:xs="http://www.w3.org/2001/XMLSchema" targetNamespace="https://www.unrealengine.com/PerfReportTool" xmlns="https://www.unrealengine.com/PerfReportTool">

	<!-- Note: we use type="xs:string" for all types here because otherwise matching variables doesn't work -->
	<!-- TODO: Schemas for summaries are partially auto-generated and not aware of the summary type -->
	<!--       We could potentially make each summary a separate element name to fix this but it would require deprecation of the old format -->
	<!--       so as not to break licensees -->
	<xs:element name="variableSet">
		<xs:complexType>
			<xs:choice maxOccurs="unbounded">
				<xs:element minOccurs="0" maxOccurs="unbounded" name="var">
					<xs:complexType>
						<xs:simpleContent>
							<xs:extension base="xs:string">
								<xs:attribute name="name" type="xs:string" use="required" />
								<xs:attribute name="multiplier" type="xs:string" use="optional" />
							</xs:extension>
						</xs:simpleContent>
					</xs:complexType>
				</xs:element>
				<xs:element ref="variableSet"/>
			</xs:choice>
			<xs:attribute name="metadataQuery" type="xs:string" use="optional" />
			<xs:attribute name="multiplier" type="xs:string" use="optional" />
		</xs:complexType>
	</xs:element>

	<xs:complexType name="summary">
		<xs:attribute name="beginevent" type="xs:string" use="optional" />
		<xs:attribute name="collapsed" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="destImage" type="xs:string" use="optional" />
		<xs:attribute name="endevent" type="xs:string" use="optional" />
		<xs:attribute name="endoffsetpercent" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="fps" type="xs:string" use="optional" />
		<xs:attribute name="framesPerLineSegment" type="xs:unsignedInt" use="optional" />
		<xs:attribute name="height" type="xs:unsignedInt" use="optional" />
		<xs:attribute name="hideInDetailedReport" type="xs:string" use="optional" />
		<xs:attribute name="hideStatPrefix" type="xs:string" use="optional" />
		<xs:attribute name="hitchThreshold" type="xs:string" use="optional" />
		<xs:attribute name="ignoreHitchTimePercent" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="ignoreMVP" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="lineSplitDistanceThreshold" type="xs:unsignedShort" use="optional" />
		<xs:attribute name="metricName" type="xs:string" use="optional" />
		<xs:attribute name="metricPrefix" type="xs:string" use="optional" />
		<xs:attribute name="offsetX" type="xs:string" use="optional" />
		<xs:attribute name="offsetY" type="xs:string" use="optional" />
		<xs:attribute name="scale" type="xs:string" use="optional" />
		<xs:attribute name="sourceImage" type="xs:string" use="optional" />
		<xs:attribute name="stripStatPrefix" type="xs:string" use="optional" />
		<xs:attribute name="summaryStatName" type="xs:string" use="optional" />
		<xs:attribute name="sortByName" type="xs:string" use="optional" />
		<xs:attribute name="suppressAveragesTable" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="title" type="xs:string" use="optional" />
		<xs:attribute name="transform" type="xs:string" use="optional" />
		<xs:attribute name="type" type="xs:string" use="required" />
		<xs:attribute name="useEngineHitchMetric" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="useUnstrippedCsvStats" type="xs:unsignedByte" use="optional" />
		<xs:attribute name="width" type="xs:unsignedInt" use="optional" />
		<xs:choice minOccurs="0" maxOccurs="unbounded">
			<xs:element name="colourThresholdsTotal" type="xs:string" />
			<xs:element name="stats" type="xs:string" />
			<xs:element name="histogramThresholds" type="xs:string" />
			<xs:element maxOccurs="unbounded" name="colourThresholds">
				<xs:complexType>
					<xs:simpleContent>
						<xs:extension base="xs:string">
							<xs:attribute name="stat" type="xs:string" use="optional" />
						</xs:extension>
					</xs:simpleContent>
				</xs:complexType>
			</xs:element>
			<xs:element name="hitchThresholds" type="xs:string" />
			<xs:element maxOccurs="unbounded" name="capture">
				<xs:complexType>
					<xs:attribute name="name" type="xs:string" use="required" />
					<xs:attribute name="startEvent" type="xs:string" use="required" />
					<xs:attribute name="endEvent" type="xs:string" use="required" />
					<xs:attribute name="includeFirstFrame" type="xs:string" use="optional" />
					<xs:attribute name="includeLastFrame" type="xs:string" use="optional" />
				</xs:complexType>
			</xs:element>
			<xs:element maxOccurs="unbounded" name="summarySection">
				<xs:complexType>
					<xs:sequence>
						<xs:element name="statFilter" type="xs:string" />
					</xs:sequence>
					<xs:attribute name="title" type="xs:string" use="required" />
				</xs:complexType>
			</xs:element>
			<xs:element name="colorDisplay">
				<xs:complexType>
					<xs:attribute name="lowEndColor" type="xs:string" use="required" />
					<xs:attribute name="highEndColor" type="xs:string" use="required" />
				</xs:complexType>
			</xs:element>
			<xs:element name="buckets">
				<xs:complexType>
					<xs:simpleContent>
						<xs:extension base="xs:string">
							<xs:attribute name="reportAboveRange" type="xs:unsignedByte" use="optional" />
							<xs:attribute name="reportBelowRange" type="xs:unsignedByte" use="optional" />
							<xs:attribute name="precision" type="xs:string" use="optional" />
						</xs:extension>
					</xs:simpleContent>
				</xs:complexType>
			</xs:element>
			<xs:element minOccurs="0" maxOccurs="unbounded" name="overlay">
				<xs:complexType>
					<xs:sequence>
						<xs:element maxOccurs="unbounded" name="event">
							<xs:complexType>
								<xs:attribute name="name" type="xs:string" use="required" />
								<xs:attribute name="shortName" type="xs:string" use="optional" />
								<xs:attribute name="lineColor" type="xs:string" use="optional" />
								<xs:attribute name="isStartEvent" type="xs:unsignedByte" use="optional" />
								<xs:attribute name="summaryStatName" type="xs:string" use="optional" />
							</xs:complexType>
						</xs:element>
					</xs:sequence>
					<xs:attribute name="xStat" type="xs:string" use="required" />
					<xs:attribute name="yStat" type="xs:string" use="required" />
					<xs:attribute name="zStat" type="xs:string" use="required" />
					<xs:attribute name="lineColor" type="xs:string" use="required" />
					<xs:attribute name="summaryStatNamePrefix" type="xs:string" use="optional" />
				</xs:complexType>
			</xs:element>
			<xs:element minOccurs="0" name="events" type="xs:string" />
			<xs:element minOccurs="0" maxOccurs="unbounded" name="column">
				<xs:complexType>
					<xs:sequence minOccurs="0">
						<xs:element name="colourThresholds" type="xs:string" />
					</xs:sequence>
					<xs:attribute name="name" type="xs:string" use="required" />
					<xs:attribute name="formula" type="xs:string" use="required" />
					<xs:attribute name="multiplier" type="xs:string" use="optional" />
					<xs:attribute name="summaryStatName" type="xs:string" use="optional" />
					<xs:attribute name="stat" type="xs:string" use="required" />
					<xs:attribute name="applyEndOffset" type="xs:unsignedByte" use="optional" />
					<xs:attribute name="threshold" type="xs:unsignedByte" use="optional" />
					<xs:attribute name="filterOutZeros" type="xs:unsignedByte" use="optional" />
					<xs:attribute name="perSecond" type="xs:unsignedByte" use="optional" />
					<xs:attribute name="otherStat" type="xs:string" use="optional" />
				</xs:complexType>
			</xs:element>
			<xs:element name="sort">
				<xs:complexType>
					<xs:attribute name="mode" type="xs:string" use="optional" />
					<xs:attribute name="bucketIndex" type="xs:unsignedByte" use="optional" />
					<xs:attribute name="sortDirection" type="xs:string" use="optional" />
				</xs:complexType>
			</xs:element>
			<xs:element maxOccurs="unbounded" name="eventCapture">
				<xs:complexType>
					<xs:attribute name="friendlyName" type="xs:string" use="required" />
					<xs:attribute name="eventString" type="xs:string" use="required" />
					<xs:attribute name="metricSuffix" type="xs:string" use="optional" />
				</xs:complexType>
			</xs:element>
		</xs:choice>
	</xs:complexType>


	<xs:element name="root">
		<xs:complexType>
			<xs:all>
				<xs:element name="statDisplayNameMappings" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="mapping">
								<xs:complexType>
									<xs:attribute name="statName" type="xs:string" use="required" />
									<xs:attribute name="displayName" type="xs:string" use="required" />
									<xs:attribute name="toolTip" type="xs:string" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="csvMetadataProxies" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="csvMetadata">
								<xs:complexType>
									<xs:simpleContent>
										<xs:extension base="xs:string">
											<xs:attribute name="name" type="xs:string" use="required" />
										</xs:extension>
									</xs:simpleContent>
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="derivedMetadataMappings" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="mapping">
								<xs:complexType>
									<xs:attribute name="destName" type="xs:string" use="required" />
									<xs:attribute name="destValue" type="xs:string" use="required" />
									<xs:attribute name="metadataQuery" type="xs:string" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="derivedCsvStats" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="derivedCsvStat">
								<xs:complexType>
									<xs:attribute name="name" type="xs:string" use="required" />
									<xs:attribute name="expression" type="xs:string" use="required" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="csvEventsToStrip" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="eventPair">
								<xs:complexType>
									<xs:attribute name="begin" type="xs:string" use="optional" />
									<xs:attribute name="end" type="xs:string" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<!-- This is entirely auto-generated. It may be too specific -->
				<xs:element name="sharedSummaries" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element name="summary" maxOccurs="unbounded">
								<xs:complexType>
									<xs:complexContent>
										<xs:extension base="summary">
											<xs:attribute name="refName" type="xs:string" use="required" />
										</xs:extension>
									</xs:complexContent>
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="summaryTableColumnFormatInfo" minOccurs="0" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="columnInfo">
								<xs:complexType>
									<xs:sequence minOccurs="0">
										<xs:element name="colourThresholds">
											<xs:complexType>
												<xs:sequence>
													<xs:element name="thresholds" type="xs:string" />
													<xs:element name="colours" type="xs:string" />
												</xs:sequence>
												<xs:attribute name="lerpColours" type="xs:string" use="optional" />
											</xs:complexType>
										</xs:element>
									</xs:sequence>
									<xs:attribute name="name" type="xs:string" use="required" />
									<xs:attribute name="autoColorize" type="xs:string" use="optional" />
									<xs:attribute name="numericFormat" type="xs:string" use="optional" />
									<xs:attribute name="maxStringLengthCollated" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="valueBucketNames" type="xs:string" use="optional" />
									<xs:attribute name="valueBucketThresholds" type="xs:string" use="optional" />
									<xs:attribute name="includeValueWithBucketName" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="noWrap" type="xs:string" use="optional" />
									<xs:attribute name="dateFormat" type="xs:string" use="optional" />
									<xs:attribute name="dateTimeZoneId" type="xs:string" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="summaryTables" minOccurs="1" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="summaryTable">
								<xs:complexType>
									<xs:sequence>
										<xs:element minOccurs="0" maxOccurs="unbounded" name="sectionBoundary">
											<xs:complexType>
												<xs:attribute name="statName" type="xs:string" use="required" />
												<xs:attribute name="startToken" type="xs:string" use="optional" />
												<xs:attribute name="endToken" type="xs:string" use="optional" />
												<xs:attribute name="inCollatedTable" type="xs:unsignedByte" use="optional" />
												<xs:attribute name="level" type="xs:unsignedByte" use="optional" />
											</xs:complexType>
										</xs:element>
										<xs:element name="filter" type="xs:string" />
									</xs:sequence>
									<xs:attribute name="name" type="xs:string" use="required" />
									<xs:attribute name="rowSort" type="xs:string" use="optional" />
									<xs:attribute name="weightByColumn" type="xs:string" use="optional" />
									<xs:attribute name="collateAverageMethod" type="xs:string" use="optional" />
									<xs:attribute name="scrollableFormatting" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="colorizeMode" type="xs:string" use="optional" />
									<xs:attribute name="columnSortMode" type="xs:string" use="optional" />
									<xs:attribute name="collatedDateVisibility" type="xs:string" use="optional" />
									<xs:attribute name="collatedStringVisibility" type="xs:string" use="optional" />
									<xs:attribute name="reverseSortRows" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="minColumnHeaderWidth" type="xs:string" use="optional" />
									<xs:attribute name="maxColumnHeaderWidth" type="xs:string" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="globalVariableSet" minOccurs="0" maxOccurs="1">
					<!-- FIXME: this needs to be made less rigid and recursive -->
					<xs:complexType>
						<xs:sequence>
							<xs:element ref="variableSet" maxOccurs="unbounded"/>
						</xs:sequence>
					</xs:complexType>
				</xs:element>
				
				<xs:element name="reporttypes" minOccurs="1" maxOccurs="1">
					<xs:complexType>
						<xs:sequence>
							<xs:element maxOccurs="unbounded" name="reporttype">
								<xs:complexType>
									<xs:sequence>
										<xs:choice maxOccurs="unbounded">
											<xs:element name="autodetection">
												<xs:complexType>
													<xs:sequence minOccurs="0">
														<xs:element maxOccurs="unbounded" name="requiredmetadata">
															<xs:complexType>
																<xs:attribute name="key" type="xs:string" use="required" />
																<xs:attribute name="allowedValues" type="xs:string" use="required" />
																<xs:attribute name="stopIfKeyFound" type="xs:unsignedByte" use="optional" />
																<xs:attribute name="ignoreIfKeyNotFound" type="xs:unsignedByte" use="optional" />
																<xs:attribute name="ignoreForDefaultFallback" type="xs:unsignedByte" use="optional" />
															</xs:complexType>
														</xs:element>
													</xs:sequence>
													<xs:attribute name="requiredstats" type="xs:string" use="optional" />
												</xs:complexType>
											</xs:element>
											<xs:element name="metadataToShow" type="xs:string" />
											<!-- This is entirely auto-generated. It may be too specific -->
											<xs:element name="summary" type="summary"/>
											<xs:element maxOccurs="unbounded" name="summaryRef">
												<xs:complexType>
													<xs:attribute name="name" type="xs:string" use="required" />
												</xs:complexType>
											</xs:element>
											<xs:element maxOccurs="unbounded" name="graph">
												<xs:complexType>
													<xs:sequence minOccurs="0">
														<xs:element name="statString" type="xs:string" />
													</xs:sequence>
													<xs:attribute name="title" type="xs:string" use="required" />
													<xs:attribute name="inSummary" type="xs:unsignedByte" use="optional" />
													<xs:attribute name="budget" type="xs:string" use="optional" />
													<xs:attribute name="inline" type="xs:unsignedByte" use="optional" />
													<xs:attribute name="parent" type="xs:string" use="optional" />
													<xs:attribute name="maxy" type="xs:string" use="optional" />
													<xs:attribute name="minFilterStatValue" type="xs:unsignedByte" use="optional" />
													<xs:attribute name="stacked" type="xs:unsignedByte" use="optional" />
													<xs:attribute name="mainStat" type="xs:string" use="optional" />
													<xs:attribute name="thickness" type="xs:unsignedByte" use="optional" />
													<xs:attribute name="smooth" type="xs:unsignedByte" use="optional" />
												</xs:complexType>
											</xs:element>
											<xs:element ref="variableSet" minOccurs="0" maxOccurs="1"/>
										</xs:choice>
									</xs:sequence>
									<xs:attribute name="name" type="xs:string" use="required" />
									<xs:attribute name="title" type="xs:string" use="required" />
									<xs:attribute name="ignoreList" type="xs:string" use="optional" />
									<xs:attribute name="stripEvents" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="vsync" type="xs:unsignedByte" use="optional" />
									<xs:attribute name="allowAutoDetect" type="xs:unsignedByte" use="optional" />
								</xs:complexType>
							</xs:element>
						</xs:sequence>
						<xs:attribute name="reportGraphsFile" type="xs:string" use="optional" />
						<xs:attribute name="default" type="xs:string" use="optional" />
					</xs:complexType>
				</xs:element>
			</xs:all>
		</xs:complexType>
	</xs:element>
</xs:schema>