@echo off

rem ## Unreal Engine UnrealBuildTool build script
rem ## Copyright Epic Games, Inc. All Rights Reserved.

setlocal

pushd "%~dp0

rem ## Verify that dotnet is present
call "%~dp0..\..\..\Build\BatchFiles\GetDotnetPath.bat"
if errorlevel 1 goto Error_NoDotnetSDK

rem ## Command line arguments
set MSBUILD_LOGLEVEL=%1
if not defined %MSBUILD_LOGLEVEL set MSBUILD_LOGLEVEL=quiet

echo Building CsvTools...
dotnet build CsvTools.sln -c Release -v %MSBUILD_LOGLEVEL%
if errorlevel 1 goto Error_CompileFailed

echo Publishing CsvTools...
if exist ..\..\..\Binaries\DotNET\CsvTools rmdir /s /q ..\..\..\Binaries\DotNET\CsvTools >nul 2>&1
if not exist ..\..\..\Binaries\DotNET\CsvTools mkdir ..\..\..\Binaries\DotNET\CsvTools >nul 2>&1
dotnet publish CSVCollate\CSVCollate.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CsvConvert\CsvConvert.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CSVFilter\CSVFilter.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CsvInfo\CsvInfo.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CSVSplit\CSVSplit.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CsvStats\CsvStats.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CSVToSVG\CSVToSVG.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish CsvToSvgLib\CsvToSvgLib.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish PerfReportTool\PerfreportTool.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%
dotnet publish RegressionsReport\RegressionsReport.csproj -c Release --output ..\..\..\Binaries\DotNET\CsvTools --no-build -v %MSBUILD_LOGLEVEL%

rem ## Success!
set PUBLISH_EXITCODE=0
goto Exit

:Error_NoDotnetSDK
echo.
echo Publish ERROR: Unable to find a install of Dotnet SDK.  Please make sure you have it installed and that `dotnet` is a globally available command.
echo.
set PUBLISH_EXITCODE=1
goto Exit

:Error_CompileFailed
echo.
echo Publish ERROR: CsvTools failed to compile.
echo.
set PUBLISH_EXITCODE=1
goto Exit

:Exit
rem ## Restore original CWD in case we change it
popd
exit /B %PUBLISH_EXITCODE%
