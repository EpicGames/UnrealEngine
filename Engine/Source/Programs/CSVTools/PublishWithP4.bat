@echo off
setlocal enabledelayedexpansion

call .\Publish.bat
if errorlevel 1 goto publishError

set "BinariesDir=%~dp0..\..\..\Binaries\DotNET\CsvTools"
echo Binaries directory: %BinariesDir%

if not exist "%BinariesDir%" (
    echo CsvTools binaries folder not found:
    echo %BinariesDir%
    goto exit
)

pushd "%~dp0\..\..\..\Binaries\DotNET\CsvTools

for /f "tokens=1" %%A in ('p4 where . ^| findstr /c:"//"') do (
    set p4Path=%%A
)
echo P4 Path: !p4Path!

p4 reconcile !p4Path!/...
if errorlevel 1 (
	echo Error with p4 reconcile
	goto exit
)

p4 --field "Description=CSV Tools Binaries" --field "Files=" change -o | p4 change -i > tempP4Changeoutput.txt

REM Read the CL
For /F "delims=" %%L IN (tempP4Changeoutput.txt) DO (
  set "line=%%L"
  set "change=!line:Change =!"
  set "change=!change: created.=!"
)
echo Creating new CL: !change!
del tempP4ChangeOutput.txt

p4 reopen -c !change! !p4Path!/...
if errorlevel 1 (
	echo Error with p4 reopen
	goto exit
)

echo Success

:exit
popd
:publishError
EndLocal

