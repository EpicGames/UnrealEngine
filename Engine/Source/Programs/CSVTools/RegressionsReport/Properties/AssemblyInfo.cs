// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
using System.Runtime.InteropServices;

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("06104e7c-f154-4864-af94-996d8bba6319")]