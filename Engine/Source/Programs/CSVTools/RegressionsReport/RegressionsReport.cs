// Copyright (C) Microsoft. All rights reserved.
// Copyright Epic Games, Inc. All Rights Reserved.
 
using System;
using System.Collections.Generic;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Text;
using System.Diagnostics;
using System.Collections;
using System.Text.Json;
using System.Data;
 
using CSVStats;
 
namespace RegressionsReport
{
    class Version
    {
		// Major.Minor.Bugfix
        private static string VersionString = "1.1.0";
 
        public static string Get() { return VersionString; }
    };
 
	class Program : CommandLineTool
	{
    static string defaultBaseHTML = @"
        <!DOCTYPE html>
        <html>
        <head>
            <title>Regressions Report</title>
            <style>
                body { font-family: Arial, sans-serif; line-height: 1.6; margin: 0; padding: 0; background-color: #f4f4f4; }
                .container { width: 80%; margin: 20px auto; padding: 20px; background: #fff; box-shadow: 0 0 10px rgba(0, 0, 0, 0.1); border-radius: 10px; }
                h1 { color: #333; }
                h2 { color: #444; border-bottom: 2px solid #ddd; padding-bottom: 5px; }
                p { color: #555; }
                .highlight { font-weight: bold; color: #e74c3c; }
                .details { cursor: pointer; color: #3498db; text-decoration: underline; margin-top: 20px; display: block; }
                .hidden { display: none; }
                ul { list-style-type: none; padding-left: 0; }
                li { margin-bottom: 10px; padding: 10px; background: #f9f9f9; border: 1px solid #eee; border-radius: 5px; }
                hr { border: 1px solid #eee; margin: 20px 0; }
            </style>
            <script>
                function toggleDetails(id) {
                    var x = document.getElementById(id);
                    if (x.classList.contains('hidden')) {
                        x.classList.remove('hidden');
                    } else {
                        x.classList.add('hidden');
                    }
                }
            </script>
        </head>
        <body>
            <div class='container'>
                <h1>Regressions Report</h1>
                <div class='highlights'>
                </div>
            </div>
        </body>
        </html>";
 
		static string toolInformation =
			"RegressionsReport v" + Version.Get() + "\n" +
			"Required Arguments:\n" +
            "  -csvFile <filename> - Path to input CSV file\n" + 
			"  -o <directory> - Path to output directory\n" +
            "  -thresholds <filename> - Path to JSON file containing thresholds\n" +
			"Optional Arguments:\n" +
            "  -filename <name> - Will set output filename (Default: \"highlights.html\")" +
            "  -insertAfterTag <string> - Insert the report after first <div class=\"string\"> tag in base HTML file (Default: \"highlights\")\n" +
			"  -base <filename> - Will set base HTML file for report\n" +
            "  -dumpContents <filename> - Will dump the summary contents to a JSON file\n" +
            "  -testName <name> - Will set the name of the test\n" +
			"";
 
        // Run the main tool
		void Run(string[] args)
		{
            ReadCommandLine(args);
 
            // Check for help flag
            if(GetBoolArg("help")) {
				WriteLine(toolInformation);
				return;
            }
 
            // Check for invalid usage
            bool requiredArguments = !GetBoolArg("csvFile") || !GetBoolArg("o") || !GetBoolArg("thresholds");
            if (requiredArguments)
			{
				WriteLine("Missing required arguments, see -help flag");
				WriteLine(toolInformation);
				return;
			}
 
            // Run tool
            WriteLine("RegressionsReport v" + Version.Get());
 
            // Set flag arguments
            string inputFile = GetArg("csvFile");
            string outputPath = GetArg("o");
            string thresholdsFile  = GetArg("thresholds");
            string outputName = GetArg("filename", "highlights.html");
            string insertAfterTag = GetArg("insertAfterTag", "highlights");
            string baseFile = GetArg("base", defaultBaseHTML);
            string dumpContentsFile = GetArg("dumpContents", "");
            string testName = GetArg("testName", "PerfTests");
 
            // Read CSV file into DataTable
            DataTable dataTable = ReadCsv(inputFile);
 
            // Need at least two columns to compare
            // TODO: Just show something in the report to signify this instead of throwing error
            if(dataTable.Rows.Count < 2) {
                WriteLine("Need at least two columns in the table");
                return;
            }
 
            // Read thresholds data from JSON file
            var thresholds = JsonSerializer.Deserialize<Dictionary<string, Dictionary<string, double>>>(File.ReadAllText(thresholdsFile));
        
            // Compute changes between newest CLs
            DataRow previousRow = dataTable.Rows[0];
            DataRow newRow = dataTable.Rows[1];
            var percentageChanges = ComputePercentageChanges(newRow, previousRow, dataTable.Columns, thresholds);
 
            // Identify regressions
            var significantRegressions = new Dictionary<string, double>();
            var minorRegressions = new Dictionary<string, double>();

            foreach (var change in percentageChanges)
            {
                double significantThreshold = (double) thresholds["significant"][change.Key];
                double minorThreshold = (double) thresholds["minor"][change.Key];
 
                if (Math.Abs(change.Value) > significantThreshold)
                {
                    significantRegressions[change.Key] = change.Value;
                }
                else if (Math.Abs(change.Value) > minorThreshold)
                {
                    minorRegressions[change.Key] = change.Value;
                }
            }
 
            // Generate regression message
            string regressionMessage;
            if(significantRegressions.Any()) {
                regressionMessage = "This CL showed significant regressions.";
            } else if (minorRegressions.Any()) {
                regressionMessage = "This CL showed minor regressions.";
            } else {
                regressionMessage =  "This CL did not have any meaningful regressions.";
            }
 
            // Generate report lines
            var significantLines = GenerateRegressionLines(significantRegressions, percentageChanges, previousRow, newRow, "significant");
            var minorLines = GenerateRegressionLines(minorRegressions, percentageChanges, previousRow, newRow, "minor");
 
            // Generate HTML report content
            string reportContent = GenerateReportContent(newRow, regressionMessage, significantLines, minorLines, percentageChanges);
 
            // Write to HTML file
            WriteHtmlReport(outputPath, outputName, baseFile, insertAfterTag, reportContent);

            // Dump contents to JSON file if flag is present
            if (!string.IsNullOrEmpty(dumpContentsFile))
            {
                DumpContentsToJson(dumpContentsFile, previousRow, newRow, significantRegressions, percentageChanges, testName);
            }
        }
 
        // Creates a DataTable from reading in a CSV file
        static DataTable ReadCsv(string filePath)
        {
            DataTable dataTable = new DataTable();
            using (var reader = new StreamReader(filePath))
            {
                bool isFirstRow = true;
                while (!reader.EndOfStream)
                {
                    var line = reader.ReadLine();
                    var values = line.Split(',');
 
                    // Check if its the first row of the CSV
                    if (isFirstRow)
                    {
                        // Add each column name to the DataTable
                        foreach (var column in values)
                        {
                            dataTable.Columns.Add(column);
                        }
                        isFirstRow = false;
                    }
                    else
                    {
                        // Add to row to DataTable
                        dataTable.Rows.Add(values);
                    }
                }
            }
 
            return dataTable;
        }
 
        // Compute perctange change between two rows in the DataTable based on some threshold values
        static Dictionary<string, double> ComputePercentageChanges(DataRow newRow, DataRow oldRow, DataColumnCollection columns, Dictionary<string, Dictionary<string, double>> thresholds)
        {
            // Calculate perctange change in specified columns
            var percentageChanges = new Dictionary<string, double>();
 
            for (int i = 0; i < columns.Count; i++)
            {
                string columnName = columns[i].ColumnName;
 
                // Only compute change between columns entries specified within the JSON file
                if(thresholds["significant"].ContainsKey(columnName) || thresholds["minor"].ContainsKey(columnName)) {
                    double newValue = Convert.ToDouble(newRow[columnName]);
                    double oldValue = Convert.ToDouble(oldRow[columnName]);
 
                    if (oldValue != 0)
                    {
                        double percentageChange = ((newValue - oldValue) / oldValue) * 100;
                        percentageChanges[columnName] = percentageChange;
                    }
                    else
                    {
                        percentageChanges[columnName] = double.PositiveInfinity;
                    }
                }
 
            }
 
            return percentageChanges;
        }

        // Generates output lines for each regression that has occurred
        static List<string> GenerateRegressionLines(Dictionary<string, double> regressions, Dictionary<string, double> changes, DataRow oldRow, DataRow newRow, string category)
        {
            return regressions.Select(reg => 
                $"{category} regression in terms of {reg.Key} - Change of {changes[reg.Key]:F2}%, from {oldRow[reg.Key]} to {newRow[reg.Key]}"
            ).ToList();
        }
 
        // Generates the new resulting content for the HTML report
        static string GenerateReportContent(DataRow newRow, string message, List<string> significantLines, List<string> minorLines, Dictionary<string, double> percentageChanges)
        {
            var significantHtml = string.Join("", significantLines.Select(line => $"<li>{line}</li>"));
            var minorHtml = string.Join("", minorLines.Select(line => $"<li>{line}</li>"));
            var detailsHtml = string.Join("", percentageChanges.Select(change => 
                $"<li>{change.Key} changed by {change.Value:F2}% to {newRow[change.Key]}</li>"
            ));
 
            return $@"
                <div class='report'>
                    <h1>Report on <span class='highlight'>{newRow["buildversion"]}</span></h1>
                    <p class='highlight'>{message}</p>
                    <div>
                        <h2>Significant Regressions</h2>
                        <ul>{significantHtml}</ul>
                    </div>
                    <div>
                        <h2>Minor Regressions</h2>
                        <ul>{minorHtml}</ul>
                    </div>
                    <span class='details' onclick='toggleDetails(""regression-details-{newRow["buildversion"]}"")'>
                        &#x25BC; Click to see all details
                    </span>
                    <div id='regression-details-{newRow["buildversion"]}' class='hidden'>
                        <ul>{detailsHtml}</ul>
                    </div>
                </div>
                <hr>";
        }
 
        // Writes the new new report HTML report
        static void WriteHtmlReport(string outputPath, string outputName, string baseHtml, string insertAfterTag, string reportContent)
        {
            string outputFilePath = Path.Combine(outputPath, outputName);
            string content;
 
            // Append if the file exists, otherwise use the base template
            if (File.Exists(outputFilePath))
            {
                content = File.ReadAllText(outputFilePath);
            }
            else
            {
                content = baseHtml;
            }
 
            // Append the report after the first instance of the specified tag
            string insertTag = $"<div class='{insertAfterTag}'>";
            int insertIndex = content.IndexOf(insertTag) + insertTag.Length;
 
            content = content.Insert(insertIndex, reportContent);
            File.WriteAllText(outputFilePath, content);
        }

        // Creates a JSON file with a content summary of the regression report it generated
        static void DumpContentsToJson(string filePath, DataRow oldRow, DataRow newRow, Dictionary<string, double> significantRegressions, Dictionary<string, double> percentageChanges, string testName)
        {
            // Store regression information in a dictionary
            var regressions = new Dictionary<string, object>();

            foreach (var regression in significantRegressions)
            {
                string stat = regression.Key;
                regressions[stat] = new
                {
                    percentage_change = $"{percentageChanges[stat]:F2}%",
                    original_value = oldRow[stat].ToString(),
                    new_value = newRow[stat].ToString()
                };
            }

            // Create new JSON object
            var jsonObject = new
            {
                had_regression = significantRegressions.Count != 0,
                commit = $"CL {newRow["buildversion"]}",
                test = testName,
                html_path = filePath.Replace(".json", ".html"),
                regressions = regressions
            };

            // Write JSON object to path provided
            string jsonString = JsonSerializer.Serialize(jsonObject, new JsonSerializerOptions { WriteIndented = true });
            File.WriteAllText(filePath, jsonString);
        }
 
        static int Main(string[] args)
        {
            Program program = new Program();
 
            if (Debugger.IsAttached)
            {
                program.Run(args);
            }
            else
            {
                try
                {
                    program.Run(args);
                }
                catch (System.Exception e)
                {
                    Console.WriteLine("[ERROR] " + e.Message);
                    return 1;
                }
            }
 
            return 0;
        }
    }
}