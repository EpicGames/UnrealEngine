<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AssemblyTitle>RegressionsReport</AssemblyTitle>
    <Product>RegressionsReport</Product>
  </PropertyGroup>
  <ItemGroup>
    <None Remove="Properties\AssemblyInfo.cs~RF273305d.TMP" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\CsvStats\CsvStats.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Drawing.Common" Version="8.0.24" />
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
</Project>
